// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class ContentBrowser : ModuleRules
{
	public ContentBrowser(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
                "MainFrame",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
			    "AssetDefinition",
				"AppFramework",
				"Core",
				"CoreUObject",
				"Documentation",
				"DeveloperSettings",
				"ApplicationCore",
                "InputCore",
				"EditorConfig",
				"Engine",
				"Slate",
				"SlateCore",
                "EditorStyle",
				"AssetTools",
				"ContentBrowserData",
				"SourceControl",
				"SourceControlWindows",
				"WorkspaceMenuStructure",
				"EditorFramework",
				"UnrealEd",
				"EditorWidgets",
				"Projects",
				"AddContentDialog",
				"DesktopPlatform",
				"AssetRegistry",
				"AssetTagsEditor",
				"ToolMenus",
				"StatusBar",
				"ToolWidgets",
				"TelemetryUtils",
				"TypedElementFramework"
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"PropertyEditor",
				"PackagesDialog",
				"CollectionManager",
				"GameProjectGeneration",
                "MainFrame"
			}
		);
		
		PublicIncludePathModuleNames.AddRange(
            new string[] {
				"AssetTools",
				"CollectionManager",
				"ContentBrowserData",
            }
        );
	}
}
