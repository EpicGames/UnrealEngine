// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/ARFilter.h"
#include "AssetTypeCategories.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDelegates.h"
#include "AssetViewContentSources.h"
#include "HAL/Platform.h"
#include "IAssetTypeActions.h"
#include "IContentBrowserSingleton.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FFilter_HideOtherDevelopers;
class FUICommandList;
class SAssetSearchBox;
class SAssetView;
class SComboButton;
class SFilterList;
class SWidget;
struct FAssetData;
struct FContentBrowserItem;
struct FGeometry;
struct FKeyEvent;

enum class ECheckBoxState : uint8;

/**
 * Small content browser designed to allow for asset picking
 */
class SAssetPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SAssetPicker ){}

		/** A struct containing details about how the asset picker should behave */
		SLATE_ARGUMENT(FAssetPickerConfig, AssetPickerConfig)

	SLATE_END_ARGS()

	virtual ~SAssetPicker();

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

	// SWidget implementation
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	// End of SWidget implementation

	/** Return the associated AssetView */
	const TSharedPtr<SAssetView>& GetAssetView() const { return AssetViewPtr; }

	/** Function that's called when someone wants to execute a Rename Command*/
	void ExecuteRenameCommand();

	TSharedPtr<SWidget> GetSearchBox() const;

	/** Clears the search box text programmatically. Triggers OnSearchTextChanged -> OnSearchTextChangedDelegate. */
	CONTENTBROWSER_API void ClearSearchText();

	/** Returns the filter list widget, or nullptr if none was created. */
	CONTENTBROWSER_API TSharedPtr<SFilterList> GetFilterList() const;

	/** Syncs the AssetPicker to select the provided items. */
	void SyncToItems(TArrayView<const FContentBrowserItem> ItemsToSync, const bool bAllowImplicitSync = false, const bool bDisableFiltersThatHideAssets = true);

private:
	/** Focuses the search box post-construct */
	EActiveTimerReturnType SetFocusPostConstruct( double InCurrentTime, float InDeltaTime );

	/** Special case handling for SAssetSearchBox key commands */
	FReply HandleKeyDownFromSearchBox(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	void FolderEntered(const FString& FolderPath);

	/** Called when the editable text needs to be set or cleared */
	void SetSearchBoxText(const FText& InSearchText);

	/** Called by the editable text control when the search text is changed by the user */
	void OnSearchBoxChanged(const FText& InSearchText);

	/** Called by the editable text control when the user commits a text change */
	void OnSearchBoxCommitted(const FText& InSearchText, ETextCommit::Type CommitInfo);

	/** Called from external code to set the filter after the widget was created */
	void SetNewBackendFilter(const FARFilter& NewFilter);

	/** Called when the user changes filters */
	void OnFilterChanged();

	/** Handler for when the "None" button is clicked */
	FReply OnNoneButtonClicked();

	/** Handler for when a new item is requested in the asset view */
	void HandleNewItemRequested(const FContentBrowserItem& NewItem);

	/** Handle forwarding picking events. We wrap OnAssetSelected here to prevent 'Direct' selections being identified as user actions */
	void HandleItemSelectionChanged(const FContentBrowserItem& InSelectedItem, ESelectInfo::Type InSelectInfo);

	/** Handler for when the user double clicks, presses enter, or presses space on an asset */
	void HandleItemsActivated(TArrayView<const FContentBrowserItem> ActivatedItems, EAssetTypeActivationMethod::Type ActivationMethod);

	/** Handler for when the user drags an asset */
	void HandleItemsDragged(TArrayView<const FContentBrowserItem> DraggedItems);

	/** Selects the paths containing the specified assets. */
	void SyncToAssets(const TArray<FAssetData>& AssetDataList);

	/** @return The currently selected asset */
	TArray< FAssetData > GetCurrentSelection();

	/** Forces a refresh */
	void RefreshAssetView(bool bRefreshSources);

	/** @return The text to highlight on the assets  */
	FText GetHighlightedText() const;

	/** @return The tooltip for the other developers filter button depending on checked state.  */
	FText GetShowOtherDevelopersToolTip() const;

	/** Toggles the filter for showing other developers assets */
	void HandleShowOtherDevelopersCheckStateChanged( const ECheckBoxState InCheckboxState );

	/** Gets if showing other developers assets */
	ECheckBoxState GetShowOtherDevelopersCheckState() const;

	/** Returns whether the ShowOtherDevelopersFilter is active i.e. if we should show content belonging to other developers */
	bool IsShowingOtherDevelopersContent() const;

	/** Called upon the Rename UICommand being executed, sends rename request to the asset view */
	void OnRenameRequested() const;

	/** Returns true if the user is able to execute a rename request */
	bool CanExecuteRenameRequested();

	/** Returns true if the user is able to execute filter by selection request */
	bool CanFilterBySelection();

	/** Executes command and applies filters based on active selection */
	void OnFilterBySelection();

	/** Bind our UI commands */
	void BindCommands();

	/** Loads settings for this asset picker if SaveSettingsName was set */
	void LoadSettings();

	/** Saves settings for this asset picker if SaveSettingsName was set */
	void SaveSettings() const;

	void HandleSearchSettingsChanged();

	/** Handler for the context menu for asset or folder items */
	TSharedPtr<SWidget> GetItemContextMenu(TArrayView<const FContentBrowserItem> SelectedItems);

	/** Converts an asset type category into an asset category path, if possible. This is done because the asset picker config still uses asset type categories. */
	TOptional<FAssetCategoryPath> ConvertAssetTypeCategoryToAssetCategoryPath(EAssetTypeCategories::Type DefaultFilterMenuExpansion);

private:

	/** The list of FrontendFilters currently applied to the asset view */
	TSharedPtr<FAssetFilterCollectionType> FrontendFilters;

	/** The asset view widget */
	TSharedPtr<SAssetView> AssetViewPtr;

	/** The search box */
	TSharedPtr<SAssetSearchBox> SearchBoxPtr;

	/** The filter list */
	TSharedPtr<SFilterList> FilterListPtr;

	/** The filter combo button */
	TSharedPtr<SComboButton> FilterComboButtonPtr;

	/** Called to when an asset is selected or the none button is pressed */
	FOnAssetSelected OnAssetSelected;

	/** Called to when an asset is double clicked */
	FOnAssetDoubleClicked OnAssetDoubleClicked;

	/** Called when enter is pressed while an asset is selected */
	FOnAssetEnterPressed OnAssetEnterPressed;

	/** Called when any number of assets are activated */
	FOnAssetsActivated OnAssetsActivated;

	/** Called any number of assets are dragged */
	FOnAssetsDragged OnAssetsDragged;

	/** Called when a folder is entered in the asset view */
	FOnPathSelected OnFolderEnteredDelegate;

	/** Fired when the search box text changes, forwarded from FAssetPickerConfig::OnSearchTextChanged. */
	FOnSearchTextChanged OnSearchTextChangedDelegate;

	/** Delegate to invoke when a context menu for an asset is opening. */
	FOnGetAssetContextMenu OnGetAssetContextMenu;

	/** Delegate to invoke when a context menu for a folder is opening. */
	FOnGetFolderContextMenu OnGetFolderContextMenu;

	/** True if the search box will take keyboard focus next frame */
	bool bPendingFocusNextFrame;

	/** Filters needed for filtering the assets */
	TSharedPtr< FAssetFilterCollectionType > FilterCollection;
	TSharedPtr<class FAssetTextFilter> TextFilter;
	TSharedPtr<FFilter_HideOtherDevelopers> HideOtherDevelopersFilter;

	EAssetTypeCategories::Type DefaultFilterMenuExpansion;

	/** The sources data currently used by the picker */
	FAssetViewContentSources CurrentContentSources;

	/** Current filter we are using, needed reset asset view after we have custom filtered */
	FARFilter CurrentBackendFilter;

	/** UICommand list, holds list of actions for processing */
	TSharedPtr< FUICommandList > Commands;

	/** If set, view settings will be saved and loaded for the asset view using this name in ini files */
	FString SaveSettingsName;

	/** If set, uses this name for the filter bar identifier instead of the SaveSettingsName */
	TOptional<FString> FilterBarSaveSettingsNameOverride;

	/** Wether to allow renaming of items */
	bool bAllowRename;
};
