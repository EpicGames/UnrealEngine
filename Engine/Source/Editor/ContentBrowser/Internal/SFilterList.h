// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeCategories.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Filters/CustomTextFilters.h"
#include "Filters/FilterBase.h"
#include "Filters/SAssetFilterBar.h"
#include "Filters/SBasicFilterBar.h"
#include "FrontendFilters.h"
#include "HAL/Platform.h"
#include "IContentBrowserSingleton.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FFrontendFilter;
class SWidget;
class UClass;
class UToolMenu;
struct FContentBrowserItem;
struct FCustomTextFilterState;
struct FFilterBarSettings;
struct FGeometry;
struct FPointerEvent;

enum class ECheckBoxState : uint8;

/**
 * A list of filters currently applied to an asset view.
 */
class SFilterList : public SAssetFilterBar<FAssetFilterType>
{
	using Super = SAssetFilterBar<FAssetFilterType>; 
public:
	/**
	 * An event delegate that is executed when a custom text filter has been created/modified/deleted in any FilterList
	 * that is using shared settings.
	 */
	DECLARE_EVENT_OneParam(SFilterList, FCustomTextFilterEvent, TSharedPtr<SWidget> /* BroadcastingFilterList */);
	
	using FOnFilterChanged = typename SAssetFilterBar<FAssetFilterType>::FOnFilterChanged;

	SLATE_BEGIN_ARGS( SFilterList )
	: _UseSharedSettings(false)
	, _FilterBarLayout(EFilterBarLayout::Horizontal)
	, _CanChangeOrientation(false)
	, _DefaultMenuExpansionCategory(EAssetCategoryPaths::Basic)
	, _bUseSectionsForCustomCategories(false)
	{
		
	}
		/** Delegate for when filters have changed */
		SLATE_EVENT( FOnFilterChanged, OnFilterChanged )

		/** Delegate that lets the user modify the menu after the fact */
		SLATE_EVENT(FOnExtendAddFilterMenu, OnExtendAddFilterMenu)
	
		/** The filter collection used to further filter down assets returned from the backend */
		SLATE_ARGUMENT( TSharedPtr<FAssetFilterCollectionType>, FrontendFilters)

		/** An array of classes to filter the menu by */
		SLATE_ARGUMENT( TArray<UClass*>, InitialClassFilters)

		/** Custom front end filters to be displayed */
		SLATE_ARGUMENT( TArray< TSharedRef<FFrontendFilter> >, ExtraFrontendFilters )
	
		/** A delegate to create a Text Filter for FilterType items. If provided, will allow creation of custom text filters
		 *  from the filter dropdown menu.
		 */
		SLATE_ARGUMENT(FCreateTextFilter, CreateTextFilter)
	
		/** A unique identifier for this filter bar needed to enable saving settings in a config file */
		SLATE_ARGUMENT(FName, FilterBarIdentifier)

		/** If true, CustomTextFilters are saved/loaded from a config shared between multiple SFilterLists
		 *	Currently used to sync all content browser custom text filters.
		 */
		SLATE_ARGUMENT(bool, UseSharedSettings)

		/** Called when the filter bar layout is changing, before the filters are added to the layout */
		SLATE_EVENT(FOnFilterBarLayoutChanging, OnFilterBarLayoutChanging)

		/** The layout that determines how the filters are laid out */
		SLATE_ARGUMENT(EFilterBarLayout, FilterBarLayout)
			
		/** If true, allow dynamically changing the orientation and saving in the config */
		SLATE_ARGUMENT(bool, CanChangeOrientation)

		/** Expands the specified asset category, if specified. If not, it will expand Basic/Common instead. */
		SLATE_ARGUMENT(TOptional<FAssetCategoryPath>, DefaultMenuExpansionCategory)

		/** If true, adds custom categories as sections (expanded) vs. as sub-menus */
		SLATE_ARGUMENT(bool, bUseSectionsForCustomCategories)

		/** The command list provided to the filter context menu */
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

	/** Set the check box state of the specified frontend filter (in the filter drop down) and pin/unpin a filter widget on/from the filter bar. When a filter is pinned (was not already pinned), it is activated and deactivated when unpinned. */
	void SetFrontendFilterCheckState(const TSharedPtr<FFrontendFilter>& InFrontendFilter, ECheckBoxState CheckState);

	/** Returns the check box state of the specified frontend filter (in the filter drop down). This tells whether the filter is pinned or not on the filter bar, but not if filter is active or not. @see IsFrontendFilterActive(). */
	ECheckBoxState GetFrontendFilterCheckState(const TSharedPtr<FFrontendFilter>& InFrontendFilter) const;

	/** Returns true if the specified frontend filter is both checked (pinned on the filter bar) and active (contributing to filter the result). */
	bool IsFrontendFilterActive(const TSharedPtr<FFrontendFilter>& InFrontendFilter) const;

	/** Retrieve a specific frontend filter */
	TSharedPtr<FFrontendFilter> GetFrontendFilter(const FString& InName) const;
	
	/** 
	 * Replaces super version of GetCombinedBackendFilter to allow returning FPathPermissionLists as well.
	 * Permission lists will be combined into a content browser data filter after a delay. Filters may modify them 
	 * via the shared ref but should not do so off the game thread.
	 */
	FARFilter GetCombinedBackendFilter(TArray<TSharedRef<const FPathPermissionList>>& OutPermissionLists) const;
	FARFilter GetCombinedBackendFilter() const = delete;
	
	/** Handler for when the floating add filter button was clicked */
	TSharedRef<SWidget> ExternalMakeAddFilterMenu();

	/** Disables any active filters that would hide the supplied items */
	void DisableFiltersThatHideItems(TArrayView<const FContentBrowserItem> ItemList);

	/** Returns the class filters specified at construction using argument 'InitialClassFilters'. */
	const TArray<UClass*>& GetInitialClassFilters();

	/** Open the dialog to create a custom filter from the given text */
	void CreateCustomFilterDialog(const FText& InText);
	
	/** Updates bIncludeClassName, bIncludeAssetPath and bIncludeCollectionNames for all custom text filters */
	void UpdateCustomTextFilterIncludes(const bool InIncludeClassName, const bool InIncludeAssetPath, const bool InIncludeCollectionNames);

	/** Add a custom widget to the filter bar alongside the filters */
	void AddWidgetToCurrentLayout(TSharedRef<SWidget> InWidget);

	using Super::SaveSettings;
	virtual void LoadSettings() override;

	/* Copy the settings from a specific instance name */
	void LoadSettings(const FName& InInstanceName);

	/** Helper functions for backwards compatibility with filters that save state until they are ported to EditorConfig */
	void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString);
	void LoadSettings(const FName& InInstanceName, const FString& IniFilename, const FString& IniSection, const FString& SettingsString);

	/** Handler for when a custom text filter is created */
	virtual void OnCreateCustomTextFilter(const FCustomTextFilterData& InFilterData, bool bApplyFilter) override;

	/** Deletes the custom text filter with the matching label */
	void DeleteCustomTextFilterByLabel(const FText& FilterLabel);

	/** Modifies the custom text filter with the matching label */
	void ModifyCustomTextFilterByLabel(const FCustomTextFilterData& NewFilterData, const FText& FilterLabel);

	/** Whether the filters are being applied to paths recursively */
	bool IsFilteringPathsRecursively() const;

	/** Sets whether the filters are being applied to paths recursively */
	void SetFilterPathsRecursively(const bool bInFilterRecursively);

	/** Toggle whether the filters are being applied to paths recursively */
	void ToggleFilterPathsRecursively();

	/** Toggle whether the filters are sorted and displayed alphabetically */
	void ToggleSortFiltersAlphabetically();

	/** Returns a pointer to the active filter container widget (Horizontal or Vertical), nullptr if neither is active */
	TSharedPtr<SWidget> GetActiveFilterContainer() const;

	/** Returns all frontend filters registered with this filter bar (built-in, extra, and extension-provided). */
	const TArray<TSharedRef<FFrontendFilter>>& GetAllFrontendFilters() const { return AllFrontendFilters_Internal; }

	/** Returns all custom class/type filters registered with this filter bar. */
	const TArray<TSharedRef<FCustomClassFilterData>>& GetCustomClassFilters() const { return CustomClassFilters; }

	/**
	 * Activates a class filter by class path:
	 * - If not yet pinned: pins and enables it.
	 * - If already pinned (e.g. disabled): enables it in-place without re-pinning.
	 */
	CONTENTBROWSER_API void ActivateClassFilterByPath(const FTopLevelAssetPath& ClassPath);

protected:
	
	/** Handler for when the add filter button was clicked */
	//virtual TSharedRef<SWidget> MakeAddFilterMenu() override;

	virtual void SaveSettingsInternal(FFilterBarSettings* InSettingsToSave) override;
	
	/** Handler for when a custom text filter is modified */
	virtual void OnModifyCustomTextFilter(const FCustomTextFilterData& InFilterData, TSharedPtr<ICustomTextFilter<FAssetFilterType>> InFilter) override;

	/** Handler for when a custom text filter is deleted */
	virtual void OnDeleteCustomTextFilter(const TSharedPtr<ICustomTextFilter<FAssetFilterType>> InFilter) override;

private:

	// /** Exists for backwards compatibility with ExternalMakeAddFilterMenu */
	// TSharedRef<SWidget> MakeAddFilterMenu(FAssetCategoryPath MenuExpansion = EAssetCategoryPaths::Basic);

	virtual UAssetFilterBarContext* CreateAssetFilterBarContext() override;

	/** Handler to Populate the Add Filter Menu. Use OnFilterAssetType in Subclasses to add classes to the exclusion list */
	virtual void PopulateAddFilterMenu(UToolMenu* Menu, TSharedPtr<FFilterCategory> MenuExpansion, FOnFilterAssetType OnFilterAssetType) override;
	
	/** Handler for when another SFilterList using shared settings creates a custom text filter */
	void OnExternalCustomTextFilterCreated(TSharedPtr<SWidget> BroadcastingFilterList);
	
	/** Empty our list of custom text filters, and load from the given config */
	void LoadCustomTextFilters(const FFilterBarSettings* FilterBarConfig);

	/** Find the custom text filter corresponding to the specified state, and restore it's state to what is specified
	 *  @return True if the filter was restored successfully, false if not
	 */
	bool RestoreCustomTextFilterState(const FCustomTextFilterState& InFilterState);
	
private:
	/** List of classes that our filters must match */
	TArray<UClass*> InitialClassFilters;

	/** Delegate for when filters have changed */
	FOnFilterChanged OnFilterChanged;

	/** A reference to AllFrontEndFilters so we can access the filters as FFrontEndFilter instead of FFilterBase<FAssetFilterType> */
	TArray< TSharedRef<FFrontendFilter> > AllFrontendFilters_Internal;

	/** An identifier shared by all SFilterLists, used to save and load settings common to every instance */
	static const FName SharedIdentifier;

	/** If bIncludeClassName is true, custom text filters will include an asset's class name in the search */
	bool bIncludeClassName;

	/** If bIncludeAssetPath is true, custom text filters will match against full Asset path */
	bool bIncludeAssetPath;

	/** If bIncludeCollectionNames is true, custom text filters will match against collection names as well */
	bool bIncludeCollectionNames;

	/** Whether this FilterList wants to load/save from the settings common to all instances */
	bool bUseSharedSettings;

	/** Whether this FilterList applies to paths recursively */
	bool bFilterPathsRecursively = true;

	/** The event that executes when a custom text filter is created/modified/deleted in any filter list that is using Shared Settings */
	static FCustomTextFilterEvent CustomTextFilterEvent;
};

/* A custom implementation of ICustomTextFilter that uses FFrontendFilter_Text to handle comparing items in the
 * Asset View to Custom Text Filters. This ensures that the advanced search syntax etc behaves properly when used
 * by a custom text filter created by saving a search.
 * This is implemented as a fake frontend filter which modifies background asynchronous text filtering behavior in the asset view.
 */
class FFrontendFilter_CustomText :
	public ICustomTextFilter<FAssetFilterType>,
	public FFrontendFilter,
	public TSharedFromThis<FFrontendFilter_CustomText>
{
public:
	FFrontendFilter_CustomText();

	// FFrontendFilter implementation
	virtual FString GetName() const override;
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTipText() const override;
	virtual FLinearColor GetColor() const override;
	virtual TOptional<FText> GetAsCustomTextFilter();
	virtual bool PassesFilter(FAssetFilterType InItem) const override { return true; }
	
	/** Updates bIncludeClassName, bIncludeAssetPath and bIncludeCollectionNames for this filter */
	void UpdateCustomTextFilterIncludes(const bool InIncludeClassName, const bool InIncludeAssetPath, const bool InIncludeCollectionNames);

	//ICustomTextFilter interface
	
	/** Set the internals of this filter from an FCustomTextFilterData */
	virtual void SetFromCustomTextFilterData(const FCustomTextFilterData& InFilterData) override;

	/** Create an FCustomTextFilterData from the internals of this filter */
	virtual FCustomTextFilterData CreateCustomTextFilterData() const override;

	/** Get the actual filter */
	virtual TSharedPtr<FFilterBase<FAssetFilterType>> GetFilter() override;

protected:

	/* The Display Name of this custom filter that the user sees */
	FText DisplayName;

	/** The raw text of the query, either simple string or advanced search syntax. */
	FText RawFilterText;

	/* The Color of this filter pill */
	FLinearColor Color;

	bool bIncludeClassName = false;
	bool bIncludeAssetPath = false;
	bool bIncludeCollectionNames = false;
};