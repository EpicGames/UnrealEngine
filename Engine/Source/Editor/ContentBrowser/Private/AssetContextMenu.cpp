// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetContextMenu.h"

#include "Algo/Transform.h"
#include "AssetDefinition.h"
#include "AssetDefinitionRegistry.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "CollectionAssetManagement.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserCommands.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserItemData.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "ContentBrowserSingleton.h"
#include "ContentBrowserStyle.h"
#include "ContentBrowserUtils.h"
#include "Engine/AssetManager.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/SlateDelegates.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformStackWalk.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "ICollectionSource.h"
#include "Internationalization/Internationalization.h"
#include "Logging/MessageLog.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/NamePermissionList.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/StringBuilder.h"
#include "Misc/WarnIfAssetsLoadedInScope.h"
#include "Modules/ModuleManager.h"
#include "SAssetView.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "TelemetryRouter.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FMenuBuilder;
class SWidget;

#define LOCTEXT_NAMESPACE "ContentBrowser"

namespace AssetContextMenu
{
	bool IsEpicInternalAssetFeatureEnabled()
	{
		static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("AssetTools.EnableEpicInternalAssetFeature"));
		return ensure(CVar) ? CVar->GetBool() : false;
	}

	static bool bAlwaysShowSetAssetScopeMenus = false;
	static FAutoConsoleVariableRef CVarAlwaysShowSetAssetScopeMenus(
		TEXT("ContentBrowser.AlwaysShowSetAssetScopeMenus"),
		bAlwaysShowSetAssetScopeMenus,
		TEXT("Enables always showing menu entries to change asset referencable scope (False: disabled, True: enabled")
	);

	bool IsAlwaysShowSetAssetScopeMenus()
	{
		return bAlwaysShowSetAssetScopeMenus;
	}

	bool GetAssetAccessSpecifierFromItem(const FContentBrowserItem& InContentBrowserItem, EAssetAccessSpecifier& OutAssetAccessSpecifier)
	{
		FAssetData ItemAssetData;
		if (InContentBrowserItem.Legacy_TryGetAssetData(ItemAssetData))
		{
			OutAssetAccessSpecifier = ItemAssetData.GetAssetAccessSpecifier();
			return true;
		}

		return false;
	}
}

FAssetContextMenu::FAssetContextMenu(const TWeakPtr<SAssetView>& InAssetView)
	: AssetView(InAssetView)
{
}

void FAssetContextMenu::BindCommands(TSharedPtr< FUICommandList >& Commands)
{
	Commands->MapAction(FGenericCommands::Get().Duplicate, FUIAction(
		FExecuteAction::CreateSP(this, &FAssetContextMenu::ExecuteDuplicate),
		FCanExecuteAction::CreateSP(this, &FAssetContextMenu::CanExecuteDuplicate)
		));

	Commands->MapAction(FGlobalEditorCommonCommands::Get().FindInContentBrowser, FUIAction(
		FExecuteAction::CreateSP(this, &FAssetContextMenu::ExecuteSyncToAssetTree),
		FCanExecuteAction::CreateSP(this, &FAssetContextMenu::CanExecuteSyncToAssetTree)
		));

	if (const TSharedPtr<SAssetView>& AssetViewPinned = AssetView.Pin())
	{
		const SAssetView* AssetViewPtr = AssetViewPinned.Get();

		Commands->MapAction(FGenericCommands::Get().Copy, FUIAction(
			FExecuteAction::CreateSP(AssetViewPtr, &SAssetView::ExecuteCopy, EAssetViewCopyType::ExportTextPath)
		));

		Commands->MapAction(FContentBrowserCommands::Get().AssetViewCopyObjectPath, FUIAction(
			FExecuteAction::CreateSP(AssetViewPtr, &SAssetView::ExecuteCopy, EAssetViewCopyType::ObjectPath)
		));

		Commands->MapAction(FContentBrowserCommands::Get().AssetViewCopyPackageName, FUIAction(
			FExecuteAction::CreateSP(AssetViewPtr, &SAssetView::ExecuteCopy, EAssetViewCopyType::PackageName)
		));
	}
}

TSharedRef<SWidget> FAssetContextMenu::MakeContextMenu(TArrayView<const FContentBrowserItem> InSelectedItems, const FAssetViewContentSources& InContentSources, TSharedPtr< FUICommandList > InCommandList)
{
	FWarnIfAssetsLoadedInScope WarnIfAssetsLoaded;
	
	SetSelectedItems(InSelectedItems);
	ContentSources = InContentSources;

	// Cache any vars that are used in determining if you can execute any actions.
	// Useful for actions whose "CanExecute" will not change or is expensive to calculate.
	CacheCanExecuteVars();

	// Get all menu extenders for this context menu from the content browser module
	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>( TEXT("ContentBrowser") );
	TArray<FContentBrowserMenuExtender_SelectedAssets> MenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();

	TSharedPtr<FExtender> MenuExtender;
	{
		TArray<FAssetData> SelectedAssets;
		for (const FContentBrowserItem& SelectedFile : SelectedFiles)
		{
			FAssetData ItemAssetData;
			if (SelectedFile.Legacy_TryGetAssetData(ItemAssetData))
			{
				SelectedAssets.Add(ItemAssetData);
			}
		}

		if (SelectedAssets.Num() > 0)
		{
			TArray<TSharedPtr<FExtender>> Extenders;
			for (int32 i = 0; i < MenuExtenderDelegates.Num(); ++i)
			{
				if (MenuExtenderDelegates[i].IsBound())
				{
					Extenders.Add(MenuExtenderDelegates[i].Execute(SelectedAssets));
				}
			}
			MenuExtender = FExtender::Combine(Extenders);
		}
	}

	UContentBrowserAssetContextMenuContext* ContextObject = NewObject<UContentBrowserAssetContextMenuContext>();
	ContextObject->AssetContextMenu = SharedThis(this);

	UToolMenus* ToolMenus = UToolMenus::Get();

	static const FName BaseMenuName("ContentBrowser.AssetContextMenu");
	static const FName ItemContextMenuName("ContentBrowser.ItemContextMenu");
	RegisterContextMenu(BaseMenuName);
	
	// Create menu hierarchy based on class hierarchy
	FName MenuName = BaseMenuName;
	{
		// TODO: Ideally all of this asset specific stuff would happen in the asset data source, however we 
		// need to keep it here for now to build the correct menu name and register the correct extenders

		// Objects must be loaded for this operation... for now
		UContentBrowserDataSource* CommonDataSource = nullptr;
		{
			TArray<FAssetData> SelectedAssets;
			bool bKeepCheckingCommonDataSource = true;
			for (const FContentBrowserItem& SelectedItem : SelectedItems)
			{
				if (bKeepCheckingCommonDataSource)
				{
					if (const FContentBrowserItemData* PrimaryInternalItem = SelectedItem.GetPrimaryInternalItem())
					{
						if (UContentBrowserDataSource* OwnerDataSource = PrimaryInternalItem->GetOwnerDataSource())
						{
							if (CommonDataSource == nullptr)
							{
								CommonDataSource = OwnerDataSource;
							}
							else if (CommonDataSource != OwnerDataSource)
							{
								CommonDataSource = nullptr;
								bKeepCheckingCommonDataSource = false;
							}
						}
					}
				}

				FAssetData ItemAssetData;
				if (SelectedItem.Legacy_TryGetAssetData(ItemAssetData))
				{
					SelectedAssets.Add(MoveTemp(ItemAssetData));
				}
			}

			ContextObject->bCanBeModified = SelectedAssets.Num() == 0;
			ContextObject->SelectedAssets = MoveTemp(SelectedAssets);
		}
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		const TSharedRef<FPathPermissionList>& WritableFolderPermission = AssetToolsModule.Get().GetWritableFolderPermissionList();
		
		ContextObject->bContainsUnsupportedAssets = false;
		for (const FContentBrowserItem& SelectedItem : SelectedItems)
		{
			if(!SelectedItem.IsSupported())
			{
				ContextObject->bContainsUnsupportedAssets = true;
				break;
			}
		}

		const TArray<FAssetData>& SelectedAssets = ContextObject->SelectedAssets;

		if (SelectedAssets.Num() > 0 && SelectedAssets.Num() == SelectedItems.Num())
		{

			// Find common class for selected objects
			UClass* CommonClass = nullptr;
			for (int32 ObjIdx = 0; ObjIdx < SelectedAssets.Num(); ++ObjIdx)
			{
				if (CommonClass == nullptr)
				{
					CommonClass = UAssetRegistryHelpers::FindAssetNativeClass(SelectedAssets[ObjIdx]);
					continue;
				}

				// Update the CommonClass until we find a common shared class, ignore anything that's not native.
				UClass* Class = UAssetRegistryHelpers::FindAssetNativeClass(SelectedAssets[ObjIdx]);
				while (Class && !Class->IsChildOf(CommonClass))
				{
					CommonClass = CommonClass->GetSuperClass();
				}
			}
			ContextObject->CommonClass = CommonClass;

			ContextObject->bCanBeModified = true;
			ContextObject->bHasCookedPackages = false;
			for (const FAssetData& SelectedAsset : SelectedAssets)
			{
				if (SelectedAsset.HasAnyPackageFlags(PKG_Cooked | PKG_FilterEditorOnly))
				{
					ContextObject->bCanBeModified = false;
					ContextObject->bHasCookedPackages = true;
					break;
				}

				if (WritableFolderPermission->HasFiltering() && !WritableFolderPermission->PassesStartsWithFilter(SelectedAsset.PackageName))
				{
					ContextObject->bCanBeModified = false;
					break;
				}

				if (const UClass* AssetClass = SelectedAsset.GetClass())
				{
					if (AssetClass->IsChildOf<UClass>())
					{
						ContextObject->bCanBeModified = false;
						break;
					}
				}
			}

			// We can have a null common class if an asset is from unloaded plugin or an missing class.
			if (CommonClass)
			{
				MenuName = UToolMenus::JoinMenuPaths(BaseMenuName, CommonClass->GetFName());

				RegisterMenuHierarchy(CommonClass);

				// Find asset actions for common class
				ContextObject->CommonAssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(ContextObject->CommonClass);
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				ContextObject->CommonAssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(ContextObject->CommonClass).Pin();
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}
		else
		{
			if (CommonDataSource)
			{
				ContextObject->bCanBeModified = true;
				ContextObject->bHasCookedPackages = false;

				if (WritableFolderPermission->HasFiltering())
				{
					for (const FContentBrowserItem& SelectedItem : SelectedItems)
					{
						if (!WritableFolderPermission->PassesStartsWithFilter(SelectedItem.GetInternalPath()))
						{
							ContextObject->bCanBeModified = false;
							break;
						}
					}
				}

				for (const FAssetData& SelectedAsset : SelectedAssets)
				{
					if (SelectedAsset.HasAnyPackageFlags(PKG_Cooked | PKG_FilterEditorOnly))
					{
						ContextObject->bCanBeModified = false;
						ContextObject->bHasCookedPackages = true;
						break;
					}

					if (const UClass* AssetClass = SelectedAsset.GetClass())
					{
						if (AssetClass->IsChildOf<UClass>())
						{
							ContextObject->bCanBeModified = false;
							break;
						}
					}
				}

				MenuName = UToolMenus::JoinMenuPaths(ItemContextMenuName, CommonDataSource->GetFName());

				if (!ToolMenus->IsMenuRegistered(MenuName))
				{
					ToolMenus->RegisterMenu(MenuName, BaseMenuName);
				}
			}
		}
	}

	ContextObject->bCanView = false;
	if (!SelectedItems.IsEmpty())
	{
		for (const FContentBrowserItem& SelectedItem : SelectedItems)
		{
			if (SelectedItem.CanView())
			{
				ContextObject->bCanView = true;
				break;
			}
		}
	}

	FToolMenuContext MenuContext(InCommandList, MenuExtender, ContextObject);

	{
		UContentBrowserDataMenuContext_FileMenu* DataContextObject = NewObject<UContentBrowserDataMenuContext_FileMenu>();
		DataContextObject->SelectedItems = SelectedItems;
		DataContextObject->CollectionSources = ContentSources.GetCollections();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Fill out deprecated SelectedCollections with game project collections for backwards compatibility.
		Algo::TransformIf(
			DataContextObject->CollectionSources,
			DataContextObject->SelectedCollections,
			[](const FCollectionRef& Collection) { return Collection.Container == FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(); },
			[](const FCollectionRef& Collection) { return FCollectionNameType(Collection.Name, Collection.Type); });
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		DataContextObject->bCanBeModified = ContextObject->bCanBeModified;
		DataContextObject->bCanView = ContextObject->bCanView;
		DataContextObject->bHasCookedPackages = ContextObject->bHasCookedPackages;
		DataContextObject->bContainsUnsupportedAssets = ContextObject->bContainsUnsupportedAssets;
		DataContextObject->ParentWidget = AssetView;
		DataContextObject->OnShowInPathsView = OnShowInPathsViewRequested;
		DataContextObject->OnRefreshView = OnAssetViewRefreshRequested;
		MenuContext.AddObject(DataContextObject);
	}

	return ToolMenus->GenerateWidget(MenuName, MenuContext);
}

void FAssetContextMenu::RegisterMenuHierarchy(UClass* InClass)
{
	static const FName BaseMenuName("ContentBrowser.AssetContextMenu");

	UToolMenus* ToolMenus = UToolMenus::Get();

	for (UClass* CurrentClass = InClass; CurrentClass; CurrentClass = CurrentClass->GetSuperClass())
	{
		FName CurrentMenuName = UToolMenus::JoinMenuPaths(BaseMenuName, CurrentClass->GetFName());
		if (!ToolMenus->IsMenuRegistered(CurrentMenuName))
		{
			FName ParentMenuName;
			UClass* ParentClass = CurrentClass->GetSuperClass();
			if (ParentClass == UObject::StaticClass() || ParentClass == nullptr)
			{
				ParentMenuName = BaseMenuName;
			}
			else
			{
				ParentMenuName = UToolMenus::JoinMenuPaths(BaseMenuName, ParentClass->GetFName());
			}

			ToolMenus->RegisterMenu(CurrentMenuName, ParentMenuName);

			if (ParentMenuName == BaseMenuName)
			{
				break;
			}
		}
	}
}

void FAssetContextMenu::RegisterContextMenu(const FName MenuName)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* Menu = ToolMenus->RegisterMenu(MenuName);
		FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");

		// TODO Remove when IAssetTypeActions is dead or fully deprecated.
		{
			// Note: Do  not use "GetActions" again when copying this code, otherwise "GetActions" menu entry will be overwritten
			Section.AddDynamicEntry("GetActions", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
					if (Context && Context->CommonAssetTypeActions.IsValid() && Context->CommonAssetTypeActions.Pin()->ShouldCallGetActions())
					{
						TArray<UObject*> SelectedObjects = Context->LoadSelectedObjectsIfNeeded();						
						//  It's possible for an unloaded object to be selected if the content browser is out of date, in that case it is unnecessary to call `GetActions`
						if (SelectedObjects.Num() > 0)
						{
							Context->CommonAssetTypeActions.Pin()->GetActions(SelectedObjects, InSection);
						}
					}
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}));

			Section.AddDynamicEntry("GetActionsLegacy", FNewToolMenuDelegateLegacy::CreateLambda([](FMenuBuilder& MenuBuilder, UToolMenu* InMenu)
			{
				UContentBrowserAssetContextMenuContext* Context = InMenu->FindContext<UContentBrowserAssetContextMenuContext>();
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
					if (Context && Context->CommonAssetTypeActions.IsValid() && Context->CommonAssetTypeActions.Pin()->ShouldCallGetActions())
					{
						TArray<UObject*> SelectedObjects = Context->LoadSelectedObjectsIfNeeded();
						if (SelectedObjects.Num() > 0)
						{
							Context->CommonAssetTypeActions.Pin()->GetActions(SelectedObjects, MenuBuilder);
						}
					}
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}));
		}		

		Menu->AddDynamicSection("AddMenuOptions", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			const UContentBrowserAssetContextMenuContext* Context = InMenu->FindContext<UContentBrowserAssetContextMenuContext>();
			if (Context && Context->AssetContextMenu.IsValid())
			{
				Context->AssetContextMenu.Pin()->AddMenuOptions(InMenu);
			}
		}));
	}
}

void FAssetContextMenu::AddMenuOptions(UToolMenu* InMenu)
{
	// Add any type-specific context menu options
	AddAssetTypeMenuOptions(InMenu);

	// Add quick access to common commands.
	AddCommonMenuOptions(InMenu);

	// Add quick access to view commands
	AddExploreMenuOptions(InMenu);

	static const IConsoleVariable* EnablePublicAssetFeatureCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("AssetTools.EnablePublicAssetFeature"));
	if (EnablePublicAssetFeatureCVar && EnablePublicAssetFeatureCVar->GetBool())
	{
		AddPublicStateMenuOptions(InMenu);
	}

	// Add reference options
	AddReferenceMenuOptions(InMenu);

	// Add collection options
	AddCollectionMenuOptions(InMenu);
}

void FAssetContextMenu::SetSelectedItems(TArrayView<const FContentBrowserItem> InSelectedItems)
{
	SelectedItems.Reset();
	SelectedItems.Append(InSelectedItems.GetData(), InSelectedItems.Num());

	SelectedFiles.Reset();
	SelectedFolders.Reset();
	for (const FContentBrowserItem& SelectedItem : SelectedItems)
	{
		if (SelectedItem.IsFile())
		{
			SelectedFiles.Add(SelectedItem);
		}

		if (SelectedItem.IsFolder())
		{
			SelectedFolders.Add(SelectedItem);
		}
	}
}

void FAssetContextMenu::SetOnShowInPathsViewRequested(const FOnShowInPathsViewRequested& InOnShowInPathsViewRequested)
{
	OnShowInPathsViewRequested = InOnShowInPathsViewRequested;
}

void FAssetContextMenu::SetOnRenameRequested(const FOnRenameRequested& InOnRenameRequested)
{
	OnRenameRequested = InOnRenameRequested;
}

void FAssetContextMenu::SetOnDuplicateRequested(const FOnDuplicateRequested& InOnDuplicateRequested)
{
	OnDuplicateRequested = InOnDuplicateRequested;
}

void FAssetContextMenu::SetOnAssetViewRefreshRequested(const FOnAssetViewRefreshRequested& InOnAssetViewRefreshRequested)
{
	OnAssetViewRefreshRequested = InOnAssetViewRefreshRequested;
}

bool FAssetContextMenu::AddCommonMenuOptions(UToolMenu* Menu)
{
	UContentBrowserDataMenuContext_FileMenu* Context = Menu->FindContext<UContentBrowserDataMenuContext_FileMenu>();
	const bool bCanBeModified = !Context || Context->bCanBeModified;
	const bool bCanBeViewed = !Context || Context->bCanView;

	{
		FToolMenuSection& Section = Menu->AddSection("CommonAssetActions", LOCTEXT("CommonAssetActionsMenuHeading", "Common"));

		if (bCanBeModified || bCanBeViewed)
		{
			// Open/Edit Asset
			Section.AddMenuEntry(FContentBrowserCommands::Get().OpenAssetsOrFolders,
				GetEditAssetEditorLabel(bCanBeModified, bCanBeViewed),
				GetEditAssetEditorTooltip(bCanBeModified, bCanBeViewed),
				GetEditAssetEditorIcon(bCanBeModified, bCanBeViewed)
			);
		}

		if (bCanBeModified)
		{
			// Rename
			Section.AddMenuEntry(FGenericCommands::Get().Rename,
				LOCTEXT("Rename", "Rename"),
				LOCTEXT("RenameTooltip", "Rename the selected item.")
			);

			// Duplicate
			Section.AddMenuEntry(FGenericCommands::Get().Duplicate,
				LOCTEXT("Duplicate", "Duplicate"),
				LOCTEXT("DuplicateTooltip", "Create a copy of the selected item(s)."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate")
			);

			// Save
			Section.AddMenuEntry(FContentBrowserCommands::Get().SaveSelectedAsset,
				LOCTEXT("SaveAsset", "Save"),
				LOCTEXT("SaveAssetTooltip", "Saves the item to file."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Save")
			);

			// Delete
			Section.AddMenuEntry(FGenericCommands::Get().Delete,
				LOCTEXT("Delete", "Delete"),
				TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FAssetContextMenu::GetDeleteToolTip))
			);
		}
	}

	return true;
}

void FAssetContextMenu::AddExploreMenuOptions(UToolMenu* Menu)
{
	UContentBrowserDataMenuContext_FileMenu* Context = Menu->FindContext<UContentBrowserDataMenuContext_FileMenu>();

	FToolMenuSection& Section = Menu->AddSection("AssetContextExploreMenuOptions", LOCTEXT("AssetContextExploreMenuOptionsHeading", "Explore"));
	{
		// Find in Content Browser
		Section.AddMenuEntry(
			FGlobalEditorCommonCommands::Get().FindInContentBrowser, 
			LOCTEXT("ShowInFolderView", "Show in Folder View"),
			LOCTEXT("ShowInFolderViewTooltip", "Selects the folder that contains this asset in the Content Browser Sources Panel.")
			);

		if (!Context->bHasCookedPackages)
		{
			// Find in Explorer
			Section.AddMenuEntry(
				"FindInExplorer",
				ContentBrowserUtils::GetExploreFolderText(),
				LOCTEXT("FindInExplorerTooltip", "Finds this asset on disk"),
				FSlateIcon(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetStyleSetName(), "ContentBrowser.ShowInExplorer"),
				FUIAction(
					FExecuteAction::CreateSP(this, &FAssetContextMenu::ExecuteFindInExplorer),
					FCanExecuteAction::CreateSP(this, &FAssetContextMenu::CanExecuteFindInExplorer)
				)
			);
		}
	}
}

bool FAssetContextMenu::AddPublicStateMenuOptions(UToolMenu* Menu)
{
	bool bHasDisallowSetPublicReason = !DisallowSetPublicReason.IsEmpty();
	if (!(bHasDisallowSetPublicReason || bCanShowSetAssetAccessMenu || bCanExecuteSetPublicAsset || bCanExecuteSetEpicInternalAsset || bCanExecuteSetPrivateAsset || bCanExecuteBulkSetPublicAsset || bCanExecuteBulkSetEpicInternalAsset || bCanExecuteBulkSetPrivateAsset))
	{
		return false;
	}

	UContentBrowserDataMenuContext_FileMenu* Context = Menu->FindContext<UContentBrowserDataMenuContext_FileMenu>();
	{
		FToolMenuSection& Section = Menu->AddSection("AssetPublicState", LOCTEXT("PublicStateHandling", "Asset State"));

		if (SelectedFiles.Num() == 1)
		{
			if (bCanExecuteSetPublicAsset || bCanExecuteSetEpicInternalAsset || bCanExecuteSetPrivateAsset || bHasDisallowSetPublicReason || bCanShowSetAssetAccessMenu)
			{
				Section.AddMenuEntry(
					"PublicAsset",
					LOCTEXT("PublicAssetToggle", "Public Asset"),
					bHasDisallowSetPublicReason ? DisallowSetPublicReason : LOCTEXT("PublicAssetToggleTooltip", "Sets the asset to be referencable by other Plugins"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), FAssetViewItem::GetPublicAssetAccessSpecifierIconStyleName(TEXT(".Small"))),
					FUIAction(
						FExecuteAction::CreateSP(this, &FAssetContextMenu::ExecuteSetAssetAccessSpecifier, EAssetAccessSpecifier::Public),
						FCanExecuteAction::CreateSP(this, &FAssetContextMenu::CanSetAssetAccessSpecifier, EAssetAccessSpecifier::Public),
						FIsActionChecked::CreateSP(this, &FAssetContextMenu::IsSelectedAssetAccessSpecifier, EAssetAccessSpecifier::Public)
					),
					EUserInterfaceActionType::RadioButton
				);

				if (bCanExecuteSetEpicInternalAsset)
				{
					Section.AddMenuEntry(
						"EpicInternalAsset",
						LOCTEXT("SetInternalAsset", "Epic Internal Asset"),
						bHasDisallowSetPublicReason ? DisallowSetPublicReason : LOCTEXT("SetInternalAssetTooltip", "Sets the asset to be referencable by Epic internal plugins and mount points"),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), FAssetViewItem::GetEpicInternalAssetAccessSpecifierIconStyleName(TEXT(".Small"))),
						FUIAction(
							FExecuteAction::CreateSP(this, &FAssetContextMenu::ExecuteSetAssetAccessSpecifier, EAssetAccessSpecifier::EpicInternal),
							FCanExecuteAction::CreateSP(this, &FAssetContextMenu::CanSetAssetAccessSpecifier, EAssetAccessSpecifier::EpicInternal),
							FIsActionChecked::CreateSP(this, &FAssetContextMenu::IsSelectedAssetAccessSpecifier, EAssetAccessSpecifier::EpicInternal)
						),
						EUserInterfaceActionType::RadioButton
					);
				}

				Section.AddMenuEntry(
					"PrivateAsset",
					LOCTEXT("SetPrivateAsset", "Private Asset"),
					LOCTEXT("SetAssetPrivateTooltip", "Sets the asset so it can't be referenced by other Plugins"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), FAssetViewItem::GetPrivateAssetAccessSpecifierIconStyleName(TEXT(".Small"))),
					FUIAction(
						FExecuteAction::CreateSP(this, &FAssetContextMenu::ExecuteSetAssetAccessSpecifier, EAssetAccessSpecifier::Private),
						FCanExecuteAction::CreateSP(this, &FAssetContextMenu::CanSetAssetAccessSpecifier, EAssetAccessSpecifier::Private),
						FIsActionChecked::CreateSP(this, &FAssetContextMenu::IsSelectedAssetAccessSpecifier, EAssetAccessSpecifier::Private)
					),
					EUserInterfaceActionType::RadioButton
				);
			}
		}
		else if (SelectedFiles.Num() > 1)
		{
			if (bCanExecuteBulkSetPublicAsset || bCanExecuteBulkSetEpicInternalAsset || bCanExecuteBulkSetPrivateAsset)
			{
				Section.AddMenuEntry(
					"MarkSelectedAsPublic",
					LOCTEXT("MarkSelectedAsPublic", "Mark Selected As Public"),
					LOCTEXT("MarkSelectedAsPublicTooltip", "Sets all selected assets to be publicly available for reference by other plugins"),
					FSlateIcon(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetStyleSetName(), "ContentBrowser.AssetActions.PublicAssetToggle"),
					FUIAction(
						FExecuteAction::CreateSP(this, &FAssetContextMenu::ExecuteBulkSetAssetAccessSpecifier, EAssetAccessSpecifier::Public),
						FCanExecuteAction::CreateSP(this, &FAssetContextMenu::CanExecuteBulkSetAssetAccessSpecifier, EAssetAccessSpecifier::Public)
					)
				);

				if (bCanExecuteBulkSetEpicInternalAsset)
				{
					Section.AddMenuEntry(
						"MarkSelectedAsInternal",
						LOCTEXT("MarkSelectedAsInternal", "Mark Selected As Epic Internal"),
						LOCTEXT("MarkSelectedAsInternalTooltip", "Sets all selected assets to be available as Epic Internal for reference by other plugins"),
						FSlateIcon(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetStyleSetName(), "ContentBrowser.AssetActions.PublicAssetToggle"),
						FUIAction(
							FExecuteAction::CreateSP(this, &FAssetContextMenu::ExecuteBulkSetAssetAccessSpecifier, EAssetAccessSpecifier::EpicInternal),
							FCanExecuteAction::CreateSP(this, &FAssetContextMenu::CanExecuteBulkSetAssetAccessSpecifier, EAssetAccessSpecifier::EpicInternal)
						)
					);
				}

				Section.AddMenuEntry(
					"MarkSelectedAsPrivate",
					LOCTEXT("MarkSelectedAsPrivate", "Mark Selected As Private"),
					LOCTEXT("MarkSelectedAsPrivateTooltip", "Sets all selected assets to be private and unavailable for reference by other plugins"),
					FSlateIcon(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetStyleSetName(), "ContentBrowser.AssetActions.PublicAssetToggle"),
					FUIAction(
						FExecuteAction::CreateSP(this, &FAssetContextMenu::ExecuteBulkSetAssetAccessSpecifier, EAssetAccessSpecifier::Private),
						FCanExecuteAction::CreateSP(this, &FAssetContextMenu::CanExecuteBulkSetAssetAccessSpecifier, EAssetAccessSpecifier::Private)
					)
				);
			}
		}
	}

	return true;
}

bool FAssetContextMenu::AddReferenceMenuOptions(UToolMenu* Menu)
{
	UContentBrowserDataMenuContext_FileMenu* Context = Menu->FindContext<UContentBrowserDataMenuContext_FileMenu>();
	{
		FToolMenuSection& Section = Menu->AddSection("AssetContextReferences", LOCTEXT("ReferencesMenuHeading", "References"));

		Section.AddMenuEntry(
			FGenericCommands::Get().Copy,
			LOCTEXT("CopyReference", "Copy Reference"),
			GetCopyTooltip(EAssetViewCopyType::ExportTextPath)
		);

		Section.AddMenuEntry(
			FContentBrowserCommands::Get().AssetViewCopyObjectPath,
			LOCTEXT("CopyObjectPath", "Copy Object Path"),
			GetCopyTooltip(EAssetViewCopyType::ObjectPath)
		);

		Section.AddMenuEntry(
			FContentBrowserCommands::Get().AssetViewCopyPackageName,
			LOCTEXT("CopyPackageName", "Copy Package Path"),
			GetCopyTooltip(EAssetViewCopyType::PackageName)
		);

		if (!Context->bHasCookedPackages)
		{
			Section.AddMenuEntry(
				"CopyFilePath",
				LOCTEXT("CopyFilePath", "Copy File Path"),
				LOCTEXT("CopyFilePathTooltip", "Copies the file paths on disk for the selected assets to the clipboard."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
				FUIAction(FExecuteAction::CreateSP(this, &FAssetContextMenu::ExecuteCopyFilePath))
			);
		}
	}

	return true;
}

FText FAssetContextMenu::GetEditAssetEditorLabel(bool bInCanEdit, bool bInCanView) const
{
	static const FText EditLabel = LOCTEXT("EditAsset", "Edit...");
	static const FText OpenLabel = LOCTEXT("OpenReadOnlyAsset", "Open as Read-Only...");

	if (bInCanEdit || bInCanView)
	{
		return bInCanEdit ? EditLabel : OpenLabel;
	}
	return FText::GetEmpty();
}

FText FAssetContextMenu::GetEditAssetEditorTooltip(bool bInCanEdit, bool bInCanView) const
{
	static const FText EditTooltip = LOCTEXT("EditAssetTooltip", "Opens the selected item(s) for edit");
	static const FText OpenTooltip = LOCTEXT("OpenAssetTooltip", "Opens the selected item(s)");

	if (bInCanEdit || bInCanView)
	{
		return bInCanEdit ? EditTooltip : OpenTooltip;
	}
	return FText::GetEmpty();
}

FSlateIcon FAssetContextMenu::GetEditAssetEditorIcon(bool bInCanEdit, bool bInCanView) const
{
	if (bInCanEdit || bInCanView)
	{
		return bInCanEdit ? FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Edit")) : FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("ContentBrowser.AssetActions.OpenReadOnly"));
	}
	return FSlateIcon();
}

FText FAssetContextMenu::GetCopyTooltip(EAssetViewCopyType InCopyType) const
{
	static const FText CopyExportPathBaseTooltip = LOCTEXT("CopyExportPathBaseTooltip", "Copies reference paths for the selected asset(s) to the clipboard:");
	static const FText CopyObjectPathBaseTooltip = LOCTEXT("CopyObjectPathBaseTooltip", "Copies object paths for the selected asset(s) to the clipboard:");
	static const FText CopyPackageNameBaseTooltip = LOCTEXT("CopyPackageNameBaseTooltip", "Copies package paths for the selected asset(s) to the clipboard:");

	TArray<FText> TextsToJoin;
	TextsToJoin.Reserve(2);

	switch (InCopyType)
	{
		case EAssetViewCopyType::ExportTextPath:
			{
				TextsToJoin.Add(CopyExportPathBaseTooltip);
			}
			break;

		case EAssetViewCopyType::ObjectPath:
			{
				TextsToJoin.Add(CopyObjectPathBaseTooltip);
			}
			break;

		case EAssetViewCopyType::PackageName:
			{
				TextsToJoin.Add(CopyPackageNameBaseTooltip);
			}
			break;
	}

	if (!TextsToJoin.IsEmpty())
	{
		TextsToJoin.Add(GetSelectionInformationForCopy(InCopyType));
	}

	return FText::Join(LOCTEXT("CopyTooltipDelimiter", "\n"), TextsToJoin);
}

FText FAssetContextMenu::GetSelectionInformationForCopy(EAssetViewCopyType InCopyType) const
{
	FText SelectionInformationForCopy;

	if (!SelectedFiles.IsEmpty())
	{
		const FContentBrowserItem& FirstSelectedFile = SelectedFiles[0];

		{
			FString AdditionalInformation = TEXT("");
			switch (InCopyType)
			{
			case EAssetViewCopyType::ExportTextPath:
				FirstSelectedFile.AppendItemReference(AdditionalInformation);
				break;

			case EAssetViewCopyType::ObjectPath:
				FirstSelectedFile.AppendItemObjectPath(AdditionalInformation);
				break;

			case EAssetViewCopyType::PackageName:
				FirstSelectedFile.AppendItemPackageName(AdditionalInformation);
				break;
			}

			SelectionInformationForCopy = FText::FromString(MoveTemp(AdditionalInformation));
		}

		if (SelectedFiles.Num() > 1)
		{
			FText MoreFileCount = FText::Format(LOCTEXT("AdditionalFileCountTooltip", "+{0} more"), FText::AsNumber(SelectedFiles.Num() - 1));

			TArray<FText> SelectionInformationTexts;
			SelectionInformationTexts.Reserve(2);
			SelectionInformationTexts.Add(MoveTemp(SelectionInformationForCopy));
			SelectionInformationTexts.Add(MoveTemp(MoreFileCount));

			SelectionInformationForCopy = FText::Join(LOCTEXT("JoinAdditionalFileCountTooltip", "\n"), SelectionInformationTexts);
		}
	}

	return SelectionInformationForCopy;
}

bool FAssetContextMenu::AddAssetTypeMenuOptions(UToolMenu* Menu)
{
	bool bAnyTypeOptions = false;

	UContentBrowserAssetContextMenuContext* Context = Menu->FindContext<UContentBrowserAssetContextMenuContext>();
	if (Context && Context->SelectedAssets.Num() > 0)
	{
		// Label "GetAssetActions" section
		FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
		if (Context->CommonAssetDefinition)
		{
			Section.Label = FText::Format(NSLOCTEXT("AssetTools", "AssetSpecificOptionsMenuHeading", "{0} Actions"), Context->CommonAssetDefinition->GetAssetDisplayName());
		}
		else if (Context->CommonClass)
		{
			Section.Label = FText::Format(NSLOCTEXT("AssetTools", "AssetSpecificOptionsMenuHeading", "{0} Actions"), FText::FromName(Context->CommonClass->GetFName()));
		}
		else
		{
			Section.Label = FText::Format(NSLOCTEXT("AssetTools", "AssetSpecificOptionsMenuHeading", "{0} Actions"), FText::FromString(TEXT("Asset")));
		}

		bAnyTypeOptions = true;
	}

	return bAnyTypeOptions;
}

bool FAssetContextMenu::AddCollectionMenuOptions(UToolMenu* Menu)
{
	class FManageCollectionsContextMenu
	{
	public:
		static bool CanManageCollectionContainer(const TSharedPtr<ICollectionContainer>& CollectionContainer)
		{
			return !CollectionContainer->IsHidden() && !CollectionContainer->IsReadOnly(ECollectionShareType::CST_All) && CollectionContainer->HasCollections();
		}

		static void CreateManageCollectionsSubMenu(UToolMenu* SubMenu, TArray<FSoftObjectPath> CurrentAssetPaths, TArray<TSharedPtr<ICollectionContainer>> CollectionContainers)
		{
			CollectionContainers.RemoveAll([](const TSharedPtr<ICollectionContainer>& CollectionContainer) { return !CanManageCollectionContainer(CollectionContainer); });

			for (TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
			{
				TAttribute<FText> Label;

				if (CollectionContainers.Num() != 1)
				{
					Label.Set(CollectionContainer->GetCollectionSource()->GetTitle());
				}

				TSharedRef<FCollectionAssetManagement> QuickAssetManagement = MakeShared<FCollectionAssetManagement>(CollectionContainer.ToSharedRef());
				QuickAssetManagement->SetCurrentAssetPaths(CurrentAssetPaths);

				TArray<FCollectionNameType> AvailableCollections;
				CollectionContainer->GetRootCollections(AvailableCollections);

				CreateManageCollectionsSubMenu(SubMenu, QuickAssetManagement, CollectionContainer, Label, MoveTemp(AvailableCollections));
			}
		}

		static void CreateManageCollectionsSubMenu(UToolMenu* SubMenu, TSharedRef<FCollectionAssetManagement> QuickAssetManagement, TSharedPtr<ICollectionContainer> CollectionContainer, TAttribute<FText> Label, TArray<FCollectionNameType> AvailableCollections)
		{
			AvailableCollections.Sort([](const FCollectionNameType& One, const FCollectionNameType& Two) -> bool
			{
				return One.Name.LexicalLess(Two.Name);
			});

			FToolMenuSection& Section = SubMenu->AddSection(CollectionContainer->GetCollectionSource()->GetName(), Label);
			for (const FCollectionNameType& AvailableCollection : AvailableCollections)
			{
				// Never display system collections
				if (AvailableCollection.Type == ECollectionShareType::CST_System)
				{
					continue;
				}

				// Can only manage assets for static collections
				ECollectionStorageMode::Type StorageMode = ECollectionStorageMode::Static;
				CollectionContainer->GetCollectionStorageMode(AvailableCollection.Name, AvailableCollection.Type, StorageMode);
				if (StorageMode != ECollectionStorageMode::Static)
				{
					continue;
				}

				TArray<FCollectionNameType> AvailableChildCollections;
				CollectionContainer->GetChildCollections(AvailableCollection.Name, AvailableCollection.Type, AvailableChildCollections);

				if (AvailableChildCollections.Num() > 0)
				{
					Section.AddSubMenu(
						NAME_None,
						FText::FromName(AvailableCollection.Name), 
						FText::GetEmpty(), 
						FNewToolMenuDelegate::CreateStatic(&FManageCollectionsContextMenu::CreateManageCollectionsSubMenu, QuickAssetManagement, CollectionContainer, TAttribute<FText>(), AvailableChildCollections),
						FUIAction(
							FExecuteAction::CreateStatic(&FManageCollectionsContextMenu::OnCollectionClicked, QuickAssetManagement, AvailableCollection),
							FCanExecuteAction::CreateStatic(&FManageCollectionsContextMenu::IsCollectionEnabled, QuickAssetManagement, AvailableCollection),
							FGetActionCheckState::CreateStatic(&FManageCollectionsContextMenu::GetCollectionCheckState, QuickAssetManagement, AvailableCollection)
							), 
						EUserInterfaceActionType::ToggleButton,
						false,
						FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(AvailableCollection.Type))
						);
				}
				else
				{
					Section.AddMenuEntry(
						NAME_None,
						FText::FromName(AvailableCollection.Name), 
						FText::GetEmpty(), 
						FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(AvailableCollection.Type)),
						FUIAction(
							FExecuteAction::CreateStatic(&FManageCollectionsContextMenu::OnCollectionClicked, QuickAssetManagement, AvailableCollection),
							FCanExecuteAction::CreateStatic(&FManageCollectionsContextMenu::IsCollectionEnabled, QuickAssetManagement, AvailableCollection),
							FGetActionCheckState::CreateStatic(&FManageCollectionsContextMenu::GetCollectionCheckState, QuickAssetManagement, AvailableCollection)
							), 
						EUserInterfaceActionType::ToggleButton
						);
				}
			}
		}

	private:
		static bool IsCollectionEnabled(TSharedRef<FCollectionAssetManagement> QuickAssetManagement, FCollectionNameType InCollectionKey)
		{
			return QuickAssetManagement->IsCollectionEnabled(InCollectionKey);
		}

		static ECheckBoxState GetCollectionCheckState(TSharedRef<FCollectionAssetManagement> QuickAssetManagement, FCollectionNameType InCollectionKey)
		{
			return QuickAssetManagement->GetCollectionCheckState(InCollectionKey);
		}

		static void OnCollectionClicked(TSharedRef<FCollectionAssetManagement> QuickAssetManagement, FCollectionNameType InCollectionKey)
		{
			const double BeginTimeSec = FPlatformTime::Seconds();
			const int32 ObjectCount = QuickAssetManagement->GetCurrentAssetCount();
			
			// The UI actions don't give you the new check state, so we need to emulate the behavior of SCheckBox
			// Basically, checked will transition to unchecked (removing items), and anything else will transition to checked (adding items)
			const bool RemoveFromCollection = GetCollectionCheckState(QuickAssetManagement, InCollectionKey) == ECheckBoxState::Checked;
			if (RemoveFromCollection)
			{
				QuickAssetManagement->RemoveCurrentAssetsFromCollection(InCollectionKey);
			}
			else
			{
				QuickAssetManagement->AddCurrentAssetsToCollection(InCollectionKey);
			}

			const double DurationSec = FPlatformTime::Seconds() - BeginTimeSec;
			
			{
				if (RemoveFromCollection)
				{
					FAssetRemovedFromCollectionTelemetryEvent AssetRemoved;
					AssetRemoved.DurationSec = DurationSec;
					AssetRemoved.NumRemoved = ObjectCount;
					AssetRemoved.CollectionShareType = InCollectionKey.Type;
					AssetRemoved.Workflow = ECollectionTelemetryAssetRemovedWorkflow::ContextMenu;
					FTelemetryRouter::Get().ProvideTelemetry(AssetRemoved);
				}
				else
				{
					FAssetAddedToCollectionTelemetryEvent AssetAdded;
					AssetAdded.DurationSec = DurationSec;
					AssetAdded.NumAdded = ObjectCount;
					AssetAdded.CollectionShareType = InCollectionKey.Type;
					AssetAdded.Workflow = ECollectionTelemetryAssetAddedWorkflow::ContextMenu;
					FTelemetryRouter::Get().ProvideTelemetry(AssetAdded);
				}
			}
		}
	};

	bool bHasAddedItems = false;

	FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();

	TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
	CollectionManagerModule.Get().GetVisibleCollectionContainers(CollectionContainers);

	FToolMenuSection& Section = Menu->AddSection("AssetContextCollections", LOCTEXT("AssetCollectionOptionsMenuHeading", "Collections"));

	// Show a sub-menu that allows you to quickly add or remove the current asset selection from the available collections
	if (CollectionContainers.FindByPredicate(FManageCollectionsContextMenu::CanManageCollectionContainer) != nullptr)
	{
		TArray<FSoftObjectPath> SelectedItemCollectionIds;
		for (const FContentBrowserItem& SelectedItem : SelectedFiles)
		{
			FSoftObjectPath ItemCollectionId;
			if (SelectedItem.TryGetCollectionId(ItemCollectionId))
			{
				SelectedItemCollectionIds.Add(ItemCollectionId);
			}
		}

		Section.AddSubMenu(
			"ManageCollections",
			LOCTEXT("ManageCollections", "Manage Collections"),
			FText::Format(LOCTEXT("ManageCollections_ToolTip", "Manage the collections that the selected {0}|plural(one=item belongs, other=items belong) to."), SelectedFiles.Num()),
			FNewToolMenuDelegate::CreateStatic(&FManageCollectionsContextMenu::CreateManageCollectionsSubMenu, SelectedItemCollectionIds, CollectionContainers),
			false, // default value
			FSlateIcon(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetStyleSetName(), "ContentBrowser.Collections")
			);

		bHasAddedItems = true;
	}

	// "Remove from collection"
	if (CanExecuteRemoveFromCollection())
	{
		Section.AddMenuEntry(
			"RemoveFromCollection",
			FText::Format(LOCTEXT("RemoveFromCollectionFmt", "Remove From {0}"), FText::FromName(ContentSources.GetCollections()[0].Name)),
			LOCTEXT("RemoveFromCollection_ToolTip", "Removes the selected item from the current collection."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP( this, &FAssetContextMenu::ExecuteRemoveFromCollection ),
				FCanExecuteAction::CreateSP( this, &FAssetContextMenu::CanExecuteRemoveFromCollection )
				)
			);

		bHasAddedItems = true;
	}


	return bHasAddedItems;
}

void FAssetContextMenu::ExecuteSyncToAssetTree()
{
	// Copy this as the sync may adjust our selected assets array
	const TArray<FContentBrowserItem> SelectedFilesCopy = SelectedFiles;
	OnShowInPathsViewRequested.ExecuteIfBound(SelectedFilesCopy);
}

void FAssetContextMenu::ExecuteFindInExplorer()
{
	ContentBrowserUtils::ExploreFolders(SelectedFiles, AssetView.Pin().ToSharedRef());
}

void FAssetContextMenu::ExecuteSaveAsset()
{
	const EContentBrowserItemSaveFlags SaveFlags = EContentBrowserItemSaveFlags::None;

	// Batch these by their data sources
	TMap<UContentBrowserDataSource*, TArray<FContentBrowserItemData>> SourcesAndItems;
	for (const FContentBrowserItem& SelectedItem : SelectedFiles)
	{
		FContentBrowserItem::FItemDataArrayView ItemDataArray = SelectedItem.GetInternalItems();
		for (const FContentBrowserItemData& ItemData : ItemDataArray)
		{
			if (UContentBrowserDataSource* ItemDataSource = ItemData.GetOwnerDataSource())
			{
				FText SaveErrorMsg;
				if (ItemDataSource->CanSaveItem(ItemData, SaveFlags, &SaveErrorMsg))
				{
					TArray<FContentBrowserItemData>& ItemsForSource = SourcesAndItems.FindOrAdd(ItemDataSource);
					ItemsForSource.Add(ItemData);
				}
				else
				{
					AssetViewUtils::ShowErrorNotifcation(SaveErrorMsg);
				}
			}
		}
	}

	// Execute the operation now
	for (const auto& SourceAndItemsPair : SourcesAndItems)
	{
		SourceAndItemsPair.Key->BulkSaveItems(SourceAndItemsPair.Value, SaveFlags);
	}
}

void FAssetContextMenu::ExecuteDuplicate() 
{
	if (SelectedFiles.Num() > 0)
	{
		OnDuplicateRequested.ExecuteIfBound(SelectedFiles);
	}
}

void FAssetContextMenu::ExecuteRename(EContentBrowserViewContext ViewContext)
{
	if (SelectedItems.Num() == 1)
	{
		OnRenameRequested.ExecuteIfBound(SelectedItems[0], ViewContext);
	}
}

void FAssetContextMenu::ExecuteDelete()
{
	// Batch these by their data sources
	TMap<UContentBrowserDataSource*, TArray<FContentBrowserItemData>> SourcesAndItems;
	for (const FContentBrowserItem& SelectedItem : SelectedFiles)
	{
		FContentBrowserItem::FItemDataArrayView ItemDataArray = SelectedItem.GetInternalItems();
		for (const FContentBrowserItemData& ItemData : ItemDataArray)
		{
			if (UContentBrowserDataSource* ItemDataSource = ItemData.GetOwnerDataSource())
			{
				FText DeleteErrorMsg;
				if (ItemDataSource->CanDeleteItem(ItemData, &DeleteErrorMsg))
				{
					TArray<FContentBrowserItemData>& ItemsForSource = SourcesAndItems.FindOrAdd(ItemDataSource);
					ItemsForSource.Add(ItemData);
				}
				else
				{
					AssetViewUtils::ShowErrorNotifcation(DeleteErrorMsg);
				}
			}
		}
	}

	// Execute the operation now
	for (const auto& SourceAndItemsPair : SourcesAndItems)
	{
		SourceAndItemsPair.Key->BulkDeleteItems(SourceAndItemsPair.Value);
	}

	if (SelectedFolders.Num() > 0)
	{
		ExecuteDeleteFolderConfirmed();
	}
}

FReply FAssetContextMenu::ExecuteDeleteFolderConfirmed()
{
	// Batch these by their data sources
	TMap<UContentBrowserDataSource*, TArray<FContentBrowserItemData>> SourcesAndItems;
	for (const FContentBrowserItem& SelectedItem : SelectedFolders)
	{
		FContentBrowserItem::FItemDataArrayView ItemDataArray = SelectedItem.GetInternalItems();
		for (const FContentBrowserItemData& ItemData : ItemDataArray)
		{
			if (UContentBrowserDataSource* ItemDataSource = ItemData.GetOwnerDataSource())
			{
				FText DeleteErrorMsg;
				if (ItemDataSource->CanDeleteItem(ItemData, &DeleteErrorMsg))
				{
					TArray<FContentBrowserItemData>& ItemsForSource = SourcesAndItems.FindOrAdd(ItemDataSource);
					ItemsForSource.Add(ItemData);
				}
				else
				{
					AssetViewUtils::ShowErrorNotifcation(DeleteErrorMsg);
				}
			}
		}
	}

	// Execute the operation now
	for (const auto& SourceAndItemsPair : SourcesAndItems)
	{
		SourceAndItemsPair.Key->BulkDeleteItems(SourceAndItemsPair.Value);
	}

	return FReply::Handled();
}

int32 FAssetContextMenu::CompareAssetAccessSpecifiers(EAssetAccessSpecifier A, EAssetAccessSpecifier B)
{
	checkf((A == EAssetAccessSpecifier::Private || A == EAssetAccessSpecifier::EpicInternal || A == EAssetAccessSpecifier::Public) &&
		(B == EAssetAccessSpecifier::Private || B == EAssetAccessSpecifier::EpicInternal || B == EAssetAccessSpecifier::Public),
		TEXT("Unknown EAssetAccessSpecifier"));

	if (A == B)
	{
		return 0;
	}

	if (A == EAssetAccessSpecifier::Private || B == EAssetAccessSpecifier::Public)
	{
		return -1;
	}

	return 1;
}

void FAssetContextMenu::ExecuteSetAssetAccessSpecifier(EAssetAccessSpecifier InAssetAccessSpecifier)
{
	if (ensure(SelectedFiles.Num() == 1))
	{
		ExecuteBulkSetAssetAccessSpecifier(InAssetAccessSpecifier);
	}
}

bool FAssetContextMenu::CanSetAssetAccessSpecifier(const EAssetAccessSpecifier InAssetAccessSpecifier) const
{
	if (InAssetAccessSpecifier == EAssetAccessSpecifier::Public)
	{
		return bCanExecuteSetPublicAsset;
	}
	else if (InAssetAccessSpecifier == EAssetAccessSpecifier::EpicInternal)
	{
		return bCanExecuteSetEpicInternalAsset;
	}

	return bCanExecuteSetPrivateAsset;
}

void FAssetContextMenu::ExecuteBulkSetAssetAccessSpecifier(EAssetAccessSpecifier DestScope)
{
	TMap<UContentBrowserDataSource*, TArray<FContentBrowserItemData>> PrivatizeSourcesAndItems;

	{
		FScopedSlowTask SlowTask((float)SelectedFiles.Num(), LOCTEXT("SetAssetScope", "Loading assets and modifying scope..."));
		SlowTask.MakeDialogDelayed(0.5f, /*bShowCancelButton*/true);

		for (const FContentBrowserItem& SelectedItem : SelectedFiles)
		{
			SlowTask.EnterProgressFrame(1);

			if (SlowTask.ShouldCancel())
			{
				break;
			}

			FContentBrowserItem::FItemDataArrayView ItemDataArray = SelectedItem.GetInternalItems();
			for (const FContentBrowserItemData& ItemData : ItemDataArray)
			{
				if (UContentBrowserDataSource* ItemDataSource = ItemData.GetOwnerDataSource())
				{
					FAssetData ItemAssetData;
					if (!ItemDataSource->Legacy_TryGetAssetData(ItemData, ItemAssetData))
					{
						continue;
					}

					if (DestScope != EAssetAccessSpecifier::Private)
					{
						const FNameBuilder ItemInternalPath(ItemData.GetInternalPath());
						const FStringView AssetPath(ItemInternalPath);
						const bool bCanBePublic = IAssetTools::Get().CanAssetBePublic(AssetPath);
						if (!bCanBePublic)
						{
							// Assets without public permissions might end up in our bulk selection. Make sure to skip those.
							continue;
						}
					}

					const int32 Result = CompareAssetAccessSpecifiers(DestScope, ItemAssetData.GetAssetAccessSpecifier());
					if (Result > 0)
					{
						// Ok to asset raise the access specifier to DestScope without calling BulkPrivatizeItems
						if (UPackage* ItemAssetPackage = ItemAssetData.GetPackage())
						{
							if (ItemAssetPackage->SetAssetAccessSpecifier(DestScope))
							{
								if (UObject* Asset = ItemAssetData.GetAsset())
								{
									// Let the data source know we updated the asset access specifier.
									Asset->PostEditChange();
								}
							}
						}
					}
					else if (Result < 0)
					{
						// Lowering asset access specifier uses BulkPrivatizeItems workflow instead of directly calling SetAssetAccessSpecifier
						FText PrivateErrorMsg;
						// CanPrivatizeItem checks !GetAssetFolderItemPayload(), CanModifyAssetFileItem(), !IsRunningPIE(), !IsRedirector()
						if (ItemDataSource->CanPrivatizeItem(ItemData, &PrivateErrorMsg))
						{
							if (ItemAssetData.GetPackage()) // Load under the slow task.
							{
								TArray<FContentBrowserItemData>& ItemsForSource = PrivatizeSourcesAndItems.FindOrAdd(ItemDataSource);
								ItemsForSource.Add(ItemData);
							}
						}
						else
						{
							AssetViewUtils::ShowErrorNotifcation(PrivateErrorMsg);
						}
					}
				}
			}
		}
	}

	if (!PrivatizeSourcesAndItems.IsEmpty())
	{
		for (const auto& SourceAndItemsPair : PrivatizeSourcesAndItems)
		{
			SourceAndItemsPair.Key->BulkPrivatizeItems(SourceAndItemsPair.Value, DestScope);
		}
	}

	// Calling PostEditChange is enough to refresh the view, but there is an issue where if you open the context menu again on the same frame,
	// SetSelectedItems gets called making a copy of the items before they get updated when the UContentBrowserDataSubsystem ticks,
	// so you end up with stale data.  Refreshing the view here will cause the SContentBrowser to call SetSelectedItems again after it ticks
	// which fixes the issue, though it creates its own problems since other decisions based on the original selection are cached (e.g. CacheCanExecuteVars).
	OnAssetViewRefreshRequested.ExecuteIfBound();
}

bool FAssetContextMenu::CanExecuteBulkSetAssetAccessSpecifier(EAssetAccessSpecifier InAssetAccessSpecifier)
{
	if (InAssetAccessSpecifier == EAssetAccessSpecifier::Public)
	{
		return bCanExecuteBulkSetPublicAsset;
	}
	else if (InAssetAccessSpecifier == EAssetAccessSpecifier::EpicInternal)
	{
		return bCanExecuteBulkSetEpicInternalAsset;
	}
	else if (InAssetAccessSpecifier == EAssetAccessSpecifier::Private)
	{
		return bCanExecuteBulkSetPrivateAsset;
	}

	return false;
}


bool FAssetContextMenu::GetAssetAccessSpecifierFromSelection(EAssetAccessSpecifier& OutAssetAccessSpecifier)
{
	if (ensure(SelectedFiles.Num() == 1))
	{
		return AssetContextMenu::GetAssetAccessSpecifierFromItem(SelectedFiles[0], OutAssetAccessSpecifier);
	}

	return false;
}

bool FAssetContextMenu::IsSelectedAssetAccessSpecifier(EAssetAccessSpecifier InAssetAccessSpecifier)
{
	EAssetAccessSpecifier AssetAccessSpecifier;
	return GetAssetAccessSpecifierFromSelection(AssetAccessSpecifier) && InAssetAccessSpecifier == AssetAccessSpecifier;
}

void FAssetContextMenu::ExecuteCopyFilePath()
{
	if (SelectedFiles.Num() > 0)
	{
		ContentBrowserUtils::CopyFilePathsToClipboard(SelectedFiles);
	}
}

void FAssetContextMenu::ExecuteRemoveFromCollection()
{
	if ( ensure(CanExecuteRemoveFromCollection()) )
	{
		TArray<FSoftObjectPath> SelectedItemCollectionIds;
		for (const FContentBrowserItem& SelectedItem : SelectedFiles)
		{
			FSoftObjectPath ItemCollectionId;
			if (SelectedItem.TryGetCollectionId(ItemCollectionId))
			{
				SelectedItemCollectionIds.Add(ItemCollectionId);
			}
		}

		if ( SelectedItemCollectionIds.Num() > 0 )
		{
			const FCollectionRef& Collection = ContentSources.GetCollections()[0];
			const double BeginTimeSec = FPlatformTime::Seconds();
			Collection.Container->RemoveFromCollection(Collection.Name, Collection.Type, SelectedItemCollectionIds);
			const double DurationSec = FPlatformTime::Seconds() - BeginTimeSec;
			OnAssetViewRefreshRequested.ExecuteIfBound();

			FAssetRemovedFromCollectionTelemetryEvent AssetRemoved;
			AssetRemoved.DurationSec = DurationSec;
			AssetRemoved.NumRemoved = SelectedItemCollectionIds.Num();
			AssetRemoved.CollectionShareType = Collection.Type;
			AssetRemoved.Workflow = ECollectionTelemetryAssetRemovedWorkflow::ContextMenu;
			FTelemetryRouter::Get().ProvideTelemetry(AssetRemoved);
		}
	}
}

bool FAssetContextMenu::CanExecuteSyncToAssetTree() const
{
	return SelectedFiles.Num() > 0;
}

bool FAssetContextMenu::CanExecuteFindInExplorer() const
{
	return bCanExecuteFindInExplorer;
}

bool FAssetContextMenu::CanExecuteRemoveFromCollection() const 
{
	return ContentSources.GetCollections().Num() == 1 &&
		!ContentSources.GetCollections()[0].Container->IsReadOnly(ContentSources.GetCollections()[0].Type) &&
		!ContentSources.IsDynamicCollection();
}

bool FAssetContextMenu::CanExecuteDuplicate() const
{
	bool bCanDuplicate = false;
	for (const FContentBrowserItem& SelectedItem : SelectedFiles)
	{
		bCanDuplicate |= SelectedItem.CanDuplicate();
	}
	return bCanDuplicate;
}

bool FAssetContextMenu::CanExecuteRename() const
{
	return ContentBrowserUtils::CanRenameFromAssetView(AssetView);
}

bool FAssetContextMenu::CanExecuteDelete() const
{
	return ContentBrowserUtils::CanDeleteFromAssetView(AssetView);
}

FText FAssetContextMenu::GetDeleteToolTip() const
{
	FText ErrorMessage;
	if (!ContentBrowserUtils::CanDeleteFromAssetView(AssetView, &ErrorMessage) && !ErrorMessage.IsEmpty())
	{
		return ErrorMessage;
	}

	return LOCTEXT("DeleteTooltip", "Delete the selected items.");
}

bool FAssetContextMenu::CanExecuteSaveAsset() const
{
	bool bCanSave = false;
	for (const FContentBrowserItem& SelectedItem : SelectedFiles)
	{
		bCanSave |= SelectedItem.CanSave(EContentBrowserItemSaveFlags::None);
	}
	return bCanSave;
}

void FAssetContextMenu::CacheCanExecuteVars()
{
	bCanExecuteFindInExplorer = false;

	bCanShowSetAssetAccessMenu = false;

	bCanExecuteSetPublicAsset = false;
	bCanExecuteSetEpicInternalAsset = false;
	bCanExecuteSetPrivateAsset = false;

	bCanExecuteBulkSetPublicAsset = false;
	bCanExecuteBulkSetEpicInternalAsset = false;
	bCanExecuteBulkSetPrivateAsset = false;

	DisallowSetPublicReason = FText::GetEmpty();

	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	const FOnContentBrowserCanShowSetAssetAccessMenu& OnContentBrowserCanShowSetAssetAccessMenu = ContentBrowserModule.GetOnContentBrowserCanShowSetAssetAccessMenu();

	const bool bIsEpicInternalAssetFeatureEnabled = AssetContextMenu::IsEpicInternalAssetFeatureEnabled();
	const TSharedPtr<FPathPermissionList>& ShowEpicInternalContentPermissionList = ContentBrowserModule.Get().GetShowEpicInternalContentPermissionList();
	bool bAnyPathPassesShowEpicInternal = !ShowEpicInternalContentPermissionList->HasFiltering();

	const TSharedPtr<FPathPermissionList>& ShowPrivateContentPermissionList = ContentBrowserModule.Get().GetShowPrivateContentPermissionList();

	auto CheckCanExecuteFindInExplorer = [](const FContentBrowserItem& SelectedItem) -> bool
	{
		FString ItemFilename;
		return SelectedItem.GetItemPhysicalPath(ItemFilename) && FPaths::FileExists(ItemFilename);
	};

	auto CheckCanAssetBePublic = [](const FStringView AssetPath, FText* DisallowReason) -> bool
	{
		return IAssetTools::Get().CanAssetBePublic(AssetPath, DisallowReason);
	};

	auto CheckCanShowSetAssetAccessMenu = [&OnContentBrowserCanShowSetAssetAccessMenu](const FStringView AssetPath, const FAssetData& Asset) -> bool
	{
		return !OnContentBrowserCanShowSetAssetAccessMenu.IsBound() || OnContentBrowserCanShowSetAssetAccessMenu.Execute(AssetPath, Asset);
	};

	if (SelectedFiles.Num() == 1)
	{
		const FContentBrowserItem& SelectedItem = SelectedFiles[0];
		bCanExecuteFindInExplorer = CheckCanExecuteFindInExplorer(SelectedItem);

		if (SelectedItem.CanEdit())
		{
			const FNameBuilder ItemInternalPath(SelectedItem.GetInternalPath());
			const FStringView AssetPath(ItemInternalPath);
			const FAssetData AssetData = IAssetRegistry::GetChecked().GetAssetByObjectPath(FSoftObjectPath(AssetPath));
			if (AssetData.IsValid() && CheckCanShowSetAssetAccessMenu(AssetPath, AssetData))
			{
				bCanShowSetAssetAccessMenu = true;

				// Only show menu entries when asset can be made externally referenceable 
				if (CheckCanAssetBePublic(AssetPath, &DisallowSetPublicReason))
				{
					bCanExecuteSetPublicAsset = true;
					bCanExecuteSetEpicInternalAsset = bIsEpicInternalAssetFeatureEnabled && (ShowEpicInternalContentPermissionList->PassesStartsWithFilter(AssetPath) || ShowPrivateContentPermissionList->PassesStartsWithFilter(AssetPath));
					bCanExecuteSetPrivateAsset = true;
				}
				else if (AssetData.GetAssetAccessSpecifier() != EAssetAccessSpecifier::Private)
				{
					// Allow user to manually fix asset that should be private
					bCanExecuteSetPrivateAsset = true;
				}
			}
		}
	}
	else
	{
		for (const FContentBrowserItem& SelectedItem : SelectedFiles)
		{
			if (!bCanExecuteFindInExplorer)
			{
				bCanExecuteFindInExplorer = CheckCanExecuteFindInExplorer(SelectedItem);
			}

			if (SelectedItem.CanEdit())
			{
				const FNameBuilder ItemInternalPath(SelectedItem.GetInternalPath());
				const FStringView AssetPath(ItemInternalPath);
				const FAssetData AssetData = IAssetRegistry::GetChecked().GetAssetByObjectPath(FSoftObjectPath(AssetPath));
				if (AssetData.IsValid())
				{
					// If any selected asset can be made public, show menu entries to bulk set as Public/Internal/Private
					if (!bCanExecuteBulkSetPublicAsset && CheckCanAssetBePublic(AssetPath, nullptr))
					{
						bCanExecuteBulkSetPublicAsset = true;
						bCanExecuteBulkSetEpicInternalAsset = bIsEpicInternalAssetFeatureEnabled;
						bCanExecuteBulkSetPrivateAsset = true;
					}

					// If any selected can be set to Epic internal
					if (!bAnyPathPassesShowEpicInternal && bIsEpicInternalAssetFeatureEnabled)
					{
						if (CheckCanAssetBePublic(AssetPath, nullptr))
						{
							if (ShowEpicInternalContentPermissionList->PassesStartsWithFilter(AssetPath) || (ShowPrivateContentPermissionList->HasFiltering() && ShowPrivateContentPermissionList->PassesStartsWithFilter(AssetPath)))
							{
								bAnyPathPassesShowEpicInternal = true;
							}
						}
					}

					// If any selected asset not private, show menu entry to bulk set to private
					if (!bCanExecuteBulkSetPrivateAsset && (AssetData.GetAssetAccessSpecifier() != EAssetAccessSpecifier::Private))
					{
						bCanExecuteBulkSetPrivateAsset = true;
					}
				}
			}
		}

		// Hide menu entries when nothing passes IAssetTools::Get().CanAssetBePublic()
		if (!bCanExecuteBulkSetPublicAsset)
		{
			bAnyPathPassesShowEpicInternal = false;
			bCanExecuteBulkSetEpicInternalAsset = false;
			bCanExecuteBulkSetPrivateAsset = false;
		}

		if (!bAnyPathPassesShowEpicInternal)
		{
			bCanExecuteBulkSetEpicInternalAsset = false;
		}
	}

	if (!bIsEpicInternalAssetFeatureEnabled)
	{
		bAnyPathPassesShowEpicInternal = false;
		bCanExecuteBulkSetEpicInternalAsset = false;
	}

	// CVar bypass to always show. For repairing broken assets stuck with the wrong flags.
	if (AssetContextMenu::IsAlwaysShowSetAssetScopeMenus())
	{
		bCanShowSetAssetAccessMenu = true;

		if (SelectedFiles.Num() == 1)
		{
			bCanExecuteSetPublicAsset = true;
			bCanExecuteSetEpicInternalAsset = bIsEpicInternalAssetFeatureEnabled;
			bCanExecuteSetPrivateAsset = true;
		}
		else
		{
			bCanExecuteBulkSetPublicAsset = true;
			bCanExecuteBulkSetEpicInternalAsset = bIsEpicInternalAssetFeatureEnabled;
			bCanExecuteBulkSetPrivateAsset = true;
		}
	}
}

#undef LOCTEXT_NAMESPACE
