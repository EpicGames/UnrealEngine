// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "ContentBrowserDataMenuContexts.h"
#include "ContentBrowserItem.h"
#include "AssetViewContentSources.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FUICommandList;
class SAssetView;
class SWidget;
class UClass;
class UToolMenu;

enum class ECheckBoxState : uint8;
enum class EContentBrowserViewContext : uint8;
enum class EAssetAccessSpecifier : uint8;
enum class EAssetViewCopyType;

class FAssetContextMenu : public TSharedFromThis<FAssetContextMenu>
{
public:
	/** Constructor */
	FAssetContextMenu(const TWeakPtr<SAssetView>& InAssetView);

	/** Binds the commands used by the asset view context menu to the content browser command list */
	void BindCommands(TSharedPtr< FUICommandList >& Commands);

	/** Makes the context menu widget */
	TSharedRef<SWidget> MakeContextMenu(TArrayView<const FContentBrowserItem> InSelectedItems, const FAssetViewContentSources& InContentSources, TSharedPtr< FUICommandList > InCommandList);

	/** Updates the list of currently selected items to those passed in */
	void SetSelectedItems(TArrayView<const FContentBrowserItem> InSelectedItems);

	/** Read-only access to the list of currently selected items */
	const TArray<FContentBrowserItem>& GetSelectedItems() const { return SelectedItems; }

	using FOnShowInPathsViewRequested = UContentBrowserDataMenuContext_FileMenu::FOnShowInPathsView;
	void SetOnShowInPathsViewRequested(const FOnShowInPathsViewRequested& InOnShowInPathsViewRequested);

	/** Delegate for when the context menu requests a rename */
	DECLARE_DELEGATE_TwoParams(FOnRenameRequested, const FContentBrowserItem& /*ItemToRename*/, EContentBrowserViewContext /*ViewContext*/);
	void SetOnRenameRequested(const FOnRenameRequested& InOnRenameRequested);

	/** Delegate for when the context menu requests an item duplication */
	DECLARE_DELEGATE_OneParam(FOnDuplicateRequested, TArrayView<const FContentBrowserItem> /*OriginalItems*/);
	void SetOnDuplicateRequested(const FOnDuplicateRequested& InOnDuplicateRequested);

	/** Delegate for when the context menu requests an asset view refresh */
	using FOnAssetViewRefreshRequested = UContentBrowserDataMenuContext_FileMenu::FOnRefreshView;
	void SetOnAssetViewRefreshRequested(const FOnAssetViewRefreshRequested& InOnAssetViewRefreshRequested);

	/** Handler to check to see if a rename command is allowed */
	bool CanExecuteRename() const;

	/** Handler for Rename */
	void ExecuteRename(EContentBrowserViewContext ViewContext);

	/** Handler to check to see if a delete command is allowed */
	bool CanExecuteDelete() const;

	/** Handler for Delete */
	void ExecuteDelete();

	/** Handler to check to see if "Save Asset" can be executed */
	bool CanExecuteSaveAsset() const;

	/** Handler for when "Save Asset" is selected */
	void ExecuteSaveAsset();

private:
	/** Handler to check to see if a duplicate command is allowed */
	bool CanExecuteDuplicate() const;

	/** Handler for Duplicate */
	void ExecuteDuplicate();

	/** Registers all unregistered menus in the hierarchy for a class */
	static void RegisterMenuHierarchy(UClass* InClass);

	/** Adds options to menu */
	void AddMenuOptions(UToolMenu* InMenu);

	/** Adds asset type-specific menu options to a menu builder. Returns true if any options were added. */
	bool AddAssetTypeMenuOptions(UToolMenu* Menu);
	
	/** Adds common menu options to a menu builder. Returns true if any options were added. */
	bool AddCommonMenuOptions(UToolMenu* Menu);

	/** Adds explore menu options to a menu builder. */
	void AddExploreMenuOptions(UToolMenu* Menu);

	/** Adds asset reference menu options to a menu builder. Returns true if any options were added. */
	bool AddReferenceMenuOptions(UToolMenu* Menu);

	/** Get the correct Label for the OpenAssetEditor command */
	FText GetEditAssetEditorLabel(bool bInCanEdit, bool bInCanView) const;

	/** Get the correct Tooltip for the OpenAssetEditor command */
	FText GetEditAssetEditorTooltip(bool bInCanEdit, bool bInCanView) const;

	/** Get the correct Icon for the OpenAssetEditor command */
	FSlateIcon GetEditAssetEditorIcon(bool bInCanEdit, bool bInCanView) const;
	
	/** Return the tooltip based on the copy type */
	FText GetCopyTooltip(EAssetViewCopyType InCopyType) const;

	/** Append information on the path for the Copy tooltip based on the current selection */
	FText GetSelectionInformationForCopy(EAssetViewCopyType InCopyType) const;

	bool AddPublicStateMenuOptions(UToolMenu* Menu);

	/** Adds menu options related to working with collections */
	bool AddCollectionMenuOptions(UToolMenu* Menu);

	/** Handler for when sync to asset tree is selected */
	void ExecuteSyncToAssetTree();

	/** Handler for when find in explorer is selected */
	void ExecuteFindInExplorer();

	/** Handler for confirmation of folder deletion */
	FReply ExecuteDeleteFolderConfirmed();

	/** Get tooltip for delete */
	FText GetDeleteToolTip() const;

	/** Try to get AssetAccessSpecifier for asset **/
	bool GetAssetAccessSpecifierFromSelection(EAssetAccessSpecifier& OutAssetAccessSpecifier);

	/** Compares two access specifiers.  Returns < 0 if A less accessible than B, > 0 if A more accessible than B, and 0 if they are equal. **/
	static int32 CompareAssetAccessSpecifiers(EAssetAccessSpecifier A, EAssetAccessSpecifier B);

	/** Handler for when new AssetAccessSpecifier is chosen.  */
	void ExecuteSetAssetAccessSpecifier(EAssetAccessSpecifier InAssetAccessSpecifier);

	/** Handler to check to see if can set to a specific AssetAccessSpecifier is allowed */
	bool CanSetAssetAccessSpecifier(const EAssetAccessSpecifier InAssetAccessSpecifier) const;

	/** Handler for setting all selected assets to a specific AssetAccessSpecifier */
	void ExecuteBulkSetAssetAccessSpecifier(EAssetAccessSpecifier InAssetAccessSpecifier);

	/** Handler to check if all selected assets can have their AssetAccessSpecifier changed */
	bool CanExecuteBulkSetAssetAccessSpecifier(EAssetAccessSpecifier InAssetAccessSpecifier);

	/** Handler for determining the selected asset's AssetAccessSpecifier state */
	bool IsSelectedAssetAccessSpecifier(EAssetAccessSpecifier InAssetAccessSpecifier);

	/** Handler for CopyFilePath */
	void ExecuteCopyFilePath();

	/** Handler for when "Remove from collection" is selected */
	void ExecuteRemoveFromCollection();

	/** Handler to check to see if a sync to asset tree command is allowed */
	bool CanExecuteSyncToAssetTree() const;

	/** Handler to check to see if a find in explorer command is allowed */
	bool CanExecuteFindInExplorer() const;	

	/** Handler to check to see if a "Remove from collection" command is allowed */
	bool CanExecuteRemoveFromCollection() const;

	/** Initializes some variable used to in "CanExecute" checks that won't change at runtime or are too expensive to check every frame. */
	void CacheCanExecuteVars();

	/** Registers the base context menu for assets */
	void RegisterContextMenu(const FName MenuName);

	TArray<FContentBrowserItem> SelectedItems;
	TArray<FContentBrowserItem> SelectedFiles;
	TArray<FContentBrowserItem> SelectedFolders;

	FAssetViewContentSources ContentSources;

	/** The asset view this context menu is a part of */
	TWeakPtr<SAssetView> AssetView;

	FOnShowInPathsViewRequested OnShowInPathsViewRequested;
	FOnRenameRequested OnRenameRequested;
	FOnDuplicateRequested OnDuplicateRequested;
	FOnAssetViewRefreshRequested OnAssetViewRefreshRequested;

	/** Cached CanExecute vars */
	bool bCanExecuteFindInExplorer = false;

	bool bCanShowSetAssetAccessMenu = false;

	bool bCanExecuteSetPublicAsset = false;
	bool bCanExecuteSetEpicInternalAsset = false;
	bool bCanExecuteSetPrivateAsset = false;

	bool bCanExecuteBulkSetPublicAsset = false;
	bool bCanExecuteBulkSetEpicInternalAsset = false;
	bool bCanExecuteBulkSetPrivateAsset = false;

	FText DisallowSetPublicReason;
};