// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetSystemContentBrowserInfoProvider.h"

#include "ActorFolder.h"
#include "ActorFolderDesc.h"
#include "Algo/Sort.h"
#include "AssetDefinition.h"
#include "AssetDefinitionAssetInfo.h"
#include "AssetDefinitionRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "AssetViewTypes.h"
#include "AutoReimport/AssetSourceFilenameCache.h"
#include "CollectionManagerModule.h"
#include "Containers/VersePath.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserUtils.h"
#include "IAssetTools.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "IContentBrowserDataModule.h"
#include "Misc/EngineBuildSettings.h"

#define LOCTEXT_NAMESPACE "AssetSystemContentBrowserInfoProvider"

FAssetSystemContentBrowserInfoProvider::FAssetSystemContentBrowserInfoProvider(const TSharedPtr<FAssetViewItem>& InAssetItem)
	: AssetItem(InAssetItem)
{
	if (AssetItem.IsValid())
	{
		OnItemDataChangedCacheDisplayTagsDelegateHandle = AssetItem->OnItemDataChanged().AddRaw(this, &FAssetSystemContentBrowserInfoProvider::CacheDisplayTags);
		OnItemDataChangedCacheDirtyExternalPackageDelegateHandle = AssetItem->OnItemDataChanged().AddRaw(this, &FAssetSystemContentBrowserInfoProvider::CacheDirtyExternalPackageInfo);
	}
	FAssetData AssetData;
	AssetItem->GetItem().Legacy_TryGetAssetData(AssetData);
	if (AssetData.IsValid())
	{
		if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(AssetData))
		{
			bShouldSaveExternalPackages = AssetDefinition->ShouldSaveExternalPackages();
		}
	}
	CacheDisplayTags();
}

FAssetSystemContentBrowserInfoProvider::~FAssetSystemContentBrowserInfoProvider()
{
	if (AssetItem.IsValid())
	{
		AssetItem->OnItemDataChanged().Remove(OnItemDataChangedCacheDisplayTagsDelegateHandle);
		AssetItem->OnItemDataChanged().Remove(OnItemDataChangedCacheDirtyExternalPackageDelegateHandle);
		OnItemDataChangedCacheDisplayTagsDelegateHandle.Reset();
		OnItemDataChangedCacheDirtyExternalPackageDelegateHandle.Reset();
	}
}

void FAssetSystemContentBrowserInfoProvider::PopulateAssetInfo(TArray<FAssetDisplayInfo>& OutAssetDisplayInfo) const
{
	if (AssetItem->IsFile())
	{
		// The tooltip contains the name, class, path, asset registry tags and source control status
		FText PublicStateText;
		const FSlateBrush* PublicStateIcon = nullptr;

		// Create a box to hold every line of info in the body of the tooltip
		TSharedRef<SVerticalBox> InfoBox = SNew(SVerticalBox);

		FAssetData ItemAssetData;
		AssetItem->GetItem().Legacy_TryGetAssetData(ItemAssetData);

		// TODO: Always use the virtual path?
		FAssetDisplayInfo PathInfo = FAssetDisplayInfo();
		PathInfo.StatusTitle = LOCTEXT("TileViewTooltipPath", "Path");
		if (ItemAssetData.IsValid())
		{
			PathInfo.StatusDescription = FText::FromName(ItemAssetData.PackagePath);
		}
		else
		{
			PathInfo.StatusDescription = IContentBrowserDataModule::Get().GetSubsystem()->ConvertVirtualPathToDisplay(AssetItem->GetItem());
		}
		OutAssetDisplayInfo.Add(PathInfo);

		if (ItemAssetData.IsValid() && FAssetToolsModule::GetModule().Get().ShowingContentVersePath())
		{
			UE::Core::FVersePath VersePath = ItemAssetData.GetVersePath();
			if (VersePath.IsValid())
			{
				FAssetDisplayInfo& VersePathInfo = OutAssetDisplayInfo.AddDefaulted_GetRef();
				VersePathInfo.StatusTitle = LOCTEXT("TileViewTooltipVersePath", "Verse Path");
				VersePathInfo.StatusDescription = FText::FromString(VersePath.ToString());
			}
		}

		if (ItemAssetData.IsValid() && ItemAssetData.PackageName != NAME_None)
		{
			const FString PackagePathWithinRoot = ContentBrowserUtils::GetPackagePathWithinRoot(ItemAssetData.PackageName.ToString());
			int32 PackageNameLength = PackagePathWithinRoot.Len();

			int32 MaxAssetPathLen = ContentBrowserUtils::GetMaxAssetPathLen();

			// Asset Path Length Info
			FAssetDisplayInfo AssetPathLengthInfo = FAssetDisplayInfo();
			AssetPathLengthInfo.StatusTitle = LOCTEXT("TileViewTooltipAssetPathLengthKey", "Asset Filepath Length");
			AssetPathLengthInfo.StatusDescription = FText::Format(LOCTEXT("TileViewTooltipAssetPathLengthValue", "{0} / {1}"), FText::AsNumber(PackageNameLength), FText::AsNumber(MaxAssetPathLen));
			OutAssetDisplayInfo.Add(AssetPathLengthInfo);

			int32 PackageNameLengthForCooking = ContentBrowserUtils::GetPackageLengthForCooking(ItemAssetData.PackageName.ToString(), FEngineBuildSettings::IsInternalBuild());

			// Cook Path Length Info
			int32 MaxCookPathLen = ContentBrowserUtils::GetMaxCookPathLen();
			FAssetDisplayInfo CookPathLenInfo = FAssetDisplayInfo();
			CookPathLenInfo.StatusTitle = LOCTEXT("TileViewTooltipPathLengthForCookingKey", "Cooking Filepath Length");
			CookPathLenInfo.StatusDescription = FText::Format(LOCTEXT("TileViewTooltipPathLengthForCookingValue", "{0} / {1}"), FText::AsNumber(PackageNameLengthForCooking), FText::AsNumber(MaxCookPathLen));
			OutAssetDisplayInfo.Add(CookPathLenInfo);
			
			PublicStateText = AssetItem->GetItemAssetAccessSpecifierText();
			if (!PublicStateText.IsEmpty())
			{
				// An icon is required for the PublicStateText to display. So, set one if there's a value to show.
				PublicStateIcon = FAppStyle::GetBrush(AssetItem->GetItemAssetAccessSpecifierIconStyleName(TEXT(".Small")));
			}
		}

		if (!AssetItem->GetItem().CanEdit())
		{
			if(AssetItem->GetItem().CanView())
			{
				PublicStateText = LOCTEXT("ViewReadOnlyAssetState", "View / Read Only");
				PublicStateIcon = FAppStyle::GetBrush("AssetEditor.ReadOnlyOpenable");

			}
			else
			{
				PublicStateText = LOCTEXT("ReadOnlyAssetState", "Read Only");
				PublicStateIcon = FAppStyle::GetBrush("Icons.Lock");
			}
		}

		if(!AssetItem->GetItem().IsSupported())
		{
			PublicStateText = LOCTEXT("UnsupportedAssetState", "Unsupported");
		}

		// Add tags
		for (const FTagContentBrowserDisplayItem& DisplayTagItem : CachedDisplayTags)
		{
			FAssetDisplayInfo DisplayTagInfo = FAssetDisplayInfo();
			DisplayTagInfo.StatusTitle = DisplayTagItem.DisplayKey;
			DisplayTagInfo.StatusDescription = DisplayTagItem.DisplayValue;
			OutAssetDisplayInfo.Add(DisplayTagInfo);
		}

		// Add asset source files
		if (ItemAssetData.IsValid())
		{
			TOptional<FAssetImportInfo> ImportInfo = FAssetSourceFilenameCache::ExtractAssetImportInfo(ItemAssetData);
			if (ImportInfo.IsSet())
			{
				for (const FAssetImportInfo::FSourceFile& File : ImportInfo->SourceFiles)
				{
					FText SourceLabel = LOCTEXT("TileViewTooltipSourceFile", "Source File");
					if (File.DisplayLabelName.Len() > 0)
					{
						SourceLabel = FText::FromString(FText(LOCTEXT("TileViewTooltipSourceFile", "Source File")).ToString() + TEXT(" (") + File.DisplayLabelName + TEXT(")"));
					}
					FAssetDisplayInfo SourceFileInfo = FAssetDisplayInfo();
					SourceFileInfo.StatusTitle = SourceLabel;
					SourceFileInfo.StatusDescription = FText::FromString(File.RelativeFilename);
					OutAssetDisplayInfo.Add(SourceFileInfo);
				}
			}
		}

		static const IConsoleVariable* EnablePublicAssetFeatureCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("AssetTools.EnablePublicAssetFeature"));
		const bool bIsPublicAssetUIEnabled = EnablePublicAssetFeatureCVar && EnablePublicAssetFeatureCVar->GetBool();

		// Restriction Info
		FAssetDisplayInfo RestrictionInfo = FAssetDisplayInfo();
		RestrictionInfo.IsVisible = bIsPublicAssetUIEnabled && !PublicStateText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed;
		RestrictionInfo.StatusIcon = PublicStateIcon;
		RestrictionInfo.StatusTitle = LOCTEXT("Restriction", "Restriction");
		RestrictionInfo.StatusDescription = PublicStateText;
		OutAssetDisplayInfo.Add(RestrictionInfo);

		// Unsupported Info
		FAssetDisplayInfo UnsupportedInfo = FAssetDisplayInfo();
		UnsupportedInfo.IsVisible = AssetItem->GetItem().IsSupported() ? EVisibility::Collapsed : EVisibility::Visible;;
		UnsupportedInfo.StatusTitle = LOCTEXT("UnsupportedAssetTitleText", "Item is not supported");
		UnsupportedInfo.StatusDescription = LOCTEXT("UnsupportedAssetDescriptionText", "This type of asset is not allowed in this project. Delete unsupported assets to avoid errors.");
		OutAssetDisplayInfo.Add(UnsupportedInfo);

		// External Package Info
		FAssetDisplayInfo ExternalPackageInfo = FAssetDisplayInfo();
		ExternalPackageInfo.IsVisible = bShouldSaveExternalPackages && !GetExternalPackagesText().IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed;;
		ExternalPackageInfo.StatusTitle = LOCTEXT("DirtyExternalPackages", "Modified external packages");
		ExternalPackageInfo.StatusDescription = GetExternalPackagesText();
		OutAssetDisplayInfo.Add(ExternalPackageInfo);

		// User Description
		FAssetDisplayInfo UserDescriptionInfo = FAssetDisplayInfo();
		UserDescriptionInfo.IsVisible = GetAssetUserDescription().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;;
		UserDescriptionInfo.StatusTitle = LOCTEXT("UserDescriptionTitle", "User Description");
		UserDescriptionInfo.StatusDescription = GetAssetUserDescription();
		OutAssetDisplayInfo.Add(UserDescriptionInfo);

		// Collection Pips
		if (ItemAssetData.IsValid())
		{
			ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();

			TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
			CollectionManager.GetVisibleCollectionContainers(CollectionContainers);

			TArray<FCollectionNameType> CollectionsContainingObject;
			for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
			{
				CollectionsContainingObject.Reset();
				CollectionContainer->GetCollectionsContainingObject(ItemAssetData.ToSoftObjectPath(), CollectionsContainingObject);

				Algo::Sort(CollectionsContainingObject, [](const FCollectionNameType& A, const FCollectionNameType& B)
					{
						int32 result = A.Name.Compare(B.Name);
						return result < 0 || (result == 0 && A.Type < B.Type);
					});

				bool bAddedCollectionHeader = false;
				for (const FCollectionNameType& CollectionContainingObject : CollectionsContainingObject)
				{
					FCollectionStatusInfo CollectionStatusInfo;
					if (CollectionContainer->GetCollectionStatusInfo(CollectionContainingObject.Name, CollectionContainingObject.Type, CollectionStatusInfo))
					{
						if (!bAddedCollectionHeader)
						{
							// StatusTitle currently used to add a separator for status, need to be changed in future version to allow more configurability
							bAddedCollectionHeader = true;
							FAssetDisplayInfo CollectionHeader = FAssetDisplayInfo();
							CollectionHeader.StatusTitle = LOCTEXT("CollectionHeaderTitle", "Collection(s)");
							CollectionHeader.StatusDescription = FText::GetEmpty();
							OutAssetDisplayInfo.Add(CollectionHeader);
						}

						FAssetDisplayInfo CollectionInfo = FAssetDisplayInfo();
						CollectionInfo.StatusTitle = FText::FromName(CollectionContainingObject.Name);
						CollectionInfo.StatusDescription = FText::AsNumber(CollectionStatusInfo.NumObjects);
						OutAssetDisplayInfo.Add(CollectionInfo);
					}
				}
			}
		}
	}
}

void FAssetSystemContentBrowserInfoProvider::CacheDisplayTags()
{
	CachedDisplayTags.Reset();

	const FContentBrowserItemDataAttributeValues AssetItemAttributes = AssetItem->GetItem().GetItemAttributes(/*bIncludeMetaData*/true);
	
	FAssetData ItemAssetData;
	AssetItem->GetItem().Legacy_TryGetAssetData(ItemAssetData);

	// Add all visible attributes
	for (const auto& AssetItemAttributePair : AssetItemAttributes)
	{
		const FName AttributeName = AssetItemAttributePair.Key;
		const FContentBrowserItemDataAttributeValue& AttributeValue = AssetItemAttributePair.Value;
		const FContentBrowserItemDataAttributeMetaData& AttributeMetaData = AttributeValue.GetMetaData();

		if (AttributeMetaData.AttributeType == UObject::FAssetRegistryTag::TT_Hidden)
		{
			continue;
		}
	
		// Build the display value for this attribute
		FText DisplayValue;
		if (AttributeValue.GetValueType() == EContentBrowserItemDataAttributeValueType::Text)
		{
			DisplayValue = AttributeValue.GetValueText();
		}
		else
		{
			const FString AttributeValueStr = AttributeValue.GetValue<FString>();

			auto ReformatNumberStringForDisplay = [](const FString& InNumberString) -> FText
			{
				// Respect the number of decimal places in the source string when converting for display
				int32 NumDecimalPlaces = 0;
				{
					int32 DotIndex = INDEX_NONE;
					if (InNumberString.FindChar(TEXT('.'), DotIndex))
					{
						NumDecimalPlaces = InNumberString.Len() - DotIndex - 1;
					}
				}
	
				if (NumDecimalPlaces > 0)
				{
					// Convert the number as a double
					double Num = 0.0;
					LexFromString(Num, *InNumberString);
	
					const FNumberFormattingOptions NumFormatOpts = FNumberFormattingOptions()
						.SetMinimumFractionalDigits(NumDecimalPlaces)
						.SetMaximumFractionalDigits(NumDecimalPlaces);
	
					return FText::AsNumber(Num, &NumFormatOpts);
				}

				const bool bIsSigned = InNumberString.Len() > 0 && (InNumberString[0] == TEXT('-') || InNumberString[0] == TEXT('+'));
				if (bIsSigned)
				{
					// Convert the number as a signed int
					int64 Num = 0;
					LexFromString(Num, *InNumberString);
					return FText::AsNumber(Num);
				}

				// Convert the number as an unsigned int
				uint64 Num = 0;
				LexFromString(Num, *InNumberString);
				return FText::AsNumber(Num);
			};
	
			bool bHasSetDisplayValue = false;
	
			// Numerical tags need to format the specified number based on the display flags
			if (!bHasSetDisplayValue && AttributeMetaData.AttributeType == UObject::FAssetRegistryTag::TT_Numerical && AttributeValueStr.IsNumeric())
			{
				bHasSetDisplayValue = true;
	
				const bool bAsMemory = !!(AttributeMetaData.DisplayFlags & UObject::FAssetRegistryTag::TD_Memory);
	
				if (bAsMemory)
				{
					// Memory should be a 64-bit unsigned number of bytes
					uint64 NumBytes = 0;
					LexFromString(NumBytes, *AttributeValueStr);
	
					DisplayValue = FText::AsMemory(NumBytes);
				}
				else
				{
					DisplayValue = ReformatNumberStringForDisplay(AttributeValueStr);
				}
			}
	
			// Dimensional tags need to be split into their component numbers, with each component number re-formatted
			if (!bHasSetDisplayValue && AttributeMetaData.AttributeType == UObject::FAssetRegistryTag::TT_Dimensional)
			{
				// Formats:
				//   123             (1D)
				//   123x234         (2D)
				//   123x234*345     (2D array)
				//   123x234x345     (3D)
				//   123x234x345*456 (3D array)
				int32 FirstXPos;
				if (AttributeValueStr.FindChar(TEXT('x'), FirstXPos))
				{
					FString FirstPart = AttributeValueStr.Left(FirstXPos);
					FString Remainder = AttributeValueStr.Mid(FirstXPos + 1);
					int32 RemainderSeparatorPos;

					if (Remainder.FindChar(TEXT('*'), RemainderSeparatorPos))
					{
						FString SecondPart = Remainder.Left(RemainderSeparatorPos);

						int SecondXPos;
						if (SecondPart.FindChar(TEXT('x'), SecondXPos))
						{
							// AxBxC*D form (3D array)
							FString ThirdPart = Remainder.Mid(SecondXPos + 1);
							FString FourthPart = Remainder.Mid(RemainderSeparatorPos + 1);

							bHasSetDisplayValue = true;
							DisplayValue = FText::Format(LOCTEXT("DisplayTag3xArrayFmt", "{0} \u00D7 {1} \u00D7 {2} ({3} elements)"),
								ReformatNumberStringForDisplay(FirstPart),
								ReformatNumberStringForDisplay(SecondPart),
								ReformatNumberStringForDisplay(ThirdPart),
								ReformatNumberStringForDisplay(FourthPart));
						}
						else
						{
							// AxB*C form (2D array)
							FString ThirdPart = Remainder.Mid(RemainderSeparatorPos + 1);

							bHasSetDisplayValue = true;
							DisplayValue = FText::Format(LOCTEXT("DisplayTag2xArrayFmt", "{0} \u00D7 {1} ({2} elements)"),
								ReformatNumberStringForDisplay(FirstPart),
								ReformatNumberStringForDisplay(SecondPart),
								ReformatNumberStringForDisplay(ThirdPart));
						}
					}
					else if (Remainder.FindChar(TEXT('x'), RemainderSeparatorPos))
					{
						// AxBxC form (3D)
						FString SecondPart = Remainder.Left(RemainderSeparatorPos);
						FString ThirdPart = Remainder.Mid(RemainderSeparatorPos + 1);

						bHasSetDisplayValue = true;
						DisplayValue = FText::Format(LOCTEXT("DisplayTag3xFmt", "{0} \u00D7 {1} \u00D7 {2}"),
							ReformatNumberStringForDisplay(FirstPart),
							ReformatNumberStringForDisplay(SecondPart),
							ReformatNumberStringForDisplay(ThirdPart));
					}
					else
					{
						// 2D form by default
						bHasSetDisplayValue = true;
						DisplayValue = FText::Format(LOCTEXT("DisplayTag2xFmt", "{0} \u00D7 {1}"),
							ReformatNumberStringForDisplay(FirstPart),
							ReformatNumberStringForDisplay(Remainder));
					}
				}
				else
				{
					// No separators, assume 1D
					bHasSetDisplayValue = true;
					DisplayValue = ReformatNumberStringForDisplay(AttributeValueStr);
				}
			}
	
			// Chronological tags need to format the specified timestamp based on the display flags
			if (!bHasSetDisplayValue && AttributeMetaData.AttributeType == UObject::FAssetRegistryTag::TT_Chronological)
			{
				bHasSetDisplayValue = true;
	
				FDateTime Timestamp;
				if (FDateTime::Parse(AttributeValueStr, Timestamp))
				{
					const bool bDisplayDate = !!(AttributeMetaData.DisplayFlags & UObject::FAssetRegistryTag::TD_Date);
					const bool bDisplayTime = !!(AttributeMetaData.DisplayFlags & UObject::FAssetRegistryTag::TD_Time);
					const FString TimeZone = (AttributeMetaData.DisplayFlags & UObject::FAssetRegistryTag::TD_InvariantTz) ? FText::GetInvariantTimeZone() : FString();
	
					if (bDisplayDate && bDisplayTime)
					{
						DisplayValue = FText::AsDateTime(Timestamp, EDateTimeStyle::Short, EDateTimeStyle::Short, TimeZone);
					}
					else if (bDisplayDate)
					{
						DisplayValue = FText::AsDate(Timestamp, EDateTimeStyle::Short, TimeZone);
					}
					else if (bDisplayTime)
					{
						DisplayValue = FText::AsTime(Timestamp, EDateTimeStyle::Short, TimeZone);
					}
				}
			}
	
			// The tag value might be localized text, so we need to parse it for display
			if (!bHasSetDisplayValue && FTextStringHelper::IsComplexText(*AttributeValueStr))
			{
				bHasSetDisplayValue = FTextStringHelper::ReadFromBuffer(*AttributeValueStr, DisplayValue) != nullptr;
			}
	
			// Do our best to build something valid from the string value
			if (!bHasSetDisplayValue)
			{
				bHasSetDisplayValue = true;
	
				// Since all we have at this point is a string, we can't be very smart here.
				// We need to strip some noise off class paths in some cases, but can't load the asset to inspect its UPROPERTYs manually due to performance concerns.
				FString ValueString = FPackageName::ExportTextPathToObjectPath(AttributeValueStr);
	
				const TCHAR StringToRemove[] = TEXT("/Script/");
				if (ValueString.StartsWith(StringToRemove))
				{
					// Remove the class path for native classes, and also remove Engine. for engine classes
					const int32 SizeOfPrefix = UE_ARRAY_COUNT(StringToRemove) - 1;
					ValueString.MidInline(SizeOfPrefix, ValueString.Len() - SizeOfPrefix, EAllowShrinking::No);
					ValueString.ReplaceInline(TEXT("Engine."), TEXT(""));
				}
	
				if (ItemAssetData.IsValid())
				{
					if (const UClass* AssetClass = ItemAssetData.GetClass())
					{
						if (const FProperty* TagField = FindFProperty<FProperty>(AssetClass, AttributeName))
						{
							const FProperty* TagProp = nullptr;
							const UEnum* TagEnum = nullptr;
							if (const FByteProperty* ByteProp = CastField<FByteProperty>(TagField))
							{
								TagProp = ByteProp;
								TagEnum = ByteProp->Enum;
							}
							else if (const FEnumProperty* EnumProp = CastField<FEnumProperty>(TagField))
							{
								TagProp = EnumProp;
								TagEnum = EnumProp->GetEnum();
							}

							// Strip off enum prefixes if they exist
							if (TagProp)
							{
								if (TagEnum)
								{
									const FString EnumPrefix = TagEnum->GenerateEnumPrefix();
									if (EnumPrefix.Len() && ValueString.StartsWith(EnumPrefix))
									{
										ValueString.RightChopInline(EnumPrefix.Len() + 1, EAllowShrinking::No);	// +1 to skip over the underscore
									}
								}

								ValueString = FName::NameToDisplayString(ValueString, false);
							}
						}
					}
				}
	
				DisplayValue = FText::AsCultureInvariant(MoveTemp(ValueString));
			}
			
			// Add suffix to the value, if one is defined for this tag
			if (!AttributeMetaData.Suffix.IsEmpty())
			{
				DisplayValue = FText::Format(LOCTEXT("DisplayTagSuffixFmt", "{0} {1}"), DisplayValue, AttributeMetaData.Suffix);
			}
		}
	
		if (!DisplayValue.IsEmpty())
		{
			CachedDisplayTags.Add(FTagContentBrowserDisplayItem(AttributeName, AttributeMetaData.DisplayName, DisplayValue, AttributeMetaData.bIsImportant));
		}
	}
}

void FAssetSystemContentBrowserInfoProvider::CacheDirtyExternalPackageInfo()
{
	if (bShouldSaveExternalPackages)
	{
		CachedDirtyExternalPackagesList.Empty();
		
		FAssetData AssetData;		
		AssetItem->GetItem().Legacy_TryGetAssetData(AssetData);
		if (AssetData.IsAssetLoaded())
		{
			if(const UObject* Asset = AssetData.GetAsset())
			{
				if (const UPackage* Package = Asset->GetPackage())
				{
					TArray<UPackage*> ExternalPackages = Package->GetExternalPackages();
					IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

					// Mirrored/copied from SSourceControlCommon.cpp
					auto RetrieveAssetName = [](const FAssetData& InAssetData) -> FString
					{
						static const FName NAME_ActorLabel(TEXT("ActorLabel"));
						if (InAssetData.FindTag(NAME_ActorLabel))
						{
							FString ResultAssetName;
							InAssetData.GetTagValue(NAME_ActorLabel, ResultAssetName);
							return ResultAssetName;
						}

						if (InAssetData.FindTag(FPrimaryAssetId::PrimaryAssetDisplayNameTag))
						{
							FString ResultAssetName;
							InAssetData.GetTagValue(FPrimaryAssetId::PrimaryAssetDisplayNameTag, ResultAssetName);
							return ResultAssetName;
						}

						if (InAssetData.AssetClassPath == UActorFolder::StaticClass()->GetClassPathName())
						{
							FString ActorFolderPath = UActorFolder::GetAssetRegistryInfoFromPackage(InAssetData.PackageName).GetDisplayName();
							if (!ActorFolderPath.IsEmpty())
							{
								return ActorFolderPath;
							}
						}

						return InAssetData.AssetName.ToString();
					};
					
					for (const UPackage* ExternalPackage : ExternalPackages)
					{
						if (ExternalPackage->IsDirty())
						{
							TArray<FAssetData> DirtyAssetDataEntries;
							AssetRegistry.GetAssetsByPackageName(*ExternalPackage->GetName(), DirtyAssetDataEntries);

							if (CachedDirtyExternalPackagesList.Len())
							{								
								CachedDirtyExternalPackagesList.Append("\n");
							}
							
							CachedDirtyExternalPackagesList.Append(ExternalPackage->GetPathName());
							
							for (const FAssetData& DirtyAssetData : DirtyAssetDataEntries)
							{
								const FString AssetName = RetrieveAssetName(DirtyAssetData);
								const FString AssetClass = DirtyAssetData.AssetClassPath.GetAssetName().ToString();
						
								CachedDirtyExternalPackagesList.Append("\n\t");
								CachedDirtyExternalPackagesList.Append(FString::Printf(TEXT("%s (%s)"), *AssetName, *AssetClass)); 
							}
						}
					}
				}
			}
		}
	}
}

FText FAssetSystemContentBrowserInfoProvider::GetExternalPackagesText() const
{
	if (CachedDirtyExternalPackagesList.Len())
	{
		return FText::FromString(CachedDirtyExternalPackagesList);
	}

	return FText::GetEmpty();
}

FText FAssetSystemContentBrowserInfoProvider::GetAssetUserDescription() const
{
	if (AssetItem && AssetItem->IsFile())
	{
		FContentBrowserItemDataAttributeValue DescriptionAttributeValue = AssetItem->GetItem().GetItemAttribute(ContentBrowserItemAttributes::ItemDescription);
		if (DescriptionAttributeValue.IsValid())
		{
			return DescriptionAttributeValue.GetValue<FText>();
		}
	}

	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
