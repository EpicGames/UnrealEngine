// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDefinition.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/IDelegateInstance.h"
#include "IAssetSystemInfoProvider.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"

class FAssetViewItem;

class FAssetSystemContentBrowserInfoProvider : public IAssetSystemInfoProvider
{
public:
	FAssetSystemContentBrowserInfoProvider() = delete;
	FAssetSystemContentBrowserInfoProvider(const TSharedPtr<FAssetViewItem>& InAssetItem);
	virtual ~FAssetSystemContentBrowserInfoProvider() override;

	virtual void PopulateAssetInfo(TArray<FAssetDisplayInfo>& OutAssetDisplayInfo) const override;

private:
	/** Cache the Display Tags for this asset, called everytime the ItemAssetData change*/
	void CacheDisplayTags();

	/** Cache the External Package Info if any, called everytime the ItemAssetData change */
	void CacheDirtyExternalPackageInfo();

	/** Get the CachedDirtyExternalPackage Text */
	FText GetExternalPackagesText() const;

	/** Get the Item Description for this item */
	FText GetAssetUserDescription() const;

private:
	/** Data for a cached display tag for this item (used in the tooltip, and also as the display string in column views) */
	struct FTagContentBrowserDisplayItem
	{
		FTagContentBrowserDisplayItem(FName InTagKey, FText InDisplayKey, FText InDisplayValue, const bool InImportant)
			: TagKey(InTagKey)
			, DisplayKey(MoveTemp(InDisplayKey))
			, DisplayValue(MoveTemp(InDisplayValue))
			, bImportant(InImportant)
		{
		}

		FName TagKey;
		FText DisplayKey;
		FText DisplayValue;
		bool bImportant;
	};

	/** The cached display tags for this item */
	TArray<FTagContentBrowserDisplayItem> CachedDisplayTags;

	/** Whether it should save external package */
	bool bShouldSaveExternalPackages = false;

	/** The cached external package to save */
	FString CachedDirtyExternalPackagesList;

	/** Delegate handle of the CacheDisplayTags */
	FDelegateHandle OnItemDataChangedCacheDisplayTagsDelegateHandle;

	/** Delegate handle of the CacheDirtyPackage */
	FDelegateHandle OnItemDataChangedCacheDirtyExternalPackageDelegateHandle;

	/** AssetViewItem of this provider */
	TSharedPtr<FAssetViewItem> AssetItem;
};
