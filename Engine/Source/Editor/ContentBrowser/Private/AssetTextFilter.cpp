// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTextFilter.h"

#include "Algo/Compare.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "Filters.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "String/ParseTokens.h"

namespace AssetTextFilter
{
/** Keys used by TestComplexExpression */
const FName NameKeyName("Name");
const FName PathKeyName("Path");
const FName ClassKeyName("Class");
const FName TypeKeyName("Type");
const FName CollectionKeyName("Collection");
const FName TagKeyName("Tag");

FRWLock CustomHandlerLock;
TArray<IAssetTextFilterHandler*> CustomHandlers;
} // namespace AssetTextFilter

FAssetTextFilter::FAssetTextFilter()
	: ReferencedDynamicCollections()
	, TextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::Complex)
{
	FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();

	CollectionManagerModule.Get().GetCollectionContainers(CollectionContainers);

	// We need to watch for collection changes so that we can keep ReferencedDynamicCollections up-to-date
	for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
	{
		FCollectionContainerHandles& Handles = CollectionContainerHandles.AddDefaulted_GetRef();

		Handles.OnIsHiddenChangedHandle = CollectionContainer->OnIsHiddenChanged().AddRaw(this, &FAssetTextFilter::HandleIsHiddenChanged);
		Handles.OnCollectionCreatedHandle = CollectionContainer->OnCollectionCreated().AddRaw(this, &FAssetTextFilter::HandleCollectionCreated);
		Handles.OnCollectionDestroyedHandle = CollectionContainer->OnCollectionDestroyed().AddRaw(this, &FAssetTextFilter::HandleCollectionDestroyed);
		Handles.OnCollectionRenamedHandle = CollectionContainer->OnCollectionRenamed().AddRaw(this, &FAssetTextFilter::HandleCollectionRenamed);
		Handles.OnCollectionUpdatedHandle = CollectionContainer->OnCollectionUpdated().AddRaw(this, &FAssetTextFilter::HandleCollectionUpdated);
	}

	OnCollectionContainerCreatedHandle = CollectionManagerModule.Get().OnCollectionContainerCreated().AddRaw(this, &FAssetTextFilter::HandleCollectionContainerCreated);
	OnCollectionContainerDestroyedHandle = CollectionManagerModule.Get().OnCollectionContainerDestroyed().AddRaw(this, &FAssetTextFilter::HandleCollectionContainerDestroyed);
}

FAssetTextFilter::~FAssetTextFilter()
{
	// Check IsModuleAvailable as we might be in the process of shutting down...
	if (FCollectionManagerModule::IsModuleAvailable())
	{
		FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();

		CollectionManagerModule.Get().OnCollectionContainerCreated().Remove(OnCollectionContainerCreatedHandle);
		CollectionManagerModule.Get().OnCollectionContainerDestroyed().Remove(OnCollectionContainerDestroyedHandle);

		for (int32 Index = 0; Index < CollectionContainers.Num(); ++Index)
		{
			const TSharedPtr<ICollectionContainer>& CollectionContainer = CollectionContainers[Index];
			FCollectionContainerHandles& Handles = CollectionContainerHandles[Index];

			CollectionContainer->OnIsHiddenChanged().Remove(Handles.OnIsHiddenChangedHandle);
			CollectionContainer->OnCollectionCreated().Remove(Handles.OnCollectionCreatedHandle);
			CollectionContainer->OnCollectionDestroyed().Remove(Handles.OnCollectionDestroyedHandle);
			CollectionContainer->OnCollectionRenamed().Remove(Handles.OnCollectionRenamedHandle);
			CollectionContainer->OnCollectionUpdated().Remove(Handles.OnCollectionUpdatedHandle);
		}
	}
}

bool FAssetTextFilter::IsEmpty() const
{
	return TextFilterExpressionEvaluator.GetFilterText().IsEmpty() && CustomTextFilters.IsEmpty();
}

TSharedPtr<FCompiledAssetTextFilter> FAssetTextFilter::Compile()
{
	TSharedPtr<FTextFilterExpressionEvaluator> CompiledEvaluator;
	TSharedPtr<const TArray<FCollectionRef>> SharedReferencedDynamicCollections;
	if (CustomTextFilters.Num() > 0)
	{
		// Combine main filter and custom saved queries with AND semantics
		TStringBuilder<2048> Builder;
		if (!TextFilterExpressionEvaluator.GetFilterText().IsEmpty())
		{
			Builder << TEXTVIEW("(") << TextFilterExpressionEvaluator.GetFilterText().ToString() << TEXTVIEW(")");
		}
		for (const FText& Text : CustomTextFilters)
		{
			if (Builder.Len() > 0)
			{
				Builder << TEXTVIEW(" AND ");
			}
			Builder << TEXTVIEW("(") << Text.ToString() << TEXTVIEW(")");
		}
		CompiledEvaluator = MakeShared<FTextFilterExpressionEvaluator>(ETextFilterExpressionEvaluatorMode::Complex);
		CompiledEvaluator->SetFilterText(FText::FromStringView(Builder.ToView()));
	
		TArray<FCollectionRef> CombinedReferencedDynamicCollections;
		TextFilterExpressionEvaluator.TestTextFilter(FFrontendFilter_GatherDynamicCollectionsExpressionContext(CollectionContainers, CombinedReferencedDynamicCollections));
		if (CombinedReferencedDynamicCollections.Num())
		{
			SharedReferencedDynamicCollections = MakeShared<const TArray<FCollectionRef>>(MoveTemp(CombinedReferencedDynamicCollections));
		}
	}
	else
	{
		CompiledEvaluator = MakeShared<FTextFilterExpressionEvaluator>(TextFilterExpressionEvaluator);
		if (bReferencedDynamicCollectionsDirty)
		{
			ReferencedDynamicCollections.Reset();
			TextFilterExpressionEvaluator.TestTextFilter(FFrontendFilter_GatherDynamicCollectionsExpressionContext(CollectionContainers, ReferencedDynamicCollections));
		}
		if (ReferencedDynamicCollections.Num())
		{
			SharedReferencedDynamicCollections = MakeShared<const TArray<FCollectionRef>>(ReferencedDynamicCollections);
		}
	}

	TSharedPtr<const TArray<TSharedPtr<ICollectionContainer>>> SharedCollectionContainers;
	if (!CollectionContainers.IsEmpty())
	{
		TArray<TSharedPtr<ICollectionContainer>> TempCollectionContainers;
		TempCollectionContainers.Reserve(CollectionContainers.Num());
		for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
		{
			if (!CollectionContainer->IsHidden())
			{
				TempCollectionContainers.Add(CollectionContainer);
			}
		}

		if (!TempCollectionContainers.IsEmpty())
		{
			SharedCollectionContainers = MakeShared<const TArray<TSharedPtr<ICollectionContainer>>>(MoveTemp(TempCollectionContainers));
		}
	}

	// "Include" flags were always propagated to FFrontendFilter_CustomText, we could in theory save that as part of the data, which would mean we would need separate evaluators.
	return MakeShared<FCompiledAssetTextFilter>(FCompiledAssetTextFilter::FPrivateToken(), CompiledEvaluator.ToSharedRef(), SharedReferencedDynamicCollections,
		SharedCollectionContainers, bIncludeClassName, bIncludeAssetPath, bIncludeCollectionNames);
}

void FAssetTextFilter::SetCustomTextFilters(TArray<FText> InQueries)
{
	if (Algo::Compare(InQueries, CustomTextFilters, [](const FText& A, const FText& B) { return A.EqualTo(B); }) == false)
	{
		CustomTextFilters = MoveTemp(InQueries);
		BroadcastChangedEvent(); // This is likely not strictly necessary as these queries will come from the content browser in the first place 
	}
}

FText FAssetTextFilter::GetRawFilterText() const
{
	return TextFilterExpressionEvaluator.GetFilterText();
}

void FAssetTextFilter::SetRawFilterText(const FText& InFilterText)
{
	if (TextFilterExpressionEvaluator.SetFilterText(InFilterText))
	{
		bReferencedDynamicCollectionsDirty = true;

		// Will trigger a re-filter with the new text
		BroadcastChangedEvent();
	}
}

FText FAssetTextFilter::GetFilterErrorText() const
{
	return TextFilterExpressionEvaluator.GetFilterErrorText();
}

void FAssetTextFilter::SetIncludeClassName(const bool InIncludeClassName)
{
	if (bIncludeClassName != InIncludeClassName)
	{
		bIncludeClassName = InIncludeClassName;

		// Will trigger a re-filter with the new setting
		BroadcastChangedEvent();
	}
}

void FAssetTextFilter::SetIncludeAssetPath(const bool InIncludeAssetPath)
{
	if (bIncludeAssetPath != InIncludeAssetPath)
	{
		bIncludeAssetPath = InIncludeAssetPath;

		// Will trigger a re-filter with the new setting
		BroadcastChangedEvent();
	}
}

bool FAssetTextFilter::GetIncludeAssetPath() const
{
	return bIncludeAssetPath;
}

void FAssetTextFilter::SetIncludeCollectionNames(const bool InIncludeCollectionNames)
{
	if (bIncludeCollectionNames != InIncludeCollectionNames)
	{
		bIncludeCollectionNames = InIncludeCollectionNames;

		// Will trigger a re-filter with the new collections
		BroadcastChangedEvent();
	}
}

bool FAssetTextFilter::GetIncludeCollectionNames() const
{
	return bIncludeCollectionNames;
}

void FAssetTextFilter::HandleCollectionContainerCreated(const TSharedRef<ICollectionContainer>& CollectionContainer)
{
	CollectionContainers.Add(CollectionContainer);
	FCollectionContainerHandles& Handles = CollectionContainerHandles.AddDefaulted_GetRef();

	Handles.OnIsHiddenChangedHandle = CollectionContainer->OnIsHiddenChanged().AddRaw(this, &FAssetTextFilter::HandleIsHiddenChanged);
	Handles.OnCollectionCreatedHandle = CollectionContainer->OnCollectionCreated().AddRaw(this, &FAssetTextFilter::HandleCollectionCreated);
	Handles.OnCollectionDestroyedHandle = CollectionContainer->OnCollectionDestroyed().AddRaw(this, &FAssetTextFilter::HandleCollectionDestroyed);
	Handles.OnCollectionRenamedHandle = CollectionContainer->OnCollectionRenamed().AddRaw(this, &FAssetTextFilter::HandleCollectionRenamed);
	Handles.OnCollectionUpdatedHandle = CollectionContainer->OnCollectionUpdated().AddRaw(this, &FAssetTextFilter::HandleCollectionUpdated);

	if (!CollectionContainer->IsHidden())
	{
		bReferencedDynamicCollectionsDirty = true;

		// Will trigger a re-filter with the new collections
		BroadcastChangedEvent();
	}
}

void FAssetTextFilter::HandleCollectionContainerDestroyed(const TSharedRef<ICollectionContainer>& CollectionContainer)
{
	int32 Index;
	if (CollectionContainers.Find(CollectionContainer, Index))
	{
		{
			FCollectionContainerHandles& Handles = CollectionContainerHandles[Index];

			CollectionContainer->OnIsHiddenChanged().Remove(Handles.OnIsHiddenChangedHandle);
			CollectionContainer->OnCollectionCreated().Remove(Handles.OnCollectionCreatedHandle);
			CollectionContainer->OnCollectionDestroyed().Remove(Handles.OnCollectionDestroyedHandle);
			CollectionContainer->OnCollectionRenamed().Remove(Handles.OnCollectionRenamedHandle);
			CollectionContainer->OnCollectionUpdated().Remove(Handles.OnCollectionUpdatedHandle);
		}

		CollectionContainers.RemoveAt(Index);
		CollectionContainerHandles.RemoveAt(Index);

		if (ReferencedDynamicCollections.ContainsByPredicate([&CollectionContainer](const FCollectionRef& DynamicCollection)
			{
				return DynamicCollection.Container == CollectionContainer;
			}))
		{
			bReferencedDynamicCollectionsDirty = true;

			// Will trigger a re-filter with the new collections
			BroadcastChangedEvent();
		}
	}
}

void FAssetTextFilter::HandleIsHiddenChanged(ICollectionContainer& CollectionContainer, bool bIsHidden)
{
	// Need to refresh when the collection container becomes visible, or it becomes hidden and we are referencing a collection in it.
	if (!bIsHidden || ReferencedDynamicCollections.ContainsByPredicate([&CollectionContainer](const FCollectionRef& DynamicCollection)
		{
			return DynamicCollection.Container.Get() == &CollectionContainer;
		}))
	{
		bReferencedDynamicCollectionsDirty = true;

		// Will trigger a re-filter with the new collections
		BroadcastChangedEvent();
	}
}

void FAssetTextFilter::HandleCollectionCreated(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection)
{
	bReferencedDynamicCollectionsDirty = true;

	// Will trigger a re-filter with the new collections
	BroadcastChangedEvent();
}

void FAssetTextFilter::HandleCollectionDestroyed(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection)
{
	if (ReferencedDynamicCollections.ContainsByPredicate([&CollectionContainer, &Collection](const FCollectionRef& DynamicCollection)
		{
			return DynamicCollection.Container.Get() == &CollectionContainer &&
				DynamicCollection.Name == Collection.Name &&
				DynamicCollection.Type == Collection.Type;
		}))
	{
		bReferencedDynamicCollectionsDirty = true;

		// Will trigger a re-filter with the new collections
		BroadcastChangedEvent();
	}
}

void FAssetTextFilter::HandleCollectionRenamed(
	ICollectionContainer& CollectionContainer, const FCollectionNameType& OriginalCollection, const FCollectionNameType& NewCollection)
{
	for (FCollectionRef& DynamicCollection : ReferencedDynamicCollections)
	{
		if (DynamicCollection.Container.Get() == &CollectionContainer &&
			DynamicCollection.Name == OriginalCollection.Name &&
			DynamicCollection.Type == OriginalCollection.Type)
		{
			DynamicCollection.Name = NewCollection.Name;
			DynamicCollection.Type = NewCollection.Type;
		}
	}
}

void FAssetTextFilter::HandleCollectionUpdated(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection)
{
	bReferencedDynamicCollectionsDirty = true;

	// Will trigger a re-filter with the new collections
	BroadcastChangedEvent();
}

FCompiledAssetTextFilter::FCompiledAssetTextFilter(
	FCompiledAssetTextFilter::FPrivateToken,
	TSharedRef<const FTextFilterExpressionEvaluator> InSharedEvaluator,
	TSharedPtr<const TArray<FCollectionRef>> InSharedReferencedDynamicCollections,
	TSharedPtr<const TArray<TSharedPtr<ICollectionContainer>>> InCollectionContainers,
	bool InIncludeClassName,
	bool InIncludeAssetPath,
	bool InIncludeCollectionNames)
	: Evaluator(MoveTemp(InSharedEvaluator))
	, ReferencedDynamicCollections(MoveTemp(InSharedReferencedDynamicCollections))
	, CollectionContainers(MoveTemp(InCollectionContainers))
	, bIncludeClassName(InIncludeClassName)
	, bIncludeAssetPath(InIncludeAssetPath)
	, bIncludeCollectionNames(InIncludeCollectionNames)
{
	TextBuffer.Reserve(2048);
}

FCompiledAssetTextFilter FCompiledAssetTextFilter::CloneForThreading()
{
	return FCompiledAssetTextFilter(*this);
}

bool FCompiledAssetTextFilter::PassesFilter(FAssetFilterType InItem)
{
	return PassesFilter(InItem, {});
}

bool FCompiledAssetTextFilter::PassesFilter(FAssetFilterType InItem, FStringView InItemVersePath)
{
	using namespace AssetTextFilter;
	FReadScopeLock Guard(CustomHandlerLock);

	AssetPtr = &InItem;
    TextBuffer.Reset();
	AssetCollectionNames.Reset();

	bIsFile = AssetPtr->IsFile();

	int32 DisplayNameLen = 0;
	int32 AssetPathLen = 0;
	int32 AssetVirtualPathLen = 0;
	int32 ExportTextPathLen = 0;
	int32 VersePathLen = 0;

    TextBuffer.Append(AssetPtr->GetDisplayName().ToString());
	DisplayNameLen = TextBuffer.Len();

	if (bIncludeAssetPath)
	{
		int32 TextBufferLenBefore = TextBuffer.Len();

		if (bIsFile)
		{
			if (AssetPtr->AppendItemObjectPath(TextBuffer))
			{
				AssetPathLen = TextBuffer.Len() - TextBufferLenBefore;

				TextBufferLenBefore = TextBuffer.Len();
			}

			AssetPtr->GetVirtualPath().AppendString(TextBuffer);

			AssetVirtualPathLen = TextBuffer.Len() - TextBufferLenBefore;
		}
		else
		{
			FName PackagePath;
			if (AssetPtr->Legacy_TryGetPackagePath(PackagePath))
			{
				PackagePath.AppendString(TextBuffer);

				AssetPathLen = TextBuffer.Len() - TextBufferLenBefore;

				TextBufferLenBefore = TextBuffer.Len();
			}

			AssetPtr->GetVirtualPath().AppendString(TextBuffer);

			AssetVirtualPathLen = TextBuffer.Len() - TextBufferLenBefore;
		}

		if (bIncludeClassName && bIsFile)
		{
			TextBufferLenBefore = TextBuffer.Len();
			// Get the full export text path as people sometimes search by copying this (requires class and asset path
			// search to be enabled in order to match)
			// TODO: this allocates a temporary FString inside the backends
			AssetPtr->AppendItemReference(TextBuffer);
			ExportTextPathLen = TextBuffer.Len() - TextBufferLenBefore;
		}
	}

	if (!InItemVersePath.IsEmpty())
	{
		int32 TextBufferLenBefore = TextBuffer.Len();
		TextBuffer += InItemVersePath;
		VersePathLen = TextBuffer.Len() - TextBufferLenBefore;
	}

    // Convert entire text buffer to uppercase
    // Assumes that for a TCHAR buffer this won't change number of code points 
	TextBuffer.ToUpperInline();
	
	const TCHAR* Cursor = *TextBuffer;
	AssetDisplayName = FStringView(Cursor, DisplayNameLen);
	Cursor += DisplayNameLen;
	AssetFullPath = FStringView(Cursor, AssetPathLen).TrimStartAndEnd();
	Cursor += AssetPathLen;
	AssetVirtualPath = FStringView(Cursor, AssetVirtualPathLen);
	Cursor += AssetVirtualPathLen;
	AssetExportTextPath  = FStringView(Cursor, ExportTextPathLen);
	AssetExportTextPath.TrimStartAndEndInline(); // Backends try to separate export text paths with newlines 
	Cursor += ExportTextPathLen;
	AssetVersePath = FStringView(Cursor, VersePathLen);
	Cursor += VersePathLen;

	// Executed last because we pass *this to DynamicQuery and it would try to access StringView before initialization.
	if ((CollectionContainers.IsValid() || ReferencedDynamicCollections.IsValid()) && bIncludeCollectionNames)
	{
		FSoftObjectPath ItemCollectionId;
		if (AssetPtr->TryGetCollectionId(ItemCollectionId))
		{
			if (CollectionContainers.IsValid())
			{
				for (const TSharedPtr<ICollectionContainer>& CollectionContainer : *CollectionContainers)
				{
					CollectionContainer->GetCollectionsContainingObject(
						ItemCollectionId,
						ECollectionShareType::CST_All,
						AssetCollectionNames,
						ECollectionRecursionFlags::SelfAndChildren);
				}
			}

			if (ReferencedDynamicCollections.IsValid())
			{
				// Test the dynamic collections from the active query against the current asset
				// We can do this as a flat list since FFrontendFilter_GatherDynamicCollectionsExpressionContext has already
				// taken care of processing the recursion
				for (const FCollectionRef& DynamicCollection : *ReferencedDynamicCollections)
				{
					bool bPassesCollectionFilter = false;
					DynamicCollection.Container->TestDynamicQuery(DynamicCollection.Name, DynamicCollection.Type, *this, bPassesCollectionFilter);
					if (bPassesCollectionFilter)
					{
						AssetCollectionNames.AddUnique(DynamicCollection.Name);
					}
				}
			}
		}
	}

	const bool bMatched = Evaluator->TestTextFilter(*this);
	AssetPtr = nullptr;
	return bMatched;
}

bool FCompiledAssetTextFilter::TestBasicStringExpression(
	const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	using namespace AssetTextFilter;
	for (IAssetTextFilterHandler* Handler : CustomHandlers)
	{
		bool bIsHandlerMatch = false;
		if (Handler->HandleTextFilterValue(*AssetPtr, InValue, InTextComparisonMode, bIsHandlerMatch))
		{
			return bIsHandlerMatch;
		}
	}

	if (InValue.CompareName(AssetPtr->GetItemName(), InTextComparisonMode))
	{
		return true;
	}

	if (InValue.CompareFStringView(AssetDisplayName, InTextComparisonMode))
	{
		return true;
	}

	if (bIncludeAssetPath)
	{
		// Mimic how we select the value to display for the path column.
		const FStringView AssetPath = !AssetVersePath.IsEmpty() ? AssetVersePath : (!AssetFullPath.IsEmpty() ? AssetFullPath : AssetVirtualPath);

		if (InValue.CompareFStringView(AssetPath, InTextComparisonMode))
		{
			return true;
		}

		bool bSuccess = false;
		UE::String::ParseTokens(AssetPath, TEXTVIEW("/"), [&bSuccess, &InValue, InTextComparisonMode](FStringView Element){
			bSuccess = bSuccess || InValue.CompareFStringView(Element, InTextComparisonMode);
		});
		if (bSuccess)
		{
			return true;
		}
	}

	if (bIncludeClassName)
	{
		const FContentBrowserItemDataAttributeValue ClassValue = AssetPtr->GetItemAttribute(NAME_Class);
		if (ClassValue.IsValid() && InValue.CompareName(ClassValue.GetValue<FName>(), InTextComparisonMode))
		{
			return true;
		}
	}

	if (bIncludeClassName && bIncludeAssetPath)
	{
		// Only test this if we're searching the class name and asset path too, as the exported text contains the type
		// and path in the string
		if (InValue.CompareFStringView(AssetExportTextPath, InTextComparisonMode))
		{
			return true;
		}
	}

	for (const FName& AssetCollectionName : AssetCollectionNames)
	{
		if (InValue.CompareName(AssetCollectionName, InTextComparisonMode))
		{
			return true;
		}
	}

	return false;
}

bool FCompiledAssetTextFilter::TestComplexExpression(
	const FName& InKey,
	const FTextFilterString& InValue,
	const ETextFilterComparisonOperation InComparisonOperation,
	const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	using namespace AssetTextFilter;
	for (IAssetTextFilterHandler* Handler : CustomHandlers)
	{
		bool bIsHandlerMatch = false;
		if (Handler->HandleTextFilterKeyValue(*AssetPtr, InKey, InValue, InComparisonOperation, InTextComparisonMode, bIsHandlerMatch))
		{
			return bIsHandlerMatch;
		}
	}

	auto ResetTextFilterStringTo = [this](const auto& Value) -> const FTextFilterString&
	{
		TextFilterString = Value;
		return TextFilterString;
	};

	// Special case for the asset name, as this isn't contained within the asset registry meta-data
	if (InKey == NameKeyName)
	{
		// Names can only work with Equal or NotEqual type tests
		if (InComparisonOperation != ETextFilterComparisonOperation::Equal
			&& InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
		{
			return false;
		}

		const bool bIsMatch = TextFilterUtils::TestBasicStringExpression(ResetTextFilterStringTo(AssetPtr->GetItemName()), InValue, InTextComparisonMode);
		return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bIsMatch : !bIsMatch;
	}

	// Special case for the asset path, as this isn't contained within the asset registry meta-data
	if (InKey == PathKeyName)
	{
		// Paths can only work with Equal or NotEqual type tests
		if (InComparisonOperation != ETextFilterComparisonOperation::Equal
			&& InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
		{
			return false;
		}

		FString ObjectPathBuffer;
		FStringView ObjectPath;

		// For the path key we check against all possible path types, not just the value displayed in the path column.
		bool bIsMatch = false;

		// AssetVersePath is always set if available.
		if (!AssetVersePath.IsEmpty() && TextFilterUtils::TestBasicStringExpression(ResetTextFilterStringTo(AssetVersePath), InValue, InTextComparisonMode))
		{
			bIsMatch = true;
		}
		else if (bIncludeAssetPath)
		{
			// Other cached paths are available.

			if (TextFilterUtils::TestBasicStringExpression(ResetTextFilterStringTo(AssetVirtualPath), InValue, InTextComparisonMode))
			{
				bIsMatch = true;
			}
			else if (!AssetFullPath.IsEmpty())
			{
				if (TextFilterUtils::TestBasicStringExpression(ResetTextFilterStringTo(AssetFullPath), InValue, InTextComparisonMode))
				{
					bIsMatch = true;
				}
				else if (bIsFile)
				{
					ObjectPath = AssetFullPath;
				}
			}
		}
		else
		{
			// Other cached paths are NOT available.

			if (TextFilterUtils::TestBasicStringExpression(ResetTextFilterStringTo(AssetPtr->GetVirtualPath()), InValue, InTextComparisonMode))
			{
				bIsMatch = true;
			}
			else
			{
				if (bIsFile)
				{
					if (AssetPtr->AppendItemObjectPath(ObjectPathBuffer))
					{
						if (TextFilterUtils::TestBasicStringExpression(ResetTextFilterStringTo(ObjectPathBuffer), InValue, InTextComparisonMode))
						{
							bIsMatch = true;
						}
						else
						{
							ObjectPath = ObjectPathBuffer;
						}
					}
				}
				else
				{
					FName PackagePath;
					if (AssetPtr->Legacy_TryGetPackagePath(PackagePath))
					{
						if (TextFilterUtils::TestBasicStringExpression(ResetTextFilterStringTo(PackagePath), InValue, InTextComparisonMode))
						{
							bIsMatch = true;
						}
					}
				}
			}
		}

		// If there is no match and the path has an object name, trim that off and try again with the package name, but only for comparison modes where it makes a difference.
		if (!bIsMatch &&
			!ObjectPath.IsEmpty() &&
			(InTextComparisonMode == ETextFilterTextComparisonMode::Exact || InTextComparisonMode == ETextFilterTextComparisonMode::EndsWith))
		{
			int32 LastSlashIndex = INDEX_NONE;
			int32 LastDotIndex = INDEX_NONE;
			if (ObjectPath.FindLastChar(TEXT('/'), LastSlashIndex) &&
				ObjectPath.FindLastChar(TEXT('.'), LastDotIndex) &&
				LastDotIndex > LastSlashIndex &&
				TextFilterUtils::TestBasicStringExpression(ResetTextFilterStringTo(ObjectPath.Left(LastDotIndex)), InValue, InTextComparisonMode))
			{
				bIsMatch = true;
			}
		}

		return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bIsMatch : !bIsMatch;
	}

	// Special case for the asset type, as this isn't contained within the asset registry meta-data
	if (InKey == ClassKeyName || InKey == TypeKeyName)
	{
		// Class names can only work with Equal or NotEqual type tests
		if (InComparisonOperation != ETextFilterComparisonOperation::Equal
			&& InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
		{
			return false;
		}

		const FContentBrowserItemDataAttributeValue ClassValue = AssetPtr->GetItemAttribute(NAME_Class);
		const bool bIsMatch =
			ClassValue.IsValid()
			&& TextFilterUtils::TestBasicStringExpression(ResetTextFilterStringTo(ClassValue.GetValue<FName>()), InValue, InTextComparisonMode);
		return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bIsMatch : !bIsMatch;
	}

	// Special case for collections, as these aren't contained within the asset registry meta-data
	if (InKey == CollectionKeyName || InKey == TagKeyName)
	{
		// Collections can only work with Equal or NotEqual type tests
		if (InComparisonOperation != ETextFilterComparisonOperation::Equal
			&& InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
		{
			return false;
		}

		bool bFoundMatch = false;
		for (const FName& AssetCollectionName : AssetCollectionNames)
		{
			if (TextFilterUtils::TestBasicStringExpression(ResetTextFilterStringTo(AssetCollectionName), InValue, InTextComparisonMode))
			{
				bFoundMatch = true;
				break;
			}
		}

		return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bFoundMatch : !bFoundMatch;
	}

	// Generic handling for anything in the asset meta-data
	{
		const FContentBrowserItemDataAttributeValue AttributeValue = AssetPtr->GetItemAttribute(InKey);
		if (AttributeValue.IsValid())
		{
			return TextFilterUtils::TestComplexExpression(
				AttributeValue.GetValue<FString>(),
				InValue,
				InComparisonOperation,
				InTextComparisonMode);
		}
	}

	return false;
}

void IAssetTextFilterHandler::RegisterHandler()
{
	using namespace AssetTextFilter;
	FWriteScopeLock Guard(CustomHandlerLock);
	CustomHandlers.Add(this);
}

void IAssetTextFilterHandler::UnregisterHandler()
{
	using namespace AssetTextFilter;
	FWriteScopeLock Guard(CustomHandlerLock);
	CustomHandlers.Remove(this);
}