// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetView/AssetViewConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetViewConfig)

TObjectPtr<UAssetViewConfig> UAssetViewConfig::Instance = nullptr;

void UAssetViewConfig::Initialize()
{
	if (!Instance)
	{
		Instance = NewObject<UAssetViewConfig>();
		Instance->AddToRoot();
	}
}

FAssetViewInstanceConfig& UAssetViewConfig::GetInstanceConfig(FName ViewName)
{
	return Instance->Instances.FindOrAdd(ViewName);
}
