// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "EditorConfigBase.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "AssetViewConfig.generated.h"

#define UE_API CONTENTBROWSER_API

USTRUCT()
struct FAssetViewInstanceConfig
{
	GENERATED_BODY()

	/** 
	 * The current thumbnail size, as cast from EThumbnailSize, because that enum is not a UENUM.
	 */
	UPROPERTY()
	uint8 ThumbnailSize = 0; 

	/** 
	 * The current thumbnail size, as cast from EAssetViewType, because that enum is not a UENUM.
	 */
	UPROPERTY()
	uint8 ViewType = 0; 

	UPROPERTY()
	TArray<FName> HiddenColumns;

	UPROPERTY()
	TArray<FName> ListHiddenColumns;
};

UCLASS(MinimalAPI, EditorConfig="AssetView")
class UAssetViewConfig : public UEditorConfigBase
{
	GENERATED_BODY()

public:
	static UE_API void Initialize();
	static UAssetViewConfig* Get() { return Instance; }

	UE_API FAssetViewInstanceConfig& GetInstanceConfig(FName ViewName);
	
	UPROPERTY(meta=(EditorConfig))
	TMap<FName, FAssetViewInstanceConfig> Instances;
	
private:
	static UE_API TObjectPtr<UAssetViewConfig> Instance;
};

#undef UE_API
