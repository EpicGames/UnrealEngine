// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetViewContentSources.h"

#include "ICollectionContainer.h"

FAssetViewContentSources::FAssetViewContentSources(FName InVirtualPath)
	: VirtualPaths({ InVirtualPath })
{
}

FAssetViewContentSources::FAssetViewContentSources(TArray<FName> InVirtualPaths)
	: VirtualPaths(MoveTemp(InVirtualPaths))
{
}

FAssetViewContentSources::FAssetViewContentSources(FCollectionRef InCollection)
	: Collections({ InCollection })
{
	SanitizeCollections();
}

FAssetViewContentSources::FAssetViewContentSources(TArray<FCollectionRef> InCollections)
	: Collections(MoveTemp(InCollections))
{
	SanitizeCollections();
}

FAssetViewContentSources::FAssetViewContentSources(TArray<FName> InVirtualPaths, TArray<FCollectionRef> InCollections)
	: VirtualPaths(MoveTemp(InVirtualPaths))
	, Collections(MoveTemp(InCollections))
{
	// Only one of VirtualPaths or Collections should be set.
	check(VirtualPaths.IsEmpty() || Collections.IsEmpty());

	SanitizeCollections();
}

void FAssetViewContentSources::SetVirtualPath(FName InVirtualPath)
{
	Reset();

	VirtualPaths.Add(InVirtualPath);
}

void FAssetViewContentSources::SetVirtualPaths(const TArray<FName>& InVirtualPaths)
{
	VirtualPaths = InVirtualPaths;

	Collections.Reset();
}

void FAssetViewContentSources::SetCollection(const FCollectionRef& InCollection)
{
	Reset();

	Collections.Add(InCollection);

	SanitizeCollections();
}

void FAssetViewContentSources::SetCollections(const TArray<FCollectionRef>& InCollections)
{
	VirtualPaths.Reset();

	Collections = InCollections;

	SanitizeCollections();
}

bool FAssetViewContentSources::IsDynamicCollection() const
{
	if (Collections.Num() == 1)
	{
		const FCollectionRef& Collection = Collections[0];
		ECollectionStorageMode::Type StorageMode = ECollectionStorageMode::Static;
		return (Collection.Container->GetCollectionStorageMode(Collection.Name, Collection.Type, StorageMode) && StorageMode == ECollectionStorageMode::Dynamic);
	}

	return false;
}

void FAssetViewContentSources::Reset()
{
	VirtualPaths.Reset();

	Collections.Reset();
}

void FAssetViewContentSources::SanitizeCollections()
{
	verify(Collections.RemoveAll([](const FCollectionRef& Collection) { return !Collection.Container.IsValid(); }) == 0);
}