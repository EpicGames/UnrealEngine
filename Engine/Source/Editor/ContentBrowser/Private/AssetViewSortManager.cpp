// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetViewSortManager.h"

#include "AssetViewTypes.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserItemData.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "Misc/CString.h"
#include "Misc/ComparisonUtility.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/UnrealNames.h"

struct FCompareFAssetItemBase
{
public:
	/** Default constructor */
	FCompareFAssetItemBase() = default;

	FCompareFAssetItemBase(const FCompareFAssetItemBase&) = delete;

	virtual ~FCompareFAssetItemBase() = default;

	FCompareFAssetItemBase& operator=(const FCompareFAssetItemBase&) = delete;

	/** Returns true if the struct has cached all item values and the index overload of Compare should be called. */
	virtual bool CacheValues(const TArray<TSharedPtr<FAssetViewItem>>& AssetItems)
	{
		return false;
	}

	/** Compare cached items. */
	virtual int32 Compare(int32 IndexA, int32 IndexB) const
	{
		checkNoEntry();
		return 0;
	}

	/** Compare uncached items. */
	virtual int32 Compare(const FAssetViewItem& A, const FAssetViewItem& B) const = 0;
};

struct FCompareFAssetItemByName final : public FCompareFAssetItemBase
{
public:
	FCompareFAssetItemByName() = default;

	int32 Compare(const FAssetViewItem& A, const FAssetViewItem& B) const override
	{
		// TODO: Have an option to sort by display name? It's slower, but more correct for non-English languages
		//return UE::ComparisonUtility::CompareWithNumericSuffix(FNameBuilder(A.GetItem().GetItemName()).ToView(), FNameBuilder(B.GetItem().GetItemName()).ToView());
		return UE::ComparisonUtility::CompareWithNumericSuffix(A.GetItem().GetDisplayName().ToString(), B.GetItem().GetDisplayName().ToString());
	}
};

struct FCompareFAssetItemByClass final : public FCompareFAssetItemBase
{
public:
	FCompareFAssetItemByClass() = default;

	bool CacheValues(const TArray<TSharedPtr<FAssetViewItem>>& AssetItems) override
	{
		CachedValues.Reset(AssetItems.Num());

		for (const TSharedPtr<FAssetViewItem>& AssetItem : AssetItems)
		{
			CachedValues.Emplace(AssetItem->GetItem());
		}

		return true;
	}

	int32 Compare(int32 IndexA, int32 IndexB) const override
	{
		return CachedValues[IndexA].CompareTo(CachedValues[IndexB]);
	}

	int32 Compare(const FAssetViewItem& A, const FAssetViewItem& B) const override
	{
		FContentBrowserItemDataAttributeValue AttributeValue;

		FText ItemTypeDisplayNameA;
		AttributeValue = A.GetItem().GetItemAttribute(ContentBrowserItemAttributes::ItemTypeDisplayName);
		if (AttributeValue.IsValid())
		{
			ItemTypeDisplayNameA = AttributeValue.GetValue<FText>();
		}

		FText ItemTypeDisplayNameB;
		AttributeValue = B.GetItem().GetItemAttribute(ContentBrowserItemAttributes::ItemTypeDisplayName);
		if (AttributeValue.IsValid())
		{
			ItemTypeDisplayNameB = AttributeValue.GetValue<FText>();
		}

		FString ItemTypeNameA;
		auto GetItemTypeNameA = [&A, &AttributeValue, &ItemTypeNameA]() -> const FString&
		{
			if (ItemTypeNameA.IsEmpty())
			{
				AttributeValue = A.GetItem().GetItemAttribute(ContentBrowserItemAttributes::ItemTypeName);
				if (AttributeValue.IsValid())
				{
					ItemTypeNameA = AttributeValue.GetValue<FString>();
				}
			}
			return ItemTypeNameA;
		};

		FString ItemTypeNameB;
		auto GetItemTypeNameB = [&B, &AttributeValue, &ItemTypeNameB]() -> const FString&
		{
			if (ItemTypeNameB.IsEmpty())
			{
				AttributeValue = B.GetItem().GetItemAttribute(ContentBrowserItemAttributes::ItemTypeName);
				if (AttributeValue.IsValid())
				{
					ItemTypeNameB = AttributeValue.GetValue<FString>();
				}
			}
			return ItemTypeNameB;
		};

		return Compare(ItemTypeDisplayNameA, ItemTypeDisplayNameB, GetItemTypeNameA, GetItemTypeNameB);
	}

private:
	template <typename GetItemTypeNameAType, typename GetItemTypeNameBType>
	static int32 Compare(const FText& ItemTypeDisplayNameA, const FText& ItemTypeDisplayNameB, GetItemTypeNameAType GetItemTypeNameA, GetItemTypeNameBType GetItemTypeNameB)
	{
		int32 Result = 0;

		if (!ItemTypeDisplayNameA.IsEmpty())
		{
			if (!ItemTypeDisplayNameB.IsEmpty())
			{
				Result = ItemTypeDisplayNameA.CompareToCaseIgnored(ItemTypeDisplayNameB);
			}
			else
			{
				Result = FTextComparison::CompareToCaseIgnored(ItemTypeDisplayNameA.ToString(), GetItemTypeNameB());
			}
		}
		else if (!ItemTypeDisplayNameB.IsEmpty())
		{
			Result = FTextComparison::CompareToCaseIgnored(GetItemTypeNameA(), ItemTypeDisplayNameB.ToString());
		}

		if (Result != 0)
		{
			return Result;
		}

		return FTextComparison::CompareToCaseIgnored(GetItemTypeNameA(), GetItemTypeNameB());
	}

	struct FCachedValue
	{
	public:
		explicit FCachedValue(const FContentBrowserItem& Item)
		{
			FContentBrowserItemDataAttributeValue AttributeValue = Item.GetItemAttribute(ContentBrowserItemAttributes::ItemTypeDisplayName);
			if (AttributeValue.IsValid())
			{
				ItemTypeDisplayName = AttributeValue.GetValue<FText>();
			}

			AttributeValue = Item.GetItemAttribute(ContentBrowserItemAttributes::ItemTypeName);
			if (AttributeValue.IsValid())
			{
				ItemTypeName = AttributeValue.GetValue<FString>();
			}
		}

		int32 CompareTo(const FCachedValue& Other) const
		{
			return Compare(ItemTypeDisplayName, Other.ItemTypeDisplayName, [this]() -> const FString& { return ItemTypeName; }, [&Other]() -> const FString& { return Other.ItemTypeName; });
		}

	private:
		FText ItemTypeDisplayName;
		FString ItemTypeName;
	};

	TArray<FCachedValue> CachedValues;
};

struct FCompareFAssetItemByDiskSize final : public FCompareFAssetItemBase
{
public:
	FCompareFAssetItemByDiskSize() = default;

	bool CacheValues(const TArray<TSharedPtr<FAssetViewItem>>& AssetItems) override
	{
		CachedValues.Reset(AssetItems.Num());

		for (const TSharedPtr<FAssetViewItem>& AssetItem : AssetItems)
		{
			CachedValues.Add(GetDiskSize(AssetItem->GetItem()));
		}

		return true;
	}

	int32 Compare(int32 IndexA, int32 IndexB) const override
	{
		return Compare(CachedValues[IndexA], CachedValues[IndexB]);
	}

	int32 Compare(const FAssetViewItem& A, const FAssetViewItem& B) const override
	{
		return Compare(GetDiskSize(A.GetItem()), GetDiskSize(B.GetItem()));
	}

private:
	static int32 Compare(int64 ValueA, int64 ValueB)
	{
		return ValueA < ValueB ? -1 : (ValueA > ValueB ? 1 : 0);
	}

	static int64 GetDiskSize(const FContentBrowserItem& Item)
	{
		FContentBrowserItemDataAttributeValue DiskSizeValue = Item.GetItemAttribute(ContentBrowserItemAttributes::ItemDiskSize);
		return DiskSizeValue.IsValid() ? DiskSizeValue.GetValue<int64>() : -1; // -1 is for items where size doesn't apply - ie. folders
	}

	TArray<int64> CachedValues;
};

struct FCompareFAssetItemByPath final : public FCompareFAssetItemBase
{
public:
	FCompareFAssetItemByPath() = default;

	int32 Compare(const FAssetViewItem& A, const FAssetViewItem& B) const override
	{
		return UE::ComparisonUtility::CompareWithNumericSuffix(A.GetAssetPathText().ToString(), B.GetAssetPathText().ToString());
	}
};

struct FCompareFAssetItemByAssetAccessSpecifier final : public FCompareFAssetItemBase
{
public:
	FCompareFAssetItemByAssetAccessSpecifier() = default;

	bool CacheValues(const TArray<TSharedPtr<FAssetViewItem>>& AssetItems) override
	{
		CachedValues.Reset(AssetItems.Num());

		for (const TSharedPtr<FAssetViewItem>& AssetItem : AssetItems)
		{
			CachedValues.Add(AssetItem->GetItemAssetAccessSpecifierText());
		}

		return true;
	}

	int32 Compare(int32 IndexA, int32 IndexB) const override
	{
		return Compare(CachedValues[IndexA], CachedValues[IndexB]);
	}

	int32 Compare(const FAssetViewItem& A, const FAssetViewItem& B) const override
	{
		return Compare(A.GetItemAssetAccessSpecifierText(), B.GetItemAssetAccessSpecifierText());
	}

private:
	static int32 Compare(const FText& ValueA, const FText& ValueB)
	{
		const bool bValueAIsEmpty = ValueA.IsEmpty();
		const bool bValueBIsEmpty = ValueB.IsEmpty();

		if (bValueAIsEmpty && bValueBIsEmpty)
		{
			return 0;
		}

		if (bValueAIsEmpty)
		{
			return 1;
		}

		if (bValueBIsEmpty)
		{
			return -1;
		}

		return ValueA.CompareTo(ValueB);
	}

	TArray<FText> CachedValues;
};

struct FCompareFAssetItemByTag final : public FCompareFAssetItemBase
{
public:
	explicit FCompareFAssetItemByTag(FName InTag)
		: Tag(InTag)
	{
	}

	bool CacheValues(const TArray<TSharedPtr<FAssetViewItem>>& AssetItems) override
	{
		CachedValues.Reset(AssetItems.Num());

		for (const TSharedPtr<FAssetViewItem>& AssetItem : AssetItems)
		{
			FString Value;
			if (AssetItem->GetTagValue(Tag, Value))
			{
				CachedValues.Emplace(MoveTemp(Value));
			}
			else
			{
				CachedValues.Emplace();
			}
		}

		return true;
	}

	int32 Compare(int32 IndexA, int32 IndexB) const override
	{
		const TOptional<FString>& ValueA = CachedValues[IndexA];
		const TOptional<FString>& ValueB = CachedValues[IndexB];

		if (ValueA.IsSet() && ValueB.IsSet())
		{
			return ValueA.GetValue().Compare(ValueB.GetValue(), ESearchCase::IgnoreCase);
		}

		return ValueA.IsSet() ? -1 : (ValueB.IsSet() ? 1 : 0);
	}

	int32 Compare(const FAssetViewItem& A, const FAssetViewItem& B) const override
	{
		FString ValueA;
		const bool bFoundValueA = A.GetTagValue(Tag, ValueA);

		FString ValueB;
		const bool bFoundValueB = B.GetTagValue(Tag, ValueB);

		if (bFoundValueA && bFoundValueB)
		{
			return ValueA.Compare(ValueB, ESearchCase::IgnoreCase);
		}

		return bFoundValueA ? -1 : (bFoundValueB ? 1 : 0);
	}

private:
	const FName Tag;
	TArray<TOptional<FString>> CachedValues;
};

struct FCompareFAssetItemByTagNumerical final : public FCompareFAssetItemBase
{
public:
	explicit FCompareFAssetItemByTagNumerical(FName InTag)
		: Tag(InTag)
	{
	}

	bool CacheValues(const TArray<TSharedPtr<FAssetViewItem>>& AssetItems) override
	{
		CachedValues.Reset(AssetItems.Num());

		FString Value;
		for (const TSharedPtr<FAssetViewItem>& AssetItem : AssetItems)
		{
			if (AssetItem->GetTagValue(Tag, Value))
			{
				float FloatValue = 0.0f;
				LexFromString(FloatValue, *Value);
				CachedValues.Emplace(FloatValue);
			}
			else
			{
				CachedValues.Emplace();
			}
		}

		return true;
	}

	int32 Compare(int32 IndexA, int32 IndexB) const override
	{
		const TOptional<float>& ValueA = CachedValues[IndexA];
		const TOptional<float>& ValueB = CachedValues[IndexB];

		if (ValueA.IsSet() && ValueB.IsSet())
		{
			return ValueA.GetValue() < ValueB.GetValue() ? -1 : (ValueA.GetValue() > ValueB.GetValue() ? 1 : 0);
		}

		return ValueA.IsSet() ? -1 : (ValueB.IsSet() ? 1 : 0);
	}

	int32 Compare(const FAssetViewItem& A, const FAssetViewItem& B) const override
	{
		FString ValueA;
		const bool bFoundValueA = A.GetTagValue(Tag, ValueA);

		FString ValueB;
		const bool bFoundValueB = B.GetTagValue(Tag, ValueB);

		if (bFoundValueA && bFoundValueB)
		{
			float FloatValueA = 0.0f, FloatValueB = 0.0f;
			LexFromString(FloatValueA, *ValueA);
			LexFromString(FloatValueB, *ValueB);

			return FloatValueA < FloatValueB ? -1 : (FloatValueA > FloatValueB ? 1 : 0);
		}

		return bFoundValueA ? -1 : (bFoundValueB ? 1 : 0);
	}

private:
	const FName Tag;
	TArray<TOptional<float>> CachedValues;
};

struct FCompareFAssetItemByTagDimensional final : public FCompareFAssetItemBase
{
public:
	explicit FCompareFAssetItemByTagDimensional(FName InTag)
		: Tag(InTag)
	{
	}

	bool CacheValues(const TArray<TSharedPtr<FAssetViewItem>>& AssetItems) override
	{
		CachedValues.Reset(AssetItems.Num());

		FString Value;
		TArray<FString> Tokens;
		for (const TSharedPtr<FAssetViewItem>& AssetItem : AssetItems)
		{
			if (AssetItem->GetTagValue(Tag, Value))
			{
				CachedValues.Emplace(ComputeValue(Value, Tokens));
			}
			else
			{
				CachedValues.Emplace();
			}
		}

		return true;
	}

	int32 Compare(int32 IndexA, int32 IndexB) const override
	{
		const TOptional<float>& ValueA = CachedValues[IndexA];
		const TOptional<float>& ValueB = CachedValues[IndexB];

		if (ValueA.IsSet() && ValueB.IsSet())
		{
			return ValueA.GetValue() < ValueB.GetValue() ? -1 : (ValueA.GetValue() > ValueB.GetValue() ? 1 : 0);
		}

		return ValueA.IsSet() ? -1 : (ValueB.IsSet() ? 1 : 0);
	}

	int32 Compare(const FAssetViewItem& A, const FAssetViewItem& B) const override
	{
		FString ValueA;
		const bool bFoundValueA = A.GetTagValue(Tag, ValueA);

		FString ValueB;
		const bool bFoundValueB = B.GetTagValue(Tag, ValueB);

		if (bFoundValueA && bFoundValueB)
		{
			TArray<FString> Tokens;
			const float FloatValueA = ComputeValue(ValueA, Tokens);
			const float FloatValueB = ComputeValue(ValueB, Tokens);

			return FloatValueA < FloatValueB ? -1 : (FloatValueA > FloatValueB ? 1 : 0);
		}

		return bFoundValueA ? -1 : (bFoundValueB ? 1 : 0);
	}

private:
	static float ComputeValue(const FString& Value, TArray<FString>& Tokens)
	{
		float FloatValue = 1.f;

		Value.ParseIntoArray(Tokens, TEXT("x"), true);
		for (const FString& Token : Tokens)
		{
			FloatValue *= FCString::Atof(*Token);
		}

		return FloatValue;
	}

	const FName Tag;
	TArray<TOptional<float>> CachedValues;
};

const FName FAssetViewSortManager::NameColumnId = "Name";
const FName FAssetViewSortManager::ClassColumnId = "Class";
const FName FAssetViewSortManager::PathColumnId = "Path";
const FName FAssetViewSortManager::RevisionControlColumnId = "RevisionControl";
const FName FAssetViewSortManager::DiskSizeColumnId = "Size";
const FName FAssetViewSortManager::AssetAccessSpecifierColumnId = "AssetAccessSpecifier";

FAssetViewSortManager::FAssetViewSortManager()
{
	ResetSort();
}

void FAssetViewSortManager::ResetSort()
{
	SortColumnId[EColumnSortPriority::Primary] = NameColumnId;
	SortMode[EColumnSortPriority::Primary] = EColumnSortMode::Ascending;
	for (int32 PriorityIdx = 1; PriorityIdx < EColumnSortPriority::Max; PriorityIdx++)
	{
		SortColumnId[PriorityIdx] = NAME_None;
		SortMode[PriorityIdx] = EColumnSortMode::None;
	}
}

bool FAssetViewSortManager::FindAndRefreshCustomColumn(const TArray<TSharedPtr<FAssetViewItem>>& AssetItems, const FName ColumnName, const TArray<FAssetViewCustomColumn>& CustomColumns, UObject::FAssetRegistryTag::ETagType& TagType) const
{
	TagType = UObject::FAssetRegistryTag::ETagType::TT_Hidden;

	const FAssetViewCustomColumn* FoundColumn = CustomColumns.FindByPredicate([&ColumnName](const FAssetViewCustomColumn& Column)
	{
		return Column.ColumnName == ColumnName;
	});

	if (FoundColumn)
	{
		for (const TSharedPtr<FAssetViewItem>& Item : AssetItems)
		{
			// Update the custom column data
			Item->CacheCustomColumns(MakeArrayView(FoundColumn, 1), true, false, false);
		}

		TagType = FoundColumn->DataType;
		return true;
	}

	return false;
}

void FAssetViewSortManager::SortList(TArray<TSharedPtr<FAssetViewItem>>& AssetItems, const FName& MajorityAssetType, const TArray<FAssetViewCustomColumn>& CustomColumns) const
{
	if (AssetItems.Num() <= 1)
	{
		return;
	}

	//double SortListStartTime = FPlatformTime::Seconds();

	int32 PrimarySign = 0;
	TArray<TPair<TUniquePtr<FCompareFAssetItemBase>, int32>, TInlineAllocator<EColumnSortPriority::Max>> SortMethods;
	bool bHasNameSortMethod = false;
	for (int32 PriorityIdx = 0; PriorityIdx < EColumnSortPriority::Max; PriorityIdx++)
	{
		int32 Sign = 0;
		switch (SortMode[PriorityIdx])
		{
		case EColumnSortMode::None:
			break;
		case EColumnSortMode::Ascending:
			Sign = 1;
			break;
		case EColumnSortMode::Descending:
			Sign = -1;
			break;
		default:
			checkNoEntry();
			break;
		}
		if (Sign == 0)
		{
			break;
		}

		const FName& Tag(SortColumnId[PriorityIdx]);
		if (Tag == NAME_None)
		{
			break;
		}

		TUniquePtr<FCompareFAssetItemBase> SortMethod;

		if (Tag == NameColumnId)
		{
			SortMethod = MakeUnique<FCompareFAssetItemByName>();
			bHasNameSortMethod = true;
		}
		else if (Tag == ClassColumnId)
		{
			SortMethod = MakeUnique<FCompareFAssetItemByClass>();
		}
		else if (Tag == PathColumnId)
		{
			SortMethod = MakeUnique<FCompareFAssetItemByPath>();
		}
		else if (Tag == DiskSizeColumnId)
		{
			SortMethod = MakeUnique<FCompareFAssetItemByDiskSize>();
		}
		else if (Tag == AssetAccessSpecifierColumnId)
		{
			SortMethod = MakeUnique<FCompareFAssetItemByAssetAccessSpecifier>();
		}
		else
		{
			UObject::FAssetRegistryTag::ETagType TagType = UObject::FAssetRegistryTag::TT_Hidden;
			const bool bFoundCustomColumn = FindAndRefreshCustomColumn(AssetItems, Tag, CustomColumns, TagType);
			
			// Find an item of the correct type so that we can get the type to sort on
			if (!bFoundCustomColumn)
			{
				for (const TSharedPtr<FAssetViewItem>& AssetItem : AssetItems)
				{
					const FContentBrowserItemDataAttributeValue ClassValue = AssetItem->GetItem().GetItemAttribute(ContentBrowserItemAttributes::ItemTypeName);
					if (ClassValue.IsValid() && ClassValue.GetValue<FName>() == MajorityAssetType)
					{
						FString UnusedValue;
						AssetItem->GetTagValue(Tag, UnusedValue, &TagType);

						// If we get hidden, that usually means that it is a new tag not serialized in this particular asset. At least 
						// one asset in the list has to have this defined, or else we wouldn't have it in the tag list at all.
						if (TagType != UObject::FAssetRegistryTag::TT_Hidden)
						{
							break;
						}
					}
				}
			}

			if (TagType == UObject::FAssetRegistryTag::TT_Numerical)
			{
				// The property is a Number, compare using atof
				SortMethod = MakeUnique<FCompareFAssetItemByTagNumerical>(Tag);
			}
			else if (TagType == UObject::FAssetRegistryTag::TT_Dimensional)
			{
				// The property is a series of Numbers representing dimensions, compare by using atof for each Number, delimited by an "x"
				SortMethod = MakeUnique<FCompareFAssetItemByTagDimensional>(Tag);
			}
			else if (TagType != UObject::FAssetRegistryTag::ETagType::TT_Hidden)
			{
				// Unknown or alphabetical, sort alphabetically either way
				SortMethod = MakeUnique<FCompareFAssetItemByTag>(Tag);
			}
		}

		if (SortMethod)
		{
			if (SortMethods.IsEmpty())
			{
				PrimarySign = Sign;
			}
			SortMethods.Emplace(MoveTemp(SortMethod), Sign);
		}
	}

	// Sort the list...
	if (SortMethods.Num() > 0)
	{
		// Use the display name as the primary tie breaker.
		if (!bHasNameSortMethod)
		{
			SortMethods.Emplace(MakeUnique<FCompareFAssetItemByName>(), PrimarySign);
		}

		auto CompareType = [](const FAssetViewItem& AssetViewItemA, const FAssetViewItem& AssetViewItemB) -> int32
		{
			// TODO: Have a view option to sort folders first or not (eg, Explorer vs Finder behavior)
			const FContentBrowserItemData* AItemData = AssetViewItemA.GetItem().GetPrimaryInternalItem();
			const FContentBrowserItemData* BItemData = AssetViewItemB.GetItem().GetPrimaryInternalItem();
			if (AItemData && AItemData->IsFolder())
			{
				if (BItemData && BItemData->IsFolder())
				{
					return AItemData->GetDisplayName().CompareTo(BItemData->GetDisplayName());
				}
				return -1;
			}
			else if (BItemData && BItemData->IsFolder())
			{
				return 1;
			}
			return 0;
		};

		// Check if the primary sort method supports a cached fast path.
		if (SortMethods[0].Key->CacheValues(AssetItems))
		{
			TArray<TPair<int32, TSharedPtr<FAssetViewItem>>> IndexedAssetItems;
			IndexedAssetItems.Reserve(AssetItems.Num());
			for (int32 Index = 0; Index < AssetItems.Num(); ++Index)
			{
				IndexedAssetItems.Emplace(Index, MoveTemp(AssetItems[Index]));
			}

			IndexedAssetItems.Sort([PrimarySign, &CompareType, &SortMethods](const TPair<int32, TSharedPtr<FAssetViewItem>>& A, const TPair<int32, TSharedPtr<FAssetViewItem>>& B)
			{
				const FAssetViewItem& AssetViewItemA = *A.Value;
				const FAssetViewItem& AssetViewItemB = *B.Value;

				// Group folders and files first.
				{
					const int32 Result = CompareType(AssetViewItemA, AssetViewItemB);
					if (Result != 0)
					{
						return PrimarySign * Result < 0;
					}
				}

				// Cached sort by primary column.
				{
					const int32 IndexA = A.Key;
					const int32 IndexB = B.Key;

					const int32 Result = SortMethods[0].Key->Compare(IndexA, IndexB);
					if (Result != 0)
					{
						return SortMethods[0].Value * Result < 0;
					}
				}

				// Sort by secondary columns.
				for (int32 Index = 1; Index < SortMethods.Num(); ++Index)
				{
					const TPair<TUniquePtr<FCompareFAssetItemBase>, int32>& SortMethod = SortMethods[Index];
					const int32 Result = SortMethod.Key->Compare(AssetViewItemA, AssetViewItemB);
					if (Result != 0)
					{
						return SortMethod.Value * Result < 0;
					}
				}

				// Use virtual paths as the final tie breaker.
				return PrimarySign * AssetViewItemA.GetItem().GetVirtualPath().Compare(AssetViewItemB.GetItem().GetVirtualPath()) < 0;
			});

			for (int32 Index = 0; Index < AssetItems.Num(); ++Index)
			{
				AssetItems[Index] = MoveTemp(IndexedAssetItems[Index].Value);
			}
		}
		else
		{
			AssetItems.Sort([PrimarySign, &CompareType, &SortMethods](const TSharedPtr<FAssetViewItem>& A, const TSharedPtr<FAssetViewItem>& B)
			{
				const FAssetViewItem& AssetViewItemA = *A;
				const FAssetViewItem& AssetViewItemB = *B;

				// Group folders and files first.
				{
					const int32 Result = CompareType(AssetViewItemA, AssetViewItemB);
					if (Result != 0)
					{
						return PrimarySign * Result < 0;
					}
				}

				// Sort by columns.
				for (const TPair<TUniquePtr<FCompareFAssetItemBase>, int32>& SortMethod : SortMethods)
				{
					const int32 Result = SortMethod.Key->Compare(AssetViewItemA, AssetViewItemB);
					if (Result != 0)
					{
						return SortMethod.Value * Result < 0;
					}
				}

				// Use virtual paths as the final tie breaker.
				return PrimarySign * AssetViewItemA.GetItem().GetVirtualPath().Compare(AssetViewItemB.GetItem().GetVirtualPath()) < 0;
			});
		}
	}

	//UE_LOGF(LogContentBrowser, Warning/*VeryVerbose*/, "FAssetViewSortManager Sort Time: %0.4f seconds.", FPlatformTime::Seconds() - SortListStartTime);
}

void FAssetViewSortManager::ExportColumnsToCSV(TArray<TSharedPtr<FAssetViewItem>>& AssetItems, TArray<FName>& ColumnList, const TArray<FAssetViewCustomColumn>& CustomColumns, FString& OutString) const
{
	// Write column headers
	for (FName Column : ColumnList)
	{
		OutString += Column.ToString();
		OutString += TEXT(",");

		UObject::FAssetRegistryTag::ETagType TagType;
		FindAndRefreshCustomColumn(AssetItems, Column, CustomColumns, TagType);
	}
	OutString += TEXT("\n");

	// Write each asset
	for (TSharedPtr<FAssetViewItem> AssetItem : AssetItems)
	{
		const FContentBrowserItemData* AssetItemData = AssetItem ? AssetItem->GetItem().GetPrimaryInternalItem() : nullptr;
		if (!AssetItemData || !AssetItemData->IsFile())
		{
			continue;
		}

		for (const FName& Column : ColumnList)
		{
			FString ValueString;
		
			if (Column == NameColumnId)
			{
				ValueString = AssetItemData->GetItemName().ToString();
			}
			else if (Column == ClassColumnId)
			{
				AssetItem->GetTagValue(ContentBrowserItemAttributes::ItemTypeName, ValueString);
			}
			else if (Column == PathColumnId)
			{
				ValueString = AssetItem->GetAssetPathText().ToString();
			}
			else if (Column == DiskSizeColumnId)
			{
				AssetItem->GetTagValue(ContentBrowserItemAttributes::ItemDiskSize, ValueString);
			}
			else
			{
				AssetItem->GetTagValue(Column, ValueString);
			}
			
			OutString += TEXT("\"");
			OutString += ValueString.Replace(TEXT("\""), TEXT("\"\""));
			OutString += TEXT("\",");
		}

		OutString += TEXT("\n");
	}
}

void FAssetViewSortManager::SetSortColumnId(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId)
{
	check(InSortPriority < EColumnSortPriority::Max);
	SortColumnId[InSortPriority] = InColumnId;

	// Prevent the same ColumnId being assigned to multiple columns
	bool bOrderChanged = false;
	for (int32 PriorityIdxA = 0; PriorityIdxA < EColumnSortPriority::Max; PriorityIdxA++)
	{
		for (int32 PriorityIdxB = 0; PriorityIdxB < EColumnSortPriority::Max; PriorityIdxB++)
		{
			if (PriorityIdxA != PriorityIdxB)
			{
				if (SortColumnId[PriorityIdxA] == SortColumnId[PriorityIdxB] && SortColumnId[PriorityIdxB] != NAME_None)
				{
					SortColumnId[PriorityIdxB] = NAME_None;
					bOrderChanged = true;
				}
			}
		}
	}

	if (bOrderChanged)
	{
		// If the order has changed, we need to remove any unneeded sorts by bumping the priority of the remaining valid ones
		for (int32 PriorityIdxA = 0, PriorityNum = 0; PriorityNum < EColumnSortPriority::Max - 1; PriorityNum++, PriorityIdxA++)
		{
			if (SortColumnId[PriorityIdxA] == NAME_None)
			{
				for (int32 PriorityIdxB = PriorityIdxA; PriorityIdxB < EColumnSortPriority::Max - 1; PriorityIdxB++)
				{
					SortColumnId[PriorityIdxB] = SortColumnId[PriorityIdxB + 1];
					SortMode[PriorityIdxB] = SortMode[PriorityIdxB + 1];
				}
				SortColumnId[EColumnSortPriority::Max - 1] = NAME_None;
				PriorityIdxA--;
			}
		}
	}
}

void FAssetViewSortManager::SetSortMode(const EColumnSortPriority::Type InSortPriority, const EColumnSortMode::Type InSortMode)
{
	check(InSortPriority < EColumnSortPriority::Max);
	SortMode[InSortPriority] = InSortMode;
}

bool FAssetViewSortManager::SetOrToggleSortColumn(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId)
{
	check(InSortPriority < EColumnSortPriority::Max);
	if (SortColumnId[InSortPriority] != InColumnId)
	{
		// Clicked a new column, default to ascending
		SortColumnId[InSortPriority] = InColumnId;
		SortMode[InSortPriority] = EColumnSortMode::Ascending;
		return true;
	}
	else
	{
		// Clicked the current column, toggle sort mode
		if (SortMode[InSortPriority] == EColumnSortMode::Ascending)
		{
			SortMode[InSortPriority] = EColumnSortMode::Descending;
		}
		else
		{
			SortMode[InSortPriority] = EColumnSortMode::Ascending;
		}
		return false;
	}
}
