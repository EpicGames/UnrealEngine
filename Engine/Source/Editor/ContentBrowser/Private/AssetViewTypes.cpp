// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetViewTypes.h"

#include "AssetRegistry/AssetData.h"
#include "Containers/UnrealString.h"
#include "Containers/VersePath.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserItemData.h"
#include "HAL/Platform.h"
#include "IAssetTools.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/PackageName.h"
#include "Templates/UnrealTemplate.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

FAssetViewItem::FAssetViewItem(int32 InIndex)
	: Index(InIndex)
	, bShowingContentVersePath(false)
	, bHasCachedVersePathText(false)
{
}

FAssetViewItem::FAssetViewItem(int32 InIndex, FContentBrowserItem&& InItem)
	: Item(MoveTemp(InItem))
	, Index(InIndex)
	, bShowingContentVersePath(false)
	, bHasCachedVersePathText(false)
{
	checkf(Item.IsValid(), TEXT("FAssetViewItem was constructed from an invalid item!"));

	CacheAssetPathText();
}

FAssetViewItem::FAssetViewItem(int32 InIndex, const FContentBrowserItem& InItem)
	: Item(InItem)
	, Index(InIndex)
	, bShowingContentVersePath(false)
	, bHasCachedVersePathText(false)
{
	checkf(Item.IsValid(), TEXT("FAssetViewItem was constructed from an invalid item!"));

	CacheAssetPathText();
}

FAssetViewItem::FAssetViewItem(int32 InIndex, FContentBrowserItemData&& InItemData)
	: Item(MoveTemp(InItemData))
	, Index(InIndex)
	, bShowingContentVersePath(false)
	, bHasCachedVersePathText(false)
{
	checkf(Item.IsValid(), TEXT("FAssetViewItem was constructed from an invalid item!"));

	CacheAssetPathText();
}

FAssetViewItem::FAssetViewItem(int32 InIndex, const FContentBrowserItemData& InItemData)
	: Item(InItemData)
	, Index(InIndex)
	, bShowingContentVersePath(false)
	, bHasCachedVersePathText(false)
{
	checkf(Item.IsValid(), TEXT("FAssetViewItem was constructed from an invalid item!"));

	CacheAssetPathText();
}

void FAssetViewItem::ResetItemData(int32 OldIndex, int32 NewIndex, FContentBrowserItemData InItemData)
{
	int32 Expected = OldIndex;
	if (Index.compare_exchange_strong(Expected, NewIndex, std::memory_order_relaxed))
	{
		Item = FContentBrowserItem{MoveTemp(InItemData)};
		// Do not broadcast event here, it will be broadcast on the main thread after bulk building/recycling of items
	}
	else
	{
		checkf(false, TEXT("Concurrency issue detected recycling FAssetViewItem (%s) from old index %d to new index %d - already reassigned to %d"),
			*WriteToString<256>(InItemData.GetVirtualPath()),
			OldIndex, NewIndex, Expected);
	}
}

void FAssetViewItem::BroadcastItemDataChanged()
{
	// Item has changed, update the asset path.
	CacheAssetPathText();

	// If CacheVersePathTextForTextFiltering has been called, update the Verse path too.
	if (bHasCachedVersePathText)
	{
		CacheVersePathText();
	}

	ItemDataChangedEvent.Broadcast();
}

void FAssetViewItem::AppendItemData(const FContentBrowserItem& InItem)
{
	Item.Append(InItem);
	// Do not broadcast event here, caller is responsible for broadcasting in a threadsafe way
}

void FAssetViewItem::AppendItemData(const FContentBrowserItemData& InItemData)
{
	Item.Append(InItemData);
	// Do not broadcast event here, caller is responsible for broadcasting in a threadsafe way
}

void FAssetViewItem::RemoveItemData(const FContentBrowserItem& InItem)
{
	Item.Remove(InItem);
	if (Item.IsValid())
	{
		BroadcastItemDataChanged();
	}
}

void FAssetViewItem::RemoveItemData(const FContentBrowserItemData& InItemData)
{
	Item.Remove(InItemData);
	if (Item.IsValid())
	{
		BroadcastItemDataChanged();
	}
}

void FAssetViewItem::RemoveItemData(const FContentBrowserMinimalItemData& InItemKey)
{
	Item.TryRemove(InItemKey);
	if (Item.IsValid())
	{
		BroadcastItemDataChanged();
	}	
}

void FAssetViewItem::ConfigureAssetPathText(bool bInShowingContentVersePath)
{
	bShowingContentVersePath = bInShowingContentVersePath;
}

void FAssetViewItem::CacheVersePathTextForTextFiltering()
{
	checkf(IsInGameThread(), TEXT("Verse paths can only be computed on the main thread."));

	if (!bHasCachedVersePathText)
	{
		CacheVersePathText();
		bHasCachedVersePathText = true;
	}
}

FText FAssetViewItem::GetAssetClassText() const
{
	if (IsFolder())
	{
		return LOCTEXT("FolderName", "Folder");
	}

	FContentBrowserItemDataAttributeValue DisplayNameAttributeValue = Item.GetItemAttribute(ContentBrowserItemAttributes::ItemTypeDisplayName);
	if (!DisplayNameAttributeValue.IsValid())
	{
		DisplayNameAttributeValue = Item.GetItemAttribute(ContentBrowserItemAttributes::ItemTypeName);
	}
	return DisplayNameAttributeValue.IsValid() ? DisplayNameAttributeValue.GetValue<FText>() : FText();
}

const FText& FAssetViewItem::GetAssetPathText() const
{
	checkf(IsInGameThread(), TEXT("Verse paths can only be computed on the main thread."));

	// Compute the Verse path on demand since it is somewhat expensive and not parallelizable.
	// This function is called when viewing the path column if it is avalible, or sorting on the path column.
	if (bShowingContentVersePath)
	{
		if (!bHasCachedVersePathText)
		{
			CacheVersePathText();
			bHasCachedVersePathText = true;
		}

		if (!CachedVersePathText.IsEmpty())
		{
			return CachedVersePathText;
		}
	}

	return CachedAssetPathText;
}

const FText& FAssetViewItem::GetVersePathTextForTextFiltering() const
{
	checkf(bHasCachedVersePathText, TEXT("Need to call CacheVersePathTextForTextFiltering first."));

	return CachedVersePathText;
}

void FAssetViewItem::ClearCachedCustomColumns()
{
	check(IsInGameThread());
	CachedCustomColumnData.Reset();
	CachedCustomColumnDisplayText.Reset();
}

void FAssetViewItem::CacheCustomColumns(TArrayView<const FAssetViewCustomColumn> CustomColumns, const bool bUpdateSortData, const bool bUpdateDisplayText, const bool bUpdateExisting)
{
	check(IsInGameThread());
	if (bUpdateExisting && CachedCustomColumnData.IsEmpty())
	{
		return;
	}

	for (const FAssetViewCustomColumn& Column : CustomColumns)
	{
		FAssetData ItemAssetData;
		if (Item.Legacy_TryGetAssetData(ItemAssetData))
		{
			if (bUpdateSortData)
			{
				if (bUpdateExisting ? CachedCustomColumnData.Contains(Column.ColumnName) : !CachedCustomColumnData.Contains(Column.ColumnName))
				{
					CachedCustomColumnData.Add(Column.ColumnName, MakeTuple(Column.OnGetColumnData.Execute(ItemAssetData, Column.ColumnName), Column.DataType));
				}
			}

			if (bUpdateDisplayText)
			{
				if (bUpdateExisting ? CachedCustomColumnDisplayText.Contains(Column.ColumnName) : !CachedCustomColumnDisplayText.Contains(Column.ColumnName))
				{
					if (Column.OnGetColumnDisplayText.IsBound())
					{
						CachedCustomColumnDisplayText.Add(Column.ColumnName, Column.OnGetColumnDisplayText.Execute(ItemAssetData, Column.ColumnName));
					}
					else
					{
						CachedCustomColumnDisplayText.Add(Column.ColumnName, FText::AsCultureInvariant(Column.OnGetColumnData.Execute(ItemAssetData, Column.ColumnName)));
					}
				}
			}
		}
	}
}

bool FAssetViewItem::GetCustomColumnDisplayValue(const FName ColumnName, FText& OutText) const
{
	if (const FText* DisplayValue = CachedCustomColumnDisplayText.Find(ColumnName))
	{
		OutText = *DisplayValue;
		return true;
	}

	return false;
}

bool FAssetViewItem::GetCustomColumnValue(const FName ColumnName, FString& OutString, UObject::FAssetRegistryTag::ETagType* OutType) const
{
	if (const auto* ColumnDataPair = CachedCustomColumnData.Find(ColumnName))
	{
		OutString = ColumnDataPair->Key;
		if (OutType)
		{
			*OutType = ColumnDataPair->Value;
		}
		return true;
	}

	return false;
}

bool FAssetViewItem::GetTagValue(const FName Tag, FString& OutString, UObject::FAssetRegistryTag::ETagType* OutType) const
{
	if (GetCustomColumnValue(Tag, OutString, OutType))
	{
		return true;
	}

	FContentBrowserItemDataAttributeValue TagValue = Item.GetItemAttribute(Tag, true);
	if (TagValue.IsValid())
	{
		OutString = TagValue.GetValue<FString>();
		if (OutType)
		{
			*OutType = TagValue.GetMetaData().AttributeType;
		}
		return true;
	}

	return false;
}

bool FAssetViewItem::GetItemAssetAccessSpecifier(EAssetAccessSpecifier& OutAssetAccessSpecifier) const
{
	return Item.GetItemAssetAccessSpecifier(OutAssetAccessSpecifier);
}

FText FAssetViewItem::GetItemAssetAccessSpecifierText() const
{
	// Items that don't have a modifable access specifier shouldn't render anything.
	if (!CanModifyItemAssetAccessSpecifier())
	{
		return FText::GetEmpty();
	}

	EAssetAccessSpecifier AssetAccessSpecifier;
	if (GetItemAssetAccessSpecifier(AssetAccessSpecifier))
	{
		return AssetAccessSpecifierAsText(AssetAccessSpecifier);
	}

	// This would only happen if there's something wrong with the underlying FContentBrowserItem.
	// It somehow returned true for CanModifyItemAssetAccessSpecifier, but didn't implement its GetItemAssetAccessSpecifier or did and returned something unknown.
	ensureMsgf(0, TEXT("Failed to read the EAssetAccessSpecifier value for FContentBrowserItem: '%s'"), *Item.GetItemName().ToString());
	return LOCTEXT("UnknownAssetState", "Unknown");
}

FName FAssetViewItem::GetItemAssetAccessSpecifierIconStyleName(const TCHAR* InSizeSuffix) const
{
	// Items that don't have a modifable access specifier shouldn't render anything.
	FName IconName = NAME_None;

	EAssetAccessSpecifier AssetAccessSpecifier = EAssetAccessSpecifier::Private;
	if (CanModifyItemAssetAccessSpecifier() && GetItemAssetAccessSpecifier(AssetAccessSpecifier) )
	{
		switch (AssetAccessSpecifier)
		{
			case EAssetAccessSpecifier::Public:
			{
				IconName = GetPublicAssetAccessSpecifierIconStyleName(InSizeSuffix);
				break;
			}

			case EAssetAccessSpecifier::Private:
			{
				IconName = GetPrivateAssetAccessSpecifierIconStyleName(InSizeSuffix);
				break;
			}

			case EAssetAccessSpecifier::EpicInternal:
			{
				IconName = GetEpicInternalAssetAccessSpecifierIconStyleName(InSizeSuffix);
				break;
			}

			default:
			{
				ensureMsgf(0, TEXT("Unrecognized EAssetAccessSpecifier value '%d' found for asset: '%s'"), EnumToUnderlyingType(AssetAccessSpecifier), *GetItem().GetItemName().ToString());
				break;
			}
		}
	}

	return IconName;
}

FName FAssetViewItem::GetPublicAssetAccessSpecifierIconStyleName(const TCHAR* InSizeSuffix)
{
	return FName(*FString::Printf(TEXT("ContentBrowser.Shared%s"), InSizeSuffix));
}

FName FAssetViewItem::GetPrivateAssetAccessSpecifierIconStyleName(const TCHAR* InSizeSuffix)
{
	static const FName IconName("Icons.Lock");
	return IconName;
}

FName FAssetViewItem::GetEpicInternalAssetAccessSpecifierIconStyleName(const TCHAR* InSizeSuffix)
{
	static const FName IconName("UnrealCircle.Thin");
	return IconName;
}

bool FAssetViewItem::CanModifyItemAssetAccessSpecifier() const
{
	return Item.CanModifyItemAssetAccessSpecifier();
}

const FContentBrowserItem& FAssetViewItem::GetItem() const
{
	return Item;
}

bool FAssetViewItem::IsFolder() const
{
	return Item.IsFolder();
}

bool FAssetViewItem::IsFile() const
{
	return Item.IsFile();
}

bool FAssetViewItem::IsTemporary() const
{
	return Item.IsTemporary();
}

FSimpleMulticastDelegate& FAssetViewItem::OnItemDataChanged()
{
	return ItemDataChangedEvent;
}

FSimpleDelegate& FAssetViewItem::OnRenameRequested()
{
	return RenameRequestedEvent;
}

FSimpleDelegate& FAssetViewItem::OnRenameCanceled()
{
	return RenameCanceledEvent;
}

FString FAssetViewItem::ItemToString_Debug(TSharedPtr<FAssetViewItem> AssetItem) 
{
	if (AssetItem.IsValid())
	{
		return AssetItem->GetItem().GetVirtualPath().ToString();
	}
	return TEXT("nullptr");
}

void FAssetViewItem::CacheAssetPathText()
{
	if (IsFile())
	{
		FString ObjectPath;
		if (Item.AppendItemObjectPath(ObjectPath))
		{
			CachedAssetPathText = FText::AsCultureInvariant(MoveTemp(ObjectPath));
			return;
		}
	}
	else
	{
		FName PackagePath;
		if (Item.Legacy_TryGetPackagePath(PackagePath))
		{
			CachedAssetPathText = FText::FromName(PackagePath);
			return;
		}
	}

	CachedAssetPathText = FText::FromName(Item.GetVirtualPath());
}

void FAssetViewItem::CacheVersePathText() const
{
	checkf(IsInGameThread(), TEXT("Verse paths can only be computed on the main thread."));

	if (IsFile())
	{
		FAssetData AssetData;
		if (Item.Legacy_TryGetAssetData(AssetData))
		{
			UE::Core::FVersePath VersePath = AssetData.GetVersePath();
			if (VersePath.IsValid())
			{
				CachedVersePathText = MoveTemp(VersePath).AsText();
				return;
			}
		}
	}
	else
	{
		FName PackagePath;
		if (Item.Legacy_TryGetPackagePath(PackagePath))
		{
			UE::Core::FVersePath VersePath = FPackageName::LongPackagePathToVersePath(FNameBuilder(PackagePath));
			if (VersePath.IsValid())
			{
				CachedVersePathText = MoveTemp(VersePath).AsText();
				return;
			}
		}
	}

	CachedVersePathText = {};
}

#undef LOCTEXT_NAMESPACE
