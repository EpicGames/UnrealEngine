// Copyright Epic Games, Inc. All Rights Reserved.


#include "AssetViewWidgets.h"

#include "ActorFolder.h"
#include "ActorFolderDesc.h"
#include "Algo/Sort.h"
#include "AssetDefinitionRegistry.h"
#include "AssetSystemContentBrowserInfoProvider.h"
#include "AssetRegistry/AssetData.h"
#include "AssetTagItemTypes.h"
#include "AssetThumbnail.h"
#include "AssetToolsModule.h"
#include "AssetViewTypes.h"
#include "AssetViewUtils.h"
#include "AutoReimport/AssetSourceFilenameCache.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "CollectionViewUtils.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Containers/VersePath.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserItemData.h"
#include "ContentBrowserModule.h"
#include "ContentBrowserStyle.h"
#include "ContentBrowserUtils.h"
#include "DragDropHandler.h"
#include "EditorFramework/AssetImportData.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/TextLayout.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IAssetTools.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "IContentBrowserDataModule.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "Input/Events.h"
#include "Interfaces/IPluginManager.h"
#include "Internationalization/BreakIterator.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/DateTime.h"
#include "Misc/EngineBuildSettings.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/PackageName.h"
#include "Misc/StringBuilder.h"
#include "Modules/ModuleManager.h"
#include "PluginDescriptor.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Rendering/SlateRenderer.h"
#include "SAssetTagItem.h"
#include "SRenameWindow.h"
#include "SThumbnailEditModeTools.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "SourceControlHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Settings/ContentBrowserSettings.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateTypes.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/StyleDefaults.h"
#include "Styling/WidgetStyle.h"
#include "Templates/Tuple.h"
#include "Types/ISlateMetaData.h"
#include "UObject/Class.h"
#include "UObject/EnumProperty.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/IToolTip.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

class FDragDropEvent;

#define LOCTEXT_NAMESPACE "ContentBrowser"

///////////////////////////////
// FAssetViewModeUtils
///////////////////////////////

namespace AssetViewWidgets 
{
bool IsTopLevelFolder(const FStringView InFolderPath)
{
	int32 SlashCount = 0;
	for (const TCHAR PathChar : InFolderPath)
	{
		if (PathChar == TEXT('/'))
		{
			if (++SlashCount > 1)
			{
				break;
			}
		}
	}

	return SlashCount == 1;
}

bool IsTopLevelFolder(const FName InFolderPath)
{
	return IsTopLevelFolder(FNameBuilder(InFolderPath));
}

}

///////////////////////////////
// FAssetViewModeUtils
///////////////////////////////

TSharedRef<SWidget> FAssetViewItemHelper::CreateListItemContents(SAssetListItem* const InListItem, const TSharedRef<SWidget>& InThumbnail, FName& OutItemShadowBorder, const FMargin& InFolderOverlayPadding)
{
	return CreateListTileItemContents(InListItem, InThumbnail, OutItemShadowBorder, InFolderOverlayPadding, false);
}

TSharedRef<SWidget> FAssetViewItemHelper::CreateTileItemContents(SAssetTileItem* const InTileItem, const TSharedRef<SWidget>& InThumbnail, FName& OutItemShadowBorder, const FMargin& InFolderOverlayPadding, bool bShowAssetAccessSpecifier)
{
	return CreateListTileItemContents(InTileItem, InThumbnail, OutItemShadowBorder, InFolderOverlayPadding, bShowAssetAccessSpecifier);
}

template <typename T>
TSharedRef<SWidget> FAssetViewItemHelper::CreateListTileItemContents(T* const InTileOrListItem, const TSharedRef<SWidget>& InThumbnail, FName& OutItemShadowBorder, const FMargin& InFolderOverlayPadding, bool bShowAssetAccessSpecifier)
{
	TSharedRef<SOverlay> ItemContentsOverlay = SNew(SOverlay);

	OutItemShadowBorder = FName("ContentBrowser.AssetTileItem.DropShadow");

	if (InTileOrListItem->IsFolder())
	{
		// TODO: Allow items to customize their widget
		TSharedPtr<FAssetViewItem>& AssetItem = InTileOrListItem->AssetItem;

		// Default values
		FName FolderBrushName = TEXT("ContentBrowser.ListViewFolderIcon");
		FName FolderShadowBrushName = TEXT("ContentBrowser.FolderItem.DropShadow");
		ContentBrowserUtils::TryGetFolderBrushAndShadowName(AssetItem->GetItem(), FolderBrushName, FolderShadowBrushName);
		
		const bool bCollectionFolder = EnumHasAnyFlags(AssetItem->GetItem().GetItemCategory(), EContentBrowserItemFlags::Category_Collection);
		ECollectionShareType::Type CollectionFolderShareType = ECollectionShareType::CST_All;
		if (bCollectionFolder)
		{
			ContentBrowserUtils::IsCollectionPath(AssetItem->GetItem().GetVirtualPath().ToString(), nullptr, nullptr, &CollectionFolderShareType);
		}

		const FSlateBrush* FolderBaseImage = FAppStyle::GetBrush(FolderBrushName);
		const FSlateBrush* DropShadowImage = FAppStyle::GetBrush(FolderShadowBrushName);

		// Folder base
		// clang-format off
		ItemContentsOverlay->AddSlot()
		.Padding(InFolderOverlayPadding)
		[
			SNew(SBorder)
			.BorderImage(DropShadowImage)
			.Padding(FMargin(0,0,2.0f,2.0f))
			[
				SNew(SImage)
				.Image(FolderBaseImage)
				.ColorAndOpacity(InTileOrListItem, &T::GetAssetColor)
			]
		];
		// clang-format on

		if (bCollectionFolder)
		{
			FLinearColor IconColor = FLinearColor::White;
			switch(CollectionFolderShareType)
			{
			case ECollectionShareType::CST_Local:
				IconColor = FColor(196, 15, 24);
				break;
			case ECollectionShareType::CST_Private:
				IconColor = FColor(192, 196, 0);
				break;
			case ECollectionShareType::CST_Shared:
				IconColor = FColor(0, 136, 0);
				break;
			default:
				break;
			}

			auto GetCollectionIconBoxSize = [InTileOrListItem]() -> FOptionalSize
			{
				return FOptionalSize(InTileOrListItem->GetThumbnailBoxSize().Get() * 0.3f);
			};

			auto GetCollectionIconBrush = [=]() -> const FSlateBrush*
			{
				const TCHAR* IconSizeSuffix = (GetCollectionIconBoxSize().Get() <= 16.0f) ? TEXT(".Small") : TEXT(".Large");
				return FAppStyle::GetBrush(ECollectionShareType::GetIconStyleName(CollectionFolderShareType, IconSizeSuffix));
			};

			// Collection share type
			ItemContentsOverlay->AddSlot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride_Lambda(GetCollectionIconBoxSize)
				.HeightOverride_Lambda(GetCollectionIconBoxSize)
				[
					SNew(SImage)
					.Image_Lambda(GetCollectionIconBrush)
					.ColorAndOpacity(IconColor)
				]
			];
		}
	}
	else
	{
		// The actual thumbnail
		ItemContentsOverlay->AddSlot()
		[
			InThumbnail
		];

		// Extra external state hook
		ItemContentsOverlay->AddSlot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		[
			SNew(SBox)
			.MaxDesiredWidth(InTileOrListItem, &T::GetExtraStateIconMaxSize)
			.MaxDesiredHeight(InTileOrListItem, &T::GetExtraStateIconMaxSize)
			[
				InTileOrListItem->GenerateExtraStateIconWidget(TAttribute<float>(InTileOrListItem, &T::GetExtraStateIconWidth))
			]
		];

		// Add the icon for the asset's EAssetAccessSpecifier
		if (bShowAssetAccessSpecifier)
		{
			ItemContentsOverlay->AddSlot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			[
				SNew(SBox)
				.MaxDesiredWidth(InTileOrListItem, &T::GetExtraStateIconMaxSize)
				.MaxDesiredHeight(InTileOrListItem, &T::GetExtraStateIconMaxSize)
				[
					InTileOrListItem->GetAssetAccessSpecifierIconBox()
				]
			];
		}

		if (!UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			// Dirty state
			ItemContentsOverlay->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Bottom)
			[
				SNew(SBox)
				.MaxDesiredWidth(InTileOrListItem, &T::GetStateIconImageSize)
				.MaxDesiredHeight(InTileOrListItem, &T::GetStateIconImageSize)
				[
					SNew(SImage)
					.Image(InTileOrListItem, &T::GetDirtyImage)
				]
			];

			// Tools for thumbnail edit mode
			ItemContentsOverlay->AddSlot()
			[
				SNew(SThumbnailEditModeTools, InTileOrListItem->AssetThumbnail)
				.SmallView(!InTileOrListItem->CanDisplayPrimitiveTools())
				.Visibility(InTileOrListItem, &T::GetThumbnailEditModeUIVisibility)
			];
		}
	}
	
	return ItemContentsOverlay;
}


///////////////////////////////
// Asset view item tool tip
///////////////////////////////

class SAssetViewItemToolTip : public SToolTip
{
public:
	SLATE_BEGIN_ARGS(SAssetViewItemToolTip)
		: _AssetViewItem()
	{ }

		SLATE_ARGUMENT(TSharedPtr<SAssetViewItem>, AssetViewItem)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		AssetViewItem = InArgs._AssetViewItem;
		FMargin TooltipPadding = FMargin(0.f);
		if (const TSharedPtr<SAssetViewItem> AssetViewItemPinned = AssetViewItem.Pin())
		{
			if (AssetViewItemPinned->IsFolder())
			{
				TooltipPadding = FMargin(12.f, 8.f, 12.f, 2.f);
			}
		}

		SToolTip::Construct(
			SToolTip::FArguments()
			.TextMargin(
				UE::Editor::ContentBrowser::IsNewStyleEnabled()
				? TooltipPadding
				: FMargin(1.0f))
			.BorderImage(
				UE::Editor::ContentBrowser::IsNewStyleEnabled()
				? FAppStyle::GetBrush("AssetThumbnail.Tooltip.Border")
				: UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.TileViewTooltip.ToolTipBorder")));
	}

	// IToolTip interface
	virtual bool IsEmpty() const override
	{
		return !AssetViewItem.IsValid();
	}

	virtual void OnOpening() override
	{
		if (const TSharedPtr<SAssetViewItem> AssetViewItemPinned = AssetViewItem.Pin())
		{
			if (AssetViewItemPinned->ThumbnailWidgetITooltip.IsValid())
			{
				// Update the Thumbnail tooltip
				AssetViewItemPinned->ThumbnailWidgetITooltip->OnOpening();
			}
			SetContentWidget(AssetViewItemPinned->CreateToolTipWidget());
		}
	}

	virtual void OnClosed() override
	{
		if (const TSharedPtr<SAssetViewItem> AssetViewItemPinned = AssetViewItem.Pin())
		{
			if (AssetViewItemPinned->ThumbnailWidgetITooltip.IsValid())
			{
				// Reset the content of the Thumbnail tooltip
				AssetViewItemPinned->ThumbnailWidgetITooltip->ResetContentWidget();
			}
		}
		ResetContentWidget();
	}

private:
	TWeakPtr<SAssetViewItem> AssetViewItem;
};


///////////////////////////////
// Asset view modes
///////////////////////////////

void SAssetTileView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Refreshing an asset view is an intensive task. Do not do this while a user
	// is dragging around content for maximum responsiveness.
	// Also prevents a re-entrancy crash caused by potentially complex thumbnail generators.
	if (!FSlateApplication::Get().IsDragDropping())
	{
		STileView<TSharedPtr<FAssetViewItem>>::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}
}

void SAssetListView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Refreshing an asset view is an intensive task. Do not do this while a user
	// is dragging around content for maximum responsiveness.
	// Also prevents a re-entrancy crash caused by potentially complex thumbnail generators.
	if (!FSlateApplication::Get().IsDragDropping())
	{
		SListView<TSharedPtr<FAssetViewItem>>::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}
}

void SAssetColumnView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Refreshing an asset view is an intensive task. Do not do this while a user
	// is dragging around content for maximum responsiveness.
	// Also prevents a re-entrancy crash caused by potentially complex thumbnail generators.
	if (!FSlateApplication::Get().IsDragDropping())
	{
		return SListView<TSharedPtr<FAssetViewItem>>::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}
}

///////////////////////////////
// SAssetViewItem
///////////////////////////////

SAssetViewItem::~SAssetViewItem()
{
	if (AssetItem.IsValid())
	{
		AssetItem->OnItemDataChanged().RemoveAll(this);
	}

	OnItemDestroyed.ExecuteIfBound(AssetItem);
}

void SAssetViewItem::Construct( const FArguments& InArgs )
{
	AssetItem = InArgs._AssetItem;
	OnRenameBegin = InArgs._OnRenameBegin;
	OnRenameCommit = InArgs._OnRenameCommit;
	OnVerifyRenameCommit = InArgs._OnVerifyRenameCommit;
	OnItemDestroyed = InArgs._OnItemDestroyed;
	ShouldAllowToolTip = InArgs._ShouldAllowToolTip;
	ThumbnailEditMode = InArgs._ThumbnailEditMode;
	HighlightText = InArgs._HighlightText;
	OnIsAssetValidForCustomToolTip = InArgs._OnIsAssetValidForCustomToolTip;
	OnGetCustomAssetToolTip = InArgs._OnGetCustomAssetToolTip;
	OnVisualizeAssetToolTip = InArgs._OnVisualizeAssetToolTip;
	OnAssetToolTipClosing = InArgs._OnAssetToolTipClosing;
	IsSelected = InArgs._IsSelected;
	bShowAssetAccessSpecifier = InArgs._ShowAssetAccessSpecifier;

	AssetAccessSpecifierIconBox = SNew(SBox);

	bDraggedOver = false;

	bItemDirty = false;
	OnAssetDataChanged();

	AssetItem->OnItemDataChanged().AddSP(this, &SAssetViewItem::OnAssetDataChanged);

	AssetDirtyBrush = UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.ContentDirty");

	// Set our tooltip - this will refresh each time it's opened to make sure it's up-to-date
	SetToolTip(SNew(SAssetViewItemToolTip).AssetViewItem(SharedThis(this)));

	SourceControlStateDelay = 0.0f;
	bSourceControlStateRequested = false;

	ISourceControlModule::Get().RegisterProviderChanged(FSourceControlProviderChanged::FDelegate::CreateSP(this, &SAssetViewItem::HandleSourceControlProviderChanged));
	SourceControlStateChangedDelegateHandle = ISourceControlModule::Get().GetProvider().RegisterSourceControlStateChanged_Handle(FSourceControlStateChanged::FDelegate::CreateSP(this, &SAssetViewItem::HandleSourceControlStateChanged));

	// Source control state may have already been cached, make sure the control is in sync with 
	// cached state as the delegate is not going to be invoked again until source control state 
	// changes. This will be necessary any time the widget is destroyed and recreated after source 
	// control state has been cached; for instance when the widget is killed via FWidgetGenerator::OnEndGenerationPass 
	// or a view is refreshed due to user filtering/navigating):
	HandleSourceControlStateChanged();

	FAssetData AssetData;
	AssetItem->GetItem().Legacy_TryGetAssetData(AssetData);
	if (AssetData.IsValid())
	{
		if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(AssetData))
		{
			bShouldSaveExternalPackages = AssetDefinition->ShouldSaveExternalPackages();
		}
	}
}

void SAssetViewItem::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	const float PrevSizeX = LastGeometry.Size.X;

	LastGeometry = AllottedGeometry;

	// Set cached wrap text width based on new "LastGeometry" value. 
	// We set this only when changed because binding a delegate to text wrapping attributes is expensive
	if( PrevSizeX != AllottedGeometry.Size.X && InlineRenameWidget.IsValid() )
	{
		const float WrapWidth = GetNameTextWrapWidth();
		InlineRenameWidget->SetWrapTextAt(WrapWidth);
	}

	UpdateDirtyState();

	UpdateSourceControlState((float)InDeltaTime);
}

TSharedPtr<IToolTip> SAssetViewItem::GetToolTip()
{
	return ShouldAllowToolTip.Get() ? SCompoundWidget::GetToolTip() : NULL;
}

void SAssetViewItem::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	bDraggedOver = AssetItem && DragDropHandler::HandleDragEnterItem(AssetItem->GetItem(), DragDropEvent);
}
	
void SAssetViewItem::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	if (AssetItem)
	{
		DragDropHandler::HandleDragLeaveItem(AssetItem->GetItem(), DragDropEvent);
	}
	bDraggedOver = false;
}

FReply SAssetViewItem::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	bDraggedOver = AssetItem && DragDropHandler::HandleDragOverItem(AssetItem->GetItem(), DragDropEvent);
	return (bDraggedOver) ? FReply::Handled() : FReply::Unhandled();
}

FReply SAssetViewItem::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, const TSharedRef<SWidget>& InParentWidget)
{
	if (AssetItem && DragDropHandler::HandleDragDropOnItem(AssetItem->GetItem(), DragDropEvent, InParentWidget))
	{
		bDraggedOver = false;
		return FReply::Handled();
	}

	if (bDraggedOver)
	{
		// We were able to handle this operation, but could not due to another error - still report this drop as handled so it doesn't fall through to other widgets
		bDraggedOver = false;
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SAssetViewItem::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	return OnDrop(MyGeometry, DragDropEvent, AsShared());
}

bool SAssetViewItem::IsNameReadOnly() const
{
	if (ThumbnailEditMode.Get())
	{
		// Read-only while editing thumbnails
		return true;
	}

	if (!AssetItem.IsValid())
	{
		// Read-only if no valid asset item
		return true;
	}

	if (AssetItem->GetItem().IsTemporary())
	{
		// Temporary items can always be renamed (required for creation/duplication, etc)
		return false;
	}

	// Read-only if we can't be renamed
	return !AssetItem->GetItem().CanRename(nullptr);
}

void SAssetViewItem::HandleBeginNameChange( const FText& OriginalText )
{
	OnRenameBegin.ExecuteIfBound(AssetItem, OriginalText.ToString(), LastGeometry.GetLayoutBoundingRect());
}

void SAssetViewItem::HandleNameCommitted( const FText& NewText, ETextCommit::Type CommitInfo )
{
	OnRenameCommit.ExecuteIfBound(AssetItem, NewText.ToString(), LastGeometry.GetLayoutBoundingRect(), CommitInfo);
}

bool  SAssetViewItem::HandleVerifyNameChanged( const FText& NewText, FText& OutErrorMessage )
{
	return !OnVerifyRenameCommit.IsBound() || OnVerifyRenameCommit.Execute(AssetItem, NewText, LastGeometry.GetLayoutBoundingRect(), OutErrorMessage);
}

void SAssetViewItem::OnAssetDataChanged()
{
	UpdateDirtyState();

	if ( InlineRenameWidget.IsValid() )
	{
		InlineRenameWidget->SetText( GetNameText() );
	}

	if (ClassTextWidget.IsValid())
	{
		ClassTextWidget->SetText(GetAssetClassText());
	}


	CacheDisplayTags();
}

void SAssetViewItem::DirtyStateChanged()
{
}

FText SAssetViewItem::GetAssetClassText() const
{
	if (!AssetItem)
	{
		return FText();
	}
	
	return AssetItem->GetAssetClassText();
}

void SAssetViewItem::HandleSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider)
{
	OldProvider.UnregisterSourceControlStateChanged_Handle(SourceControlStateChangedDelegateHandle);
	SourceControlStateChangedDelegateHandle = NewProvider.RegisterSourceControlStateChanged_Handle(FSourceControlStateChanged::FDelegate::CreateSP(this, &SAssetViewItem::HandleSourceControlStateChanged));
	
	// Reset this so the state will be queried from the new provider on the next Tick
	SourceControlStateDelay = 0.0f;
	bSourceControlStateRequested = false;
	
	HandleSourceControlStateChanged();
}

void SAssetViewItem::HandleSourceControlStateChanged()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SAssetViewItem::HandleSourceControlStateChanged);

	if (AssetItem && AssetItem->IsFile() && !AssetItem->IsTemporary() && ISourceControlModule::Get().IsEnabled())
	{
		FString AssetFilename;
		if (AssetItem->GetItem().GetItemPhysicalPath(AssetFilename))
		{
			FSourceControlStatePtr SourceControlState = ISourceControlModule::Get().GetProvider().GetState(AssetFilename, EStateCacheUsage::Use);
			if (SourceControlState)
			{
				if (SCCStateWidget.IsValid())
				{
					FSlateIcon SCCIcon = SourceControlState->GetIcon();
					bHasCCStateBrush = SCCIcon.GetIcon() != FStyleDefaults::GetNoBrush();
					SCCStateWidget->SetFromSlateIcon(SCCIcon);
				}
			}
		}
	}
}

void SAssetViewItem::CacheDirtyExternalPackageInfo()
{
	if(bShouldSaveExternalPackages)
	{
		CachedDirtyPackagesList.Empty();
		
		FAssetData AssetData;		
		AssetItem->GetItem().Legacy_TryGetAssetData(AssetData);
		if (AssetData.IsAssetLoaded())
		{
			if(const UObject* Asset = AssetData.GetAsset())
			{
				if (const UPackage* Package = Asset->GetPackage())
				{
					TArray<UPackage*> ExternalPackages = Package->GetExternalPackages();
					IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

					// Mirrored/copied from SSourceControlCommon.cpp
					auto RetrieveAssetName = [](const FAssetData& InAssetData) -> FString
					{
						static const FName NAME_ActorLabel(TEXT("ActorLabel"));
						if (InAssetData.FindTag(NAME_ActorLabel))
						{
							FString ResultAssetName;
							InAssetData.GetTagValue(NAME_ActorLabel, ResultAssetName);
							return ResultAssetName;
						}

						if (InAssetData.FindTag(FPrimaryAssetId::PrimaryAssetDisplayNameTag))
						{
							FString ResultAssetName;
							InAssetData.GetTagValue(FPrimaryAssetId::PrimaryAssetDisplayNameTag, ResultAssetName);
							return ResultAssetName;
						}

						if (InAssetData.AssetClassPath == UActorFolder::StaticClass()->GetClassPathName())
						{
							FString ActorFolderPath = UActorFolder::GetAssetRegistryInfoFromPackage(InAssetData.PackageName).GetDisplayName();
							if (!ActorFolderPath.IsEmpty())
							{
								return ActorFolderPath;
							}
						}

						return InAssetData.AssetName.ToString();
					};
					
					for (const UPackage* ExternalPackage : ExternalPackages)
					{
						if (ExternalPackage->IsDirty())
						{
							TArray<FAssetData> DirtyAssetDataEntries;
							AssetRegistry.GetAssetsByPackageName(*ExternalPackage->GetName(), DirtyAssetDataEntries);

							if (CachedDirtyPackagesList.Len())
							{								
								CachedDirtyPackagesList.Append("\n");
							}
							
							CachedDirtyPackagesList.Append(ExternalPackage->GetPathName());
							
							for (const FAssetData& DirtyAssetData : DirtyAssetDataEntries)
							{
								const FString AssetName = RetrieveAssetName(DirtyAssetData);
								const FString AssetClass = DirtyAssetData.AssetClassPath.GetAssetName().ToString();
						
								CachedDirtyPackagesList.Append("\n\t");
								CachedDirtyPackagesList.Append(FString::Printf(TEXT("%s (%s)"), *AssetName, *AssetClass)); 
							}
						}
					}
				}
			}
		}
	}
}

bool SAssetViewItem::ShouldExpandTooltip() const
{
	return GetDefault<UContentBrowserSettings>()->GetAlwaysExpandTooltips();
}

const FSlateBrush* SAssetViewItem::GetDirtyImage() const
{
	return IsDirty() ? AssetDirtyBrush : NULL;
}

TSharedRef<SWidget> SAssetViewItem::GenerateSourceControlIconWidget()
{
	TSharedRef<SLayeredImage> Image = SNew(SLayeredImage).Image(FStyleDefaults::GetNoBrush());
	SCCStateWidget = Image;

	return Image;
}

TSharedRef<SWidget> SAssetViewItem::GenerateExtraStateIconWidget(TAttribute<float> InMaxExtraStateIconWidth) const
{
	const TArray<FAssetViewExtraStateGenerator>& Generators = FModuleManager::GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser")).GetAllAssetViewExtraStateGenerators();
	if (AssetItem && AssetItem->IsFile() && Generators.Num() > 0)
	{
		FAssetData ItemAssetData;
		if (AssetItem->GetItem().Legacy_TryGetAssetData(ItemAssetData))
		{
			// Add extra state icons
			TSharedPtr<SHorizontalBox> Content = SNew(SHorizontalBox);

			for (const FAssetViewExtraStateGenerator& Generator : Generators)
			{
				if (Generator.IconGenerator.IsBound())
				{
					Content->AddSlot()
						.HAlign(HAlign_Left)
						.AutoWidth()
						.MaxWidth(InMaxExtraStateIconWidth)
						[
							Generator.IconGenerator.Execute(ItemAssetData)
						];
				}
			}
			return Content.ToSharedRef();
		}
	}
	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SAssetViewItem::GenerateExtraStateTooltipWidget() const
{
	const TArray<FAssetViewExtraStateGenerator>& Generators = FModuleManager::GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser")).GetAllAssetViewExtraStateGenerators();
	if (AssetItem && AssetItem->IsFile() && Generators.Num() > 0)
	{
		FAssetData ItemAssetData;
		if (AssetItem->GetItem().Legacy_TryGetAssetData(ItemAssetData))
		{
			TSharedPtr<SVerticalBox> Content = SNew(SVerticalBox);
			for (const auto& Generator : Generators)
			{
				if (Generator.ToolTipGenerator.IsBound() && Generator.IconGenerator.IsBound())
				{
					Content->AddSlot()
						.Padding(FMargin(0, 2.0F))
						.AutoHeight()
						[
							SNew(SHorizontalBox)

							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(FMargin(0, 0, 2.0f, 0))
							[
								Generator.IconGenerator.Execute(ItemAssetData)
							]

							+SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							[
								Generator.ToolTipGenerator.Execute(ItemAssetData)
							]
						];
				}
			}

			if (Content->NumSlots() == 0)
			{
				return SNullWidget::NullWidget;
			}
			return Content.ToSharedRef();
		}
	}
	return SNullWidget::NullWidget;
}

TSharedPtr<SWidget> SAssetViewItem::GenerateExtraStateTooltipWidgetNewStyle() const
{
	return GenerateExtraStateTooltipWidget().ToSharedPtr();
}

EVisibility SAssetViewItem::GetThumbnailEditModeUIVisibility() const
{
	return !IsFolder() && ThumbnailEditMode.Get() ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<SWidget> SAssetViewItem::CreateToolTipWidget() const
{
	if ( AssetItem.IsValid() )
	{
		// Legacy custom asset tooltips
		if (OnGetCustomAssetToolTip.IsBound())
		{
			FAssetData ItemAssetData;
			if (AssetItem->GetItem().Legacy_TryGetAssetData(ItemAssetData))
			{
				const bool bTryCustomAssetToolTip = !OnIsAssetValidForCustomToolTip.IsBound() || OnIsAssetValidForCustomToolTip.Execute(ItemAssetData);
				if (bTryCustomAssetToolTip)
				{
					return OnGetCustomAssetToolTip.Execute(ItemAssetData);
				}
			}
		}

		// TODO: Remove this special caseness so that folders can also have visible attributes
		if(AssetItem->IsFile())
		{
			// New style tooltip based on the AssetThumbnail one, won't be called in the old version as it is never set
			if (ThumbnailWidgetITooltip.IsValid())
			{
				return ThumbnailWidgetITooltip->GetContentWidget();
			}

			// The tooltip contains the name, class, path, asset registry tags and source control status
			const FText NameText = GetNameText();
			const FText ClassText = FText::Format(LOCTEXT("ClassName", "({0})"), GetAssetClassText());

			FText PublicStateText;
			const FSlateBrush* PublicStateIcon = nullptr;
			FName PublicStateTextBorder = "ContentBrowser.TileViewTooltip.PillBorder";

			// Create a box to hold every line of info in the body of the tooltip
			TSharedRef<SVerticalBox> InfoBox = SNew(SVerticalBox);

			FAssetData ItemAssetData;
			AssetItem->GetItem().Legacy_TryGetAssetData(ItemAssetData);

			// TODO: Always use the virtual path?
			if (ItemAssetData.IsValid())
			{
				AddToToolTipInfoBox(InfoBox, LOCTEXT("TileViewTooltipPath", "Path"), FText::FromName(ItemAssetData.PackagePath), false);
			}
			else
			{
				AddToToolTipInfoBox(InfoBox, LOCTEXT("TileViewTooltipPath", "Path"), IContentBrowserDataModule::Get().GetSubsystem()->ConvertVirtualPathToDisplay(AssetItem->GetItem()), false);
			}

			if (ItemAssetData.IsValid() && FAssetToolsModule::GetModule().Get().ShowingContentVersePath())
			{
				UE::Core::FVersePath VersePath = ItemAssetData.GetVersePath();
				if (VersePath.IsValid())
				{
					AddToToolTipInfoBox(InfoBox, LOCTEXT("TileViewTooltipVersePath", "Verse Path"), FText::FromString(VersePath.ToString()), false);
				}
			}

			if (ItemAssetData.IsValid() && ItemAssetData.PackageName != NAME_None)
			{
				const FString PackagePathWithinRoot = ContentBrowserUtils::GetPackagePathWithinRoot(ItemAssetData.PackageName.ToString());
				int32 PackageNameLength = PackagePathWithinRoot.Len();

				int32 MaxAssetPathLen = ContentBrowserUtils::GetMaxAssetPathLen();
				AddToToolTipInfoBox( InfoBox, LOCTEXT( "TileViewTooltipAssetPathLengthKey", "Asset Filepath Length" ), FText::Format( LOCTEXT( "TileViewTooltipAssetPathLengthValue", "{0} / {1}" ),
					FText::AsNumber( PackageNameLength ), FText::AsNumber( MaxAssetPathLen ) ), PackageNameLength > MaxAssetPathLen ? true : false );

				int32 PackageNameLengthForCooking = ContentBrowserUtils::GetPackageLengthForCooking(ItemAssetData.PackageName.ToString(), FEngineBuildSettings::IsInternalBuild());

				int32 MaxCookPathLen = ContentBrowserUtils::GetMaxCookPathLen();
				AddToToolTipInfoBox(InfoBox, LOCTEXT("TileViewTooltipPathLengthForCookingKey", "Cooking Filepath Length"), FText::Format(LOCTEXT("TileViewTooltipPathLengthForCookingValue", "{0} / {1}"),
					FText::AsNumber(PackageNameLengthForCooking), FText::AsNumber(MaxCookPathLen)), PackageNameLengthForCooking > MaxCookPathLen ? true : false);

				PublicStateText = AssetItem->GetItemAssetAccessSpecifierText();
			}

			if (!AssetItem->GetItem().CanEdit())
			{
				if(AssetItem->GetItem().CanView())
				{
					PublicStateText = LOCTEXT("ViewReadOnlyAssetState", "View / Read Only");
					PublicStateIcon = FAppStyle::GetBrush("AssetEditor.ReadOnlyOpenable");

				}
				else
				{
					PublicStateText = LOCTEXT("ReadOnlyAssetState", "Read Only");
					PublicStateIcon = FAppStyle::GetBrush("Icons.Lock");
				}
			}

			if(!AssetItem->GetItem().IsSupported())
			{
				PublicStateText = LOCTEXT("UnsupportedAssetState", "Unsupported");
				PublicStateTextBorder = "ContentBrowser.TileViewTooltip.UnsupportedAssetPillBorder";
			}

			// Add tags
			for (const auto& DisplayTagItem : CachedDisplayTags)
			{
				AddToToolTipInfoBox(InfoBox, DisplayTagItem.DisplayKey, DisplayTagItem.DisplayValue, DisplayTagItem.bImportant);
			}

			// Add asset source files
			if (ItemAssetData.IsValid())
			{
				TOptional<FAssetImportInfo> ImportInfo = FAssetSourceFilenameCache::ExtractAssetImportInfo(ItemAssetData);
				if (ImportInfo.IsSet())
				{
					for (const auto& File : ImportInfo->SourceFiles)
					{
						FText SourceLabel = LOCTEXT("TileViewTooltipSourceFile", "Source File");
						if (File.DisplayLabelName.Len() > 0)
						{
							SourceLabel = FText::FromString(FText(LOCTEXT("TileViewTooltipSourceFile", "Source File")).ToString() + TEXT(" (") + File.DisplayLabelName + TEXT(")"));
						}
						AddToToolTipInfoBox(InfoBox, SourceLabel, FText::FromString(File.RelativeFilename), false);
					}
				}
			}

			TSharedRef<SVerticalBox> OverallTooltipVBox = SNew(SVerticalBox);

			static const IConsoleVariable* EnablePublicAssetFeatureCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("AssetTools.EnablePublicAssetFeature"));
			const bool bIsPublicAssetUIEnabled = EnablePublicAssetFeatureCVar && EnablePublicAssetFeatureCVar->GetBool();

			// Top section (asset name, type, is checked out)
			OverallTooltipVBox->AddSlot()
				.AutoHeight()
				.Padding(0, 0, 0, 4)
				[
					SNew(SBorder)
					.Padding(6)
					.BorderImage(UE::ContentBrowser::Private::FContentBrowserStyle::Get().Get().GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(0, 0, 4, 0)
							[
								SNew(STextBlock)
								.Text(NameText)
								.Font(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetFontStyle("ContentBrowser.TileViewTooltip.NameFont"))
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(ClassText)
								.HighlightText(HighlightText)
							]
							+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.Padding(10.0f, 4.0f)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Right) 
							[
								SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush(PublicStateTextBorder))
								.Visibility(bIsPublicAssetUIEnabled && !PublicStateText.IsEmpty() ? EVisibility::Visible : EVisibility::Hidden)
								.Padding(FMargin(12.0f, 2.0f, 12.0f, 2.0f))
								[
									SNew(SHorizontalBox)
									+SHorizontalBox::Slot()
									.AutoWidth()
									.HAlign(HAlign_Left)
									.VAlign(VAlign_Center)
									.Padding(0.0f)
									[
										SNew(SBox)
										.Visibility(PublicStateIcon ? EVisibility::Visible : EVisibility::Collapsed)
										.HeightOverride(16.0f)
										.WidthOverride(16.0f)
										[
											SNew(SImage)
											.Image(PublicStateIcon)
										]
											
									]
									+SHorizontalBox::Slot()
									.HAlign(HAlign_Left)
									.VAlign(VAlign_Center)
									.Padding(4.0f, 0.0f, 0.0f, 0.0f)
									[
										SNew(STextBlock)
										.Text(PublicStateText)
										.HighlightText(HighlightText)
									]
								]
							]
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(STextBlock)
							.Visibility_Lambda([this]()
							{
								return AssetItem->GetItem().IsSupported() ? EVisibility::Collapsed : EVisibility::Visible;
							})
							.Text(LOCTEXT("UnsupportedAssetDescriptionText", "This type of asset is not allowed in this project. Delete unsupported assets to avoid errors."))
							.ColorAndOpacity(FStyleColors::Warning)
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(STextBlock)
							.Visibility(this, &SAssetViewItem::GetSourceControlTextVisibility)
							.Text(this, &SAssetViewItem::GetSourceControlText)
							.ColorAndOpacity(FLinearColor(0.1f, 0.5f, 1.f, 1.f))
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(STextBlock)
							.Visibility_Lambda([this]()-> EVisibility { return bShouldSaveExternalPackages && !GetExternalPackagesText().IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed; })
							.Text(LOCTEXT("DirtyExternalPackages", "Modified external packages:"))
							.ColorAndOpacity(FStyleColors::Warning)
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(STextBlock)
							.Visibility_Lambda([this]()-> EVisibility { return bShouldSaveExternalPackages && !GetExternalPackagesText().IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed; })
							.Text(this, &SAssetViewItem::GetExternalPackagesText)
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							GenerateExtraStateTooltipWidget()
						]
					]
				];

			// Middle section (user description, if present)
			FText UserDescription = GetAssetUserDescription();
			if (!UserDescription.IsEmpty())
			{
				OverallTooltipVBox->AddSlot()
					.AutoHeight()
					.Padding(0, 0, 0, 4)
					[
						SNew(SBorder)
						.Padding(6)
						.BorderImage(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
						[
							SNew(STextBlock)
							.WrapTextAt(700.0f)
							.Font(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetFontStyle("ContentBrowser.TileViewTooltip.AssetUserDescriptionFont"))
							.Text(UserDescription)
						]
					];
			}

			// Bottom section (asset registry tags)
			OverallTooltipVBox->AddSlot()
				.AutoHeight()
				[
					SNew(SBorder)
					.Padding(6)
					.BorderImage(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
					[
						InfoBox
					]
				];

			// Final section (collection pips)
			if (ItemAssetData.IsValid())
			{
				ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();

				TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
				CollectionManager.GetVisibleCollectionContainers(CollectionContainers);

				const FSoftObjectPath ObjectPath = ItemAssetData.ToSoftObjectPath();

				TSharedPtr<SWrapBox> CollectionPipsWrapBox;
				TArray<FCollectionNameType> CollectionsContainingObject;
				for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
				{
					CollectionsContainingObject.Reset();
					CollectionContainer->GetCollectionsContainingObject(ObjectPath, CollectionsContainingObject);

					if (CollectionsContainingObject.Num() > 0)
					{
						Algo::Sort(CollectionsContainingObject, [](const FCollectionNameType& A, const FCollectionNameType& B)
							{
								int32 result = A.Name.Compare(B.Name);
								return result < 0 || (result == 0 && A.Type < B.Type);
							});

						if (!CollectionPipsWrapBox)
						{
							CollectionPipsWrapBox = SNew(SWrapBox)
								.PreferredSize(700.0f);
						}

						for (const FCollectionNameType& CollectionContainingObject : CollectionsContainingObject)
						{
							FCollectionStatusInfo CollectionStatusInfo;
							if (CollectionContainer->GetCollectionStatusInfo(CollectionContainingObject.Name, CollectionContainingObject.Type, CollectionStatusInfo))
							{
								CollectionPipsWrapBox->AddSlot()
									.Padding(0, 4, 4, 0)
									[
										SNew(SAssetTagItem)
											.ViewMode(EAssetTagItemViewMode::Compact)
											.BaseColor(CollectionViewUtils::ResolveColor(*CollectionContainer, CollectionContainingObject.Name, CollectionContainingObject.Type))
											.DisplayName(FText::FromName(CollectionContainingObject.Name))
											.CountText(FText::AsNumber(CollectionStatusInfo.NumObjects))
									];
							}
						}
					}
				}

				if (CollectionPipsWrapBox)
				{
					OverallTooltipVBox->AddSlot()
					.AutoHeight()
					.Padding(0, 4, 0, 0)
					[
						SNew(SBorder)
						.Padding(FMargin(6, 2, 6, 6))
						.BorderImage(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
						[
							CollectionPipsWrapBox.ToSharedRef()
						]
					];
				}
			}

			return SNew(SBorder)
				.Padding(6)
				.BorderImage( UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.TileViewTooltip.NonContentBorder") )
				[
					OverallTooltipVBox
				];
		}
		else
		{
			const FText FolderName = GetNameText();
			const FText FolderPath = IContentBrowserDataModule::Get().GetSubsystem()->ConvertVirtualPathToDisplay(AssetItem->GetItem());

			// Create a box to hold every line of info in the body of the tooltip
			TSharedRef<SVerticalBox> InfoBox = SNew(SVerticalBox);

			AddToToolTipInfoBox( InfoBox, LOCTEXT("TileViewTooltipPath", "Path"), FolderPath, false );

			const FName InternalPath = AssetItem->GetItem().GetInternalPath();
			if (!InternalPath.IsNone())
			{
				FNameBuilder FolderPathBuilder(InternalPath);
				if (AssetViewWidgets::IsTopLevelFolder(FStringView(FolderPathBuilder)))
				{
					FStringView PluginName(FolderPathBuilder);
					PluginName.RightChopInline(1);

					if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName))
					{
						if (Plugin->GetDescriptor().Description.Len() > 0)
						{
							AddToToolTipInfoBox( InfoBox, LOCTEXT("TileViewTooltipPluginDescription", "Plugin Description"), FText::FromString(Plugin->GetDescriptor().Description), false );
						}
					}
				}
			}

			if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
			{
				// Default values
				FName FolderBrushName = TEXT("ContentBrowser.ListViewFolderIcon");
				FName FolderShadowBrushName = TEXT("ContentBrowser.FolderItem.DropShadow");
				ContentBrowserUtils::TryGetFolderBrushAndShadowName(AssetItem->GetItem(), FolderBrushName, FolderShadowBrushName);

				return SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.f, 0.f, 0.f, 6.f)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(FolderName)
								.ColorAndOpacity(FStyleColors::White)
								.Font(FAppStyle::GetFontStyle("ContentBrowser.Tooltip.EntryFont"))
							]
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(0.f, 0.f, 4.f, 0.f)
							[
								SNew(SBox)
								.WidthOverride(16.f)
								.HeightOverride(16.f)
								[
									SNew(SImage)
									.ColorAndOpacity(GetAssetColor())
									.Image(FAppStyle::GetBrush(FolderBrushName))
								]
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Font(FAppStyle::GetFontStyle("ContentBrowser.Tooltip.EntryFont"))
								.Text(LOCTEXT("FolderNameBracketedLabel", "Folder"))
							]
						]
					]

					+ SVerticalBox::Slot()
					.Padding(FMargin(0.f,6.f))
					.AutoHeight()
					[
						SNew(SSeparator)
						.Orientation(Orient_Horizontal)
						.Thickness(1.f)
						.ColorAndOpacity(COLOR("#484848FF"))
						.SeparatorImage(FAppStyle::Get().GetBrush("WhiteBrush"))
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						InfoBox
					];
			}
			else
			{
				return SNew(SBorder)
					.Padding(6)
					.BorderImage( UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.TileViewTooltip.NonContentBorder") )
					[
						SNew(SVerticalBox)

						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0, 0, 0, 4)
						[
							SNew(SBorder)
							.Padding(6)
							.BorderImage( UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.TileViewTooltip.ContentBorder") )
							[
								SNew(SVerticalBox)

								+SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(SHorizontalBox)

									+SHorizontalBox::Slot()
									.AutoWidth()
									.VAlign(VAlign_Center)
									.Padding(0, 0, 4, 0)
									[
										SNew(STextBlock)
										.Text( FolderName )
										.Font( UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetFontStyle("ContentBrowser.TileViewTooltip.NameFont") )
									]

									+SHorizontalBox::Slot()
									.AutoWidth()
									.VAlign(VAlign_Center)
									[
										SNew(STextBlock)
										.Text( LOCTEXT("FolderNameBracketed", "(Folder)") )
									]
								]
							]
						]

						+SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SBorder)
							.Padding(6)
							.BorderImage( UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.TileViewTooltip.ContentBorder") )
							[
								InfoBox
							]
						]
					];
			}
		}
	}
	else
	{
		// Return an empty tooltip since the asset item wasn't valid
		return SNullWidget::NullWidget;
	}
}

EVisibility SAssetViewItem::GetSourceControlTextVisibility() const
{
	return GetSourceControlText().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
}

FText SAssetViewItem::GetSourceControlText() const
{
	if (AssetItem && AssetItem->IsFile() && !AssetItem->IsTemporary() && ISourceControlModule::Get().IsEnabled())
	{
		FString AssetFilename;
		if (AssetItem->GetItem().GetItemPhysicalPath(AssetFilename))
		{
			FSourceControlStatePtr SourceControlState = ISourceControlModule::Get().GetProvider().GetState(AssetFilename, EStateCacheUsage::Use);
			if (SourceControlState)
			{
				return SourceControlState->GetStatusText().Get(FText::GetEmpty());
			}
		}
	}

	return FText::GetEmpty();
}

FText SAssetViewItem::GetExternalPackagesText() const
{
	if (CachedDirtyPackagesList.Len())
	{
		return FText::FromString(CachedDirtyPackagesList);
	}

	return FText::GetEmpty();
}

FText SAssetViewItem::GetAssetUserDescription() const
{
	if (AssetItem && AssetItem->IsFile())
	{
		FContentBrowserItemDataAttributeValue DescriptionAttributeValue = AssetItem->GetItem().GetItemAttribute(ContentBrowserItemAttributes::ItemDescription);
		if (DescriptionAttributeValue.IsValid())
		{
			return DescriptionAttributeValue.GetValue<FText>();
		}
	}

	return FText::GetEmpty();
}

void SAssetViewItem::AddToToolTipInfoBox(const TSharedRef<SVerticalBox>& InfoBox, const FText& Key, const FText& Value, bool bImportant) const
{
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		InfoBox->AddSlot()
		.Padding(0.f, 0.f, 0.f, 6.f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 4, 0)
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("ContentBrowser.Tooltip.EntryFont"))
				.Text(FText::Format(NSLOCTEXT("AssetThumbnailToolTip", "AssetViewTooltipFormat", "{0}:"), Key))
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("ContentBrowser.Tooltip.EntryFont"))
				.ColorAndOpacity(FStyleColors::White)
				.Text(Value)
			]
		];
	}
	else
	{
		FWidgetStyle ImportantStyle;
		ImportantStyle.SetForegroundColor(FLinearColor(1, 0.5, 0, 1));

		InfoBox->AddSlot()
		.AutoHeight()
		.Padding(0, 1)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 4, 0)
			[
				SNew(STextBlock) .Text( FText::Format(LOCTEXT("AssetViewTooltipFormat", "{0}:"), Key ) )
				.ColorAndOpacity(bImportant ? ImportantStyle.GetSubduedForegroundColor() : FSlateColor::UseSubduedForeground())
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock) .Text( Value )
				.ColorAndOpacity(bImportant ? ImportantStyle.GetForegroundColor() : FSlateColor::UseForeground())
				.HighlightText((Key.ToString() == TEXT("Path")) ? HighlightText : FText())
				.WrapTextAt(700.0f)
			]
		];
	}

}

void SAssetViewItem::UpdateDirtyState()
{
	bool bNewIsDirty = false;

	// Only update the dirty state for non-temporary items
	if (AssetItem && !AssetItem->IsTemporary())
	{
		bNewIsDirty = AssetItem->GetItem().IsDirty();
	}

	if (bNewIsDirty != bItemDirty)
	{
		bItemDirty = bNewIsDirty;
		DirtyStateChanged();
	}
		
	CacheDirtyExternalPackageInfo();
}

bool SAssetViewItem::IsDirty() const
{
	return bItemDirty;
}

void SAssetViewItem::UpdateSourceControlState(float InDeltaTime)
{
	SourceControlStateDelay += InDeltaTime;

	if (AssetItem && AssetItem->IsFile() && !AssetItem->IsTemporary() && !bSourceControlStateRequested && SourceControlStateDelay > 1.0f && ISourceControlModule::Get().IsEnabled())
	{
		FString AssetFilename;
		if (AssetItem->GetItem().GetItemPhysicalPath(AssetFilename))
		{
			ISourceControlModule::Get().QueueStatusUpdate(AssetFilename);
			bSourceControlStateRequested = true;
		}
	}
}

void SAssetViewItem::CacheDisplayTags()
{
	CachedDisplayTags.Reset();

	const FContentBrowserItemDataAttributeValues AssetItemAttributes = AssetItem->GetItem().GetItemAttributes(/*bIncludeMetaData*/true);
	
	FAssetData ItemAssetData;
	AssetItem->GetItem().Legacy_TryGetAssetData(ItemAssetData);

	// Add all visible attributes
	for (const auto& AssetItemAttributePair : AssetItemAttributes)
	{
		const FName AttributeName = AssetItemAttributePair.Key;
		const FContentBrowserItemDataAttributeValue& AttributeValue = AssetItemAttributePair.Value;
		const FContentBrowserItemDataAttributeMetaData& AttributeMetaData = AttributeValue.GetMetaData();

		if (AttributeMetaData.AttributeType == UObject::FAssetRegistryTag::TT_Hidden)
		{
			continue;
		}
	
		// Build the display value for this attribute
		FText DisplayValue;
		if (AttributeValue.GetValueType() == EContentBrowserItemDataAttributeValueType::Text)
		{
			DisplayValue = AttributeValue.GetValueText();
		}
		else
		{
			const FString AttributeValueStr = AttributeValue.GetValue<FString>();

			auto ReformatNumberStringForDisplay = [](const FString& InNumberString) -> FText
			{
				// Respect the number of decimal places in the source string when converting for display
				int32 NumDecimalPlaces = 0;
				{
					int32 DotIndex = INDEX_NONE;
					if (InNumberString.FindChar(TEXT('.'), DotIndex))
					{
						NumDecimalPlaces = InNumberString.Len() - DotIndex - 1;
					}
				}
	
				if (NumDecimalPlaces > 0)
				{
					// Convert the number as a double
					double Num = 0.0;
					LexFromString(Num, *InNumberString);
	
					const FNumberFormattingOptions NumFormatOpts = FNumberFormattingOptions()
						.SetMinimumFractionalDigits(NumDecimalPlaces)
						.SetMaximumFractionalDigits(NumDecimalPlaces);
	
					return FText::AsNumber(Num, &NumFormatOpts);
				}

				const bool bIsSigned = InNumberString.Len() > 0 && (InNumberString[0] == TEXT('-') || InNumberString[0] == TEXT('+'));
				if (bIsSigned)
				{
					// Convert the number as a signed int
					int64 Num = 0;
					LexFromString(Num, *InNumberString);
					return FText::AsNumber(Num);
				}

				// Convert the number as an unsigned int
				uint64 Num = 0;
				LexFromString(Num, *InNumberString);
				return FText::AsNumber(Num);
			};
	
			bool bHasSetDisplayValue = false;
	
			// Numerical tags need to format the specified number based on the display flags
			if (!bHasSetDisplayValue && AttributeMetaData.AttributeType == UObject::FAssetRegistryTag::TT_Numerical && AttributeValueStr.IsNumeric())
			{
				bHasSetDisplayValue = true;
	
				const bool bAsMemory = !!(AttributeMetaData.DisplayFlags & UObject::FAssetRegistryTag::TD_Memory);
	
				if (bAsMemory)
				{
					// Memory should be a 64-bit unsigned number of bytes
					uint64 NumBytes = 0;
					LexFromString(NumBytes, *AttributeValueStr);
	
					DisplayValue = FText::AsMemory(NumBytes);
				}
				else
				{
					DisplayValue = ReformatNumberStringForDisplay(AttributeValueStr);
				}
			}
	
			// Dimensional tags need to be split into their component numbers, with each component number re-formatted
			if (!bHasSetDisplayValue && AttributeMetaData.AttributeType == UObject::FAssetRegistryTag::TT_Dimensional)
			{
				// Formats:
				//   123             (1D)
				//   123x234         (2D)
				//   123x234*345     (2D array)
				//   123x234x345     (3D)
				//   123x234x345*456 (3D array)
				int32 FirstXPos;
				if (AttributeValueStr.FindChar(TEXT('x'), FirstXPos))
				{
					FString FirstPart = AttributeValueStr.Left(FirstXPos);
					FString Remainder = AttributeValueStr.Mid(FirstXPos + 1);
					int32 RemainderSeparatorPos;

					if (Remainder.FindChar(TEXT('*'), RemainderSeparatorPos))
					{
						FString SecondPart = Remainder.Left(RemainderSeparatorPos);

						int SecondXPos;
						if (SecondPart.FindChar(TEXT('x'), SecondXPos))
						{
							// AxBxC*D form (3D array)
							FString ThirdPart = Remainder.Mid(SecondXPos + 1);
							FString FourthPart = Remainder.Mid(RemainderSeparatorPos + 1);

							bHasSetDisplayValue = true;
							DisplayValue = FText::Format(LOCTEXT("DisplayTag3xArrayFmt", "{0} \u00D7 {1} \u00D7 {2} ({3} elements)"),
								ReformatNumberStringForDisplay(FirstPart),
								ReformatNumberStringForDisplay(SecondPart),
								ReformatNumberStringForDisplay(ThirdPart),
								ReformatNumberStringForDisplay(FourthPart));
						}
						else
						{
							// AxB*C form (2D array)
							FString ThirdPart = Remainder.Mid(RemainderSeparatorPos + 1);

							bHasSetDisplayValue = true;
							DisplayValue = FText::Format(LOCTEXT("DisplayTag2xArrayFmt", "{0} \u00D7 {1} ({2} elements)"),
								ReformatNumberStringForDisplay(FirstPart),
								ReformatNumberStringForDisplay(SecondPart),
								ReformatNumberStringForDisplay(ThirdPart));
						}
					}
					else if (Remainder.FindChar(TEXT('x'), RemainderSeparatorPos))
					{
						// AxBxC form (3D)
						FString SecondPart = Remainder.Left(RemainderSeparatorPos);
						FString ThirdPart = Remainder.Mid(RemainderSeparatorPos + 1);

						bHasSetDisplayValue = true;
						DisplayValue = FText::Format(LOCTEXT("DisplayTag3xFmt", "{0} \u00D7 {1} \u00D7 {2}"),
							ReformatNumberStringForDisplay(FirstPart),
							ReformatNumberStringForDisplay(SecondPart),
							ReformatNumberStringForDisplay(ThirdPart));
					}
					else
					{
						// 2D form by default
						bHasSetDisplayValue = true;
						DisplayValue = FText::Format(LOCTEXT("DisplayTag2xFmt", "{0} \u00D7 {1}"),
							ReformatNumberStringForDisplay(FirstPart),
							ReformatNumberStringForDisplay(Remainder));
					}
				}
				else
				{
					// No separators, assume 1D
					bHasSetDisplayValue = true;
					DisplayValue = ReformatNumberStringForDisplay(AttributeValueStr);
				}
			}
	
			// Chronological tags need to format the specified timestamp based on the display flags
			if (!bHasSetDisplayValue && AttributeMetaData.AttributeType == UObject::FAssetRegistryTag::TT_Chronological)
			{
				bHasSetDisplayValue = true;
	
				FDateTime Timestamp;
				if (FDateTime::Parse(AttributeValueStr, Timestamp))
				{
					const bool bDisplayDate = !!(AttributeMetaData.DisplayFlags & UObject::FAssetRegistryTag::TD_Date);
					const bool bDisplayTime = !!(AttributeMetaData.DisplayFlags & UObject::FAssetRegistryTag::TD_Time);
					const FString TimeZone = (AttributeMetaData.DisplayFlags & UObject::FAssetRegistryTag::TD_InvariantTz) ? FText::GetInvariantTimeZone() : FString();
	
					if (bDisplayDate && bDisplayTime)
					{
						DisplayValue = FText::AsDateTime(Timestamp, EDateTimeStyle::Short, EDateTimeStyle::Short, TimeZone);
					}
					else if (bDisplayDate)
					{
						DisplayValue = FText::AsDate(Timestamp, EDateTimeStyle::Short, TimeZone);
					}
					else if (bDisplayTime)
					{
						DisplayValue = FText::AsTime(Timestamp, EDateTimeStyle::Short, TimeZone);
					}
				}
			}
	
			// The tag value might be localized text, so we need to parse it for display
			if (!bHasSetDisplayValue && FTextStringHelper::IsComplexText(*AttributeValueStr))
			{
				bHasSetDisplayValue = FTextStringHelper::ReadFromBuffer(*AttributeValueStr, DisplayValue) != nullptr;
			}
	
			// Do our best to build something valid from the string value
			if (!bHasSetDisplayValue)
			{
				bHasSetDisplayValue = true;
	
				// Since all we have at this point is a string, we can't be very smart here.
				// We need to strip some noise off class paths in some cases, but can't load the asset to inspect its UPROPERTYs manually due to performance concerns.
				FString ValueString = FPackageName::ExportTextPathToObjectPath(AttributeValueStr);
	
				const TCHAR StringToRemove[] = TEXT("/Script/");
				if (ValueString.StartsWith(StringToRemove))
				{
					// Remove the class path for native classes, and also remove Engine. for engine classes
					const int32 SizeOfPrefix = UE_ARRAY_COUNT(StringToRemove) - 1;
					ValueString.MidInline(SizeOfPrefix, ValueString.Len() - SizeOfPrefix, EAllowShrinking::No);
					ValueString.ReplaceInline(TEXT("Engine."), TEXT(""));
				}
	
				if (ItemAssetData.IsValid())
				{
					if (const UClass* AssetClass = ItemAssetData.GetClass())
					{
						if (const FProperty* TagField = FindFProperty<FProperty>(AssetClass, AttributeName))
						{
							const FProperty* TagProp = nullptr;
							const UEnum* TagEnum = nullptr;
							if (const FByteProperty* ByteProp = CastField<FByteProperty>(TagField))
							{
								TagProp = ByteProp;
								TagEnum = ByteProp->Enum;
							}
							else if (const FEnumProperty* EnumProp = CastField<FEnumProperty>(TagField))
							{
								TagProp = EnumProp;
								TagEnum = EnumProp->GetEnum();
							}

							// Strip off enum prefixes if they exist
							if (TagProp)
							{
								if (TagEnum)
								{
									const FString EnumPrefix = TagEnum->GenerateEnumPrefix();
									if (EnumPrefix.Len() && ValueString.StartsWith(EnumPrefix))
									{
										ValueString.RightChopInline(EnumPrefix.Len() + 1, EAllowShrinking::No);	// +1 to skip over the underscore
									}
								}

								ValueString = FName::NameToDisplayString(ValueString, false);
							}
						}
					}
				}
	
				DisplayValue = FText::AsCultureInvariant(MoveTemp(ValueString));
			}
			
			// Add suffix to the value, if one is defined for this tag
			if (!AttributeMetaData.Suffix.IsEmpty())
			{
				DisplayValue = FText::Format(LOCTEXT("DisplayTagSuffixFmt", "{0} {1}"), DisplayValue, AttributeMetaData.Suffix);
			}
		}
	
		if (!DisplayValue.IsEmpty())
		{
			CachedDisplayTags.Add(FTagDisplayItem(AttributeName, AttributeMetaData.DisplayName, DisplayValue, AttributeMetaData.bIsImportant));
		}
	}
}

const FSlateBrush* SAssetViewItem::GetBorderImage() const
{
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		return bDraggedOver ? FAppStyle::GetBrush("AssetThumbnail.AssetBorder") : FAppStyle::GetBrush("NoBorder");
	}
	else
	{
		return bDraggedOver ? FAppStyle::GetBrush("Menu.Background") : FAppStyle::GetBrush("NoBorder");
	}
}

bool SAssetViewItem::IsFolder() const
{
	return AssetItem && AssetItem->IsFolder();
}

FText SAssetViewItem::GetNameText() const
{
	return AssetItem
		? AssetItem->GetItem().GetDisplayName()
		: FText();
}

FSlateColor SAssetViewItem::GetAssetColor() const
{
	if (AssetItem)
	{
		FContentBrowserItemDataAttributeValue ColorAttributeValue = AssetItem->GetItem().GetItemAttribute(ContentBrowserItemAttributes::ItemColor);
		if (ColorAttributeValue.IsValid())
		{
			const FString ColorStr = ColorAttributeValue.GetValue<FString>();

			FLinearColor Color;
			if (Color.InitFromString(ColorStr))
			{
				return Color;
			}
		}
		else if (AssetItem->GetItem().IsFolder())
		{
			const bool bCollectionFolder = EnumHasAnyFlags(AssetItem->GetItem().GetItemCategory(), EContentBrowserItemFlags::Category_Collection);
			if (bCollectionFolder)
			{
				TSharedPtr<ICollectionContainer> CollectionContainer;
				FName CollectionName;
				ECollectionShareType::Type CollectionFolderShareType = ECollectionShareType::CST_All;
				ContentBrowserUtils::IsCollectionPath(AssetItem->GetItem().GetVirtualPath().ToString(), &CollectionContainer , &CollectionName, &CollectionFolderShareType);
				
				if (TOptional<FLinearColor> Color = CollectionViewUtils::GetCustomColor(CollectionContainer.Get(), CollectionName, CollectionFolderShareType))
				{
					return Color.GetValue();
				}
			}
			else
			{
				if (TOptional<FLinearColor> Color = ContentBrowserUtils::GetPathColor(AssetItem->GetItem().GetInvariantPath().ToString()))
				{
					return Color.GetValue();
				}
			}
		}

		if(!AssetItem->GetItem().IsSupported())
		{
			return FSlateColor::UseForeground();
		}
	}
	return ContentBrowserUtils::GetDefaultColor();
}

bool SAssetViewItem::OnVisualizeTooltip(const TSharedPtr<SWidget>& TooltipContent)
{
	// OnVisualizeTooltip will be called when tooltips are opening for any children of SAssetColumnViewRow
	// So we only want custom visualization for the parent row's SAssetViewItemToolTip
	TSharedPtr<IToolTip> ThisTooltip = GetToolTip();
	if (!ThisTooltip.IsValid() || (ThisTooltip->AsWidget() != TooltipContent))
	{
		return false;
	}

	if(OnVisualizeAssetToolTip.IsBound() && TooltipContent.IsValid() && AssetItem && AssetItem->IsFile())
	{
		FAssetData ItemAssetData;
		if (AssetItem->GetItem().Legacy_TryGetAssetData(ItemAssetData))
		{
			return OnVisualizeAssetToolTip.Execute(TooltipContent, ItemAssetData);
		}
	}

	// No custom behavior, return false to allow slate to visualize the widget
	return false;
}

void SAssetViewItem::OnToolTipClosing()
{
	OnAssetToolTipClosing.ExecuteIfBound();
}

TSharedRef<SWidget> SAssetViewItem::GetAssetAccessSpecifierIconBox() const
{
	if (ensure(AssetAccessSpecifierIconBox.IsValid()))
	{
		return AssetAccessSpecifierIconBox.ToSharedRef();
	}

	return SNullWidget::NullWidget;
}

void SAssetViewItem::UpdateAssetAccessSpecifierIcon()
{
	TSharedRef<SWidget> IconWidget = SNullWidget::NullWidget;

	// If there's a valid, modifiable EAssetAccessSpecifier, render its state. If there isn't one, or it isn't modifiable, it's pointless to show it.
	EAssetAccessSpecifier AssetAccessSpecifier = EAssetAccessSpecifier::Private;
	if (AssetItem.IsValid())
	{
		FName IconName = AssetItem->GetItemAssetAccessSpecifierIconStyleName();
		if (IconName != NAME_None)
		{
			// Wrap the icon in an outer background layer to improve visibility on light thumbnails
			const float Padding = 6.f;
			const float OuterBoxSize = 16.f;
			IconWidget = SNew(SBox)
			.Padding(0, 0, Padding, Padding)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("AssetThumbnail", ".AssetThumbnailStatusBar"))
				[
					SNew(SBox)
					.WidthOverride(OuterBoxSize)
					.HeightOverride(OuterBoxSize)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(IconName))
					]
				]
			];
		}
	}

	if (ensure(AssetAccessSpecifierIconBox.IsValid()))
	{
		AssetAccessSpecifierIconBox->SetContent(IconWidget);
	}
}

FText SAssetViewItem::GetAssetAccessSpecifierText() const
{
	return AssetItem.IsValid() ? AssetItem->GetItemAssetAccessSpecifierText()
		: FText::GetEmpty();
}

///////////////////////////////
// SAssetListItem
///////////////////////////////

SAssetListItem::~SAssetListItem()
{

}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SAssetListItem::Construct( const FArguments& InArgs )
{
	SAssetViewItem::Construct( SAssetViewItem::FArguments()
		.AssetItem(InArgs._AssetItem)
		.OnRenameBegin(InArgs._OnRenameBegin)
		.OnRenameCommit(InArgs._OnRenameCommit)
		.OnVerifyRenameCommit(InArgs._OnVerifyRenameCommit)
		.OnItemDestroyed(InArgs._OnItemDestroyed)
		.ShouldAllowToolTip(InArgs._ShouldAllowToolTip)
		.ThumbnailEditMode(InArgs._ThumbnailEditMode)
		.ShowAssetAccessSpecifier(InArgs._ShowAssetAccessSpecifier)
		.HighlightText(InArgs._HighlightText)
		.OnIsAssetValidForCustomToolTip(InArgs._OnIsAssetValidForCustomToolTip)
		.OnGetCustomAssetToolTip(InArgs._OnGetCustomAssetToolTip)
		.OnVisualizeAssetToolTip(InArgs._OnVisualizeAssetToolTip)
		.OnAssetToolTipClosing( InArgs._OnAssetToolTipClosing )
		.IsSelected(InArgs._IsSelected)
		);

	AssetThumbnail = InArgs._AssetThumbnail;
	ItemHeight = InArgs._ItemHeight;
	IsSelectedExclusively = InArgs._IsSelectedExclusively;
	HighlightText = InArgs._HighlightText;
	ThumbnailPadding = InArgs._ThumbnailPadding;
	CurrentThumbnailSize = InArgs._CurrentThumbnailSize;

	if ( AssetItem.IsValid() && AssetThumbnail.IsValid() )
	{
		FAssetThumbnailConfig ThumbnailConfig;
		ThumbnailConfig.bAllowFadeIn = true;
		ThumbnailConfig.bAllowHintText = InArgs._AllowThumbnailHintLabel;
		ThumbnailConfig.bForceGenericThumbnail = AssetItem->GetItem().GetItemTemporaryReason() == EContentBrowserItemFlags::Temporary_Creation;
		ThumbnailConfig.AllowAssetSpecificThumbnailOverlay = !ThumbnailConfig.bForceGenericThumbnail;
		ThumbnailConfig.ThumbnailLabel = InArgs._ThumbnailLabel;
		ThumbnailConfig.HighlightedText = InArgs._HighlightText;
		ThumbnailConfig.HintColorAndOpacity = InArgs._ThumbnailHintColorAndOpacity;

		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			ThumbnailConfig.AdditionalTooltipInSmallView = MakeAttributeSP(this, &SAssetListItem::GenerateExtraStateTooltipWidgetNewStyle);
			ThumbnailConfig.IsEditModeVisible = MakeAttributeSP(this, &SAssetListItem::GetThumbnailEditModeUIVisibility);
			ThumbnailConfig.AllowAssetSpecificThumbnailOverlay = MakeAttributeSP(this, &SAssetListItem::ShouldAllowAssetActionOverlay, ThumbnailConfig.bForceGenericThumbnail);
			ThumbnailConfig.AllowAssetSpecificThumbnailOverlayIndicator = MakeAttributeSP(this, &SAssetListItem::ShouldAllowAssetActionOverlayIndicator, ThumbnailConfig.bForceGenericThumbnail);
			ThumbnailConfig.GenericThumbnailSize = MakeAttributeSP(this, &SAssetListItem::GetGenericThumbnailSize);
			ThumbnailConfig.AssetSystemInfoProvider = MakeShared<FAssetSystemContentBrowserInfoProvider>(AssetItem);
			ThumbnailConfig.AllowAssetStatusThumbnailOverlay = MakeAttributeSP(this, &SAssetListItem::ShouldAllowAssetStatusOverlay, ThumbnailConfig.bForceGenericThumbnail);
			ThumbnailConfig.ShowAssetColor = true;
			ThumbnailConfig.ShowAssetBorder = true;
			ThumbnailConfig.AlwaysExpandTooltip = MakeAttributeSP(this, &SAssetListItem::ShouldExpandTooltip);
		}

		if(!AssetItem->GetItem().IsSupported())
		{
			ThumbnailConfig.ClassThumbnailBrushOverride = FName("Icons.WarningWithColor.Thumbnail");
		}

		{
			FContentBrowserItemDataAttributeValue ColorAttributeValue = AssetItem->GetItem().GetItemAttribute(ContentBrowserItemAttributes::ItemColor);
			if (ColorAttributeValue.IsValid())
			{
				const FString ColorStr = ColorAttributeValue.GetValue<FString>();

				FLinearColor Color;
				if (Color.InitFromString(ColorStr))
				{
					ThumbnailConfig.AssetTypeColorOverride = Color;
				}
			}
		}

		ThumbnailWidget = AssetThumbnail->MakeThumbnailWidget(ThumbnailConfig);

		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			// Use the same tooltip as the Thumbnail
			if (const TSharedPtr<IToolTip>& ThumbnailTooltip = ThumbnailWidget->GetToolTip())
			{
				ThumbnailWidgetITooltip = ThumbnailTooltip;
			}
			// reset the thumbnail tooltip to avoid calling that, we handle our tooltip
			ThumbnailWidget->SetToolTip(TSharedPtr<IToolTip>());
		}
	}
	else
	{
		ThumbnailWidget = SNew(SImage) .Image( FAppStyle::GetDefaultBrush() );
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SAssetListItem::OnAssetDataChanged()
{
	SAssetViewItem::OnAssetDataChanged();

	if (PathTextWidget.IsValid())
	{
		PathTextWidget->SetText(GetAssetPathText());
	}

	if (AssetThumbnail)
	{
		bool bSetThumbnail = false;
		if (AssetItem)
		{
			bSetThumbnail = AssetItem->GetItem().UpdateThumbnail(*AssetThumbnail);
		}
		if (!bSetThumbnail)
		{
			AssetThumbnail->SetAsset(FAssetData());
		}
	}

	UpdateAssetAccessSpecifierIcon();

	if (AssetAccessSpecifierText.IsValid())
	{
		AssetAccessSpecifierText->SetText(GetAssetAccessSpecifierText());
	}
}

void SAssetListItem::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SAssetViewItem::OnMouseEnter(MyGeometry, MouseEvent);

	if (AssetThumbnail.IsValid())
	{
		AssetThumbnail->SetRealTime(true);
	}
}

void SAssetListItem::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SAssetViewItem::OnMouseLeave(MouseEvent);

	if (AssetThumbnail.IsValid())
	{
		AssetThumbnail->SetRealTime(false);
	}
}

FSlateColor SAssetListItem::GetColumnTextColor(FIsSelected InIsSelected) const
{
	const bool bIsSelected = InIsSelected.IsBound() ? InIsSelected.Execute() : false;
	const bool bIsHoveredOrDraggedOver = IsHovered() || bDraggedOver;
	if (bIsSelected || bIsHoveredOrDraggedOver)
	{
		return FStyleColors::White;
	}

	return FSlateColor::UseForeground();
}

FText SAssetListItem::GetAssetPathText() const
{
	return AssetItem ? AssetItem->GetAssetPathText() : FText::GetEmpty();
}

FText SAssetListItem::GetAssetTagText(FName InTag) const
{
	if (AssetItem)
	{
		// Check custom type
		{
			FText TagText;
			if (AssetItem->GetCustomColumnDisplayValue(InTag, TagText))
			{
				return TagText;
			}
		}

		// Check display tags
		{
			const FTagDisplayItem* FoundTagItem = CachedDisplayTags.FindByPredicate([InTag](const FTagDisplayItem& TagItem)
			{
				return TagItem.TagKey == InTag;
			});
			
			if (FoundTagItem)
			{
				return FoundTagItem->DisplayValue;
			}
		}

	}
	
	return FText();
}

int32 SAssetListItem::GetWidgetSwitcherIndex() const
{
	constexpr int32 RealThumbnailIndex = 1;
	constexpr int32 IconThumbnailIndex = 0;

	if (CurrentThumbnailSize.IsSet())
	{
		if (IsFolder())
		{
			return CurrentThumbnailSize.Get() >= EThumbnailSize::Medium ? RealThumbnailIndex : IconThumbnailIndex;
		}

		if (CurrentThumbnailSize.Get() == EThumbnailSize::Tiny)
		{
			return IconThumbnailIndex;
		}
	}

	// By default show the real thumbnail
	return RealThumbnailIndex;
}

EVisibility SAssetListItem::GetDirtyBadgeVisibility() const
{
	return CurrentThumbnailSize.IsSet() && CurrentThumbnailSize.Get() <= EThumbnailSize::Medium ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedRef<SWidget> SAssetListItem::GenerateWidgetForColumn(const FName& ColumnName, FIsSelected InIsSelected)
{
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		return GenerateWidgetForColumnNewStyle(ColumnName, InIsSelected);
	}

	TSharedPtr<SWidget> Content;

	using namespace UE::ContentBrowser::Private;

	if (ColumnName == "Name")
	{
		FName ItemShadowBorderName;
		TSharedRef<SWidget> ItemContents = FAssetViewItemHelper::CreateListItemContents(this, ThumbnailWidget.ToSharedRef(), ItemShadowBorderName, 0.f);

		Content = SNew(SBorder)
		.BorderImage(this, &SAssetViewItem::GetBorderImage)
		.Padding(0)
		.AddMetaData<FTagMetaData>(FTagMetaData(AssetItem->GetItem().GetVirtualPath()))
		[
			SNew(SHorizontalBox)

			// Viewport
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew( SBox )
				.Padding(ThumbnailPadding - 4.f)
				.WidthOverride( this, &SAssetListItem::GetThumbnailBoxSize )
				.HeightOverride( this, &SAssetListItem::GetThumbnailBoxSize )
				[
					ItemContents
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(6, 0, 0, 0)
			.VAlign(VAlign_Center)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0, 1)
				[
					SAssignNew(InlineRenameWidget, SInlineEditableTextBlock)
					.Font(FContentBrowserStyle::Get().GetFontStyle("ContentBrowser.AssetTileViewNameFont"))
					.Text( GetNameText() )
					.OnBeginTextEdit(this, &SAssetListItem::HandleBeginNameChange)
					.OnTextCommitted(this, &SAssetListItem::HandleNameCommitted)
					.OnVerifyTextChanged(this, &SAssetListItem::HandleVerifyNameChanged)
					.HighlightText(HighlightText)
					.IsSelected(IsSelectedExclusively)
					.IsReadOnly(this, &SAssetListItem::IsNameReadOnly)
					.ColorAndOpacity(this, &SAssetListItem::GetColumnTextColor, InIsSelected)
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0, 1)
				[
					// Class
					SAssignNew(ClassTextWidget, STextBlock)
					.Font(FContentBrowserStyle::Get().GetFontStyle("ContentBrowser.AssetListViewClassFont"))
					.Text(GetAssetClassText())
					.HighlightText(HighlightText)
					.ColorAndOpacity(this, &SAssetListItem::GetColumnTextColor, InIsSelected)
				]
			]
		];

		if(AssetItem.IsValid())
		{
			AssetItem->OnRenameRequested().BindSP(InlineRenameWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);
			AssetItem->OnRenameCanceled().BindSP(InlineRenameWidget.Get(), &SInlineEditableTextBlock::ExitEditingMode);
		}

	}
	else if ( ColumnName == "RevisionControl")
	{
		Content = SNew(SBox)
			.WidthOverride(16.0f)
			.HeightOverride(16.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				GenerateSourceControlIconWidget()
			];
	}
	
	HandleSourceControlStateChanged();

	return Content.ToSharedRef();
}

int32 SAssetListItem::GetGenericThumbnailSize() const
{
	const int32 ThumbnailSizeIdx = (int32)CurrentThumbnailSize.Get();
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		static int32 GenericThumbnailSizes[(int32)EThumbnailSize::MAX] = { 22, 24, 32, 48, 64, 80 };
		return GenericThumbnailSizes[ThumbnailSizeIdx];
	}
	else
	{
		static int32 GenericThumbnailSizes[(int32)EThumbnailSize::MAX] = { 24, 32, 64, 128, 160, 200 };
		return GenericThumbnailSizes[ThumbnailSizeIdx];
	}
}

bool SAssetListItem::ShouldAllowAssetStatusOverlay(bool bForceGenericThumbnail) const
{
	return !bForceGenericThumbnail && CurrentThumbnailSize.Get() >= EThumbnailSize::Large;
}

bool SAssetListItem::ShouldAllowAssetActionOverlay(bool bForceGenericThumbnail) const
{
	return !bForceGenericThumbnail;
}

bool SAssetListItem::ShouldAllowAssetActionOverlayIndicator(bool bForceGenericThumbnail) const
{
	return !bForceGenericThumbnail && CurrentThumbnailSize.Get() >= EThumbnailSize::Small;
}

TSharedRef<SWidget> SAssetListItem::GenerateWidgetForColumnNewStyle(const FName& ColumnName, FIsSelected InIsSelected)
{
	TSharedPtr<SWidget> Content;
	const FMargin DefaultColumnPadding = FMargin(8.f, 0.f, 0.f, 0.f);

	if (ColumnName == "Name")
	{
		FName ItemShadowBorderName;
		TSharedRef<SWidget> ItemContents = FAssetViewItemHelper::CreateListItemContents(this, ThumbnailWidget.ToSharedRef(), ItemShadowBorderName, 0.f);

		const FSlateBrush* IconBrush;
		if (IsFolder())
		{
			if (ContentBrowserUtils::IsItemDeveloperContent(AssetItem->GetItem()))
			{
				IconBrush = UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.ColumnViewDeveloperFolderIcon");
			}
			else
			{
				IconBrush = UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.ColumnViewFolderIcon");
			}
		}
		else
		{
			if (!AssetItem->GetItem().IsSupported()) 
			{
				IconBrush = FAppStyle::GetBrush("Icons.WarningWithColor");
			}
			else
			{
				IconBrush = UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.FileImage");
			}
		}

		// Make dirty icon overlay a reasonable size in relation to the icon size (note: it is assumed this icon is square)
		const float IconOverlaySize = IconBrush->ImageSize.X * 0.6f;

		Content = SNew(SBorder)
		.BorderImage(this, &SAssetViewItem::GetBorderImage)
		.Padding(DefaultColumnPadding)
		.AddMetaData<FTagMetaData>(FTagMetaData(AssetItem->GetItem().GetVirtualPath()))
		[
			SNew(SHorizontalBox)

			// Viewport
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride( this, &SAssetListItem::GetThumbnailBoxSize )
				.HeightOverride( this, &SAssetListItem::GetThumbnailBoxSize )
				[
					SNew(SOverlay)

					+ SOverlay::Slot()
					[
						SNew(SWidgetSwitcher)
						.WidgetIndex(this, &SAssetListItem::GetWidgetSwitcherIndex)

						+ SWidgetSwitcher::Slot()
						.VAlign(VAlign_Fill)
						.HAlign(HAlign_Fill)
						[
							SNew(SOverlay)

							+ SOverlay::Slot()
							[
								SNew(SImage)
								.Image(IconBrush)
								.ColorAndOpacity(this, &SAssetViewItem::GetAssetColor)
							]
						]

						+ SWidgetSwitcher::Slot()
						.VAlign(VAlign_Fill)
						.HAlign(HAlign_Fill)
						[
							ItemContents
						]
					]

					// Dirty state
					+SOverlay::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Bottom)
					[
						SNew(SBox)
						.WidthOverride(IconOverlaySize)
						.HeightOverride(IconOverlaySize)
						.Visibility(this, &SAssetListItem::GetDirtyBadgeVisibility)
						[
							SNew(SImage)
							.Image(this, &SAssetListItem::GetDirtyImage)
						]
					]
				]
			]

			+SHorizontalBox::Slot()
			.Padding(6, 0, 0, 0)
			.VAlign(VAlign_Center)
			[
				SAssignNew(InlineRenameWidget, SInlineEditableTextBlock)
				.Font(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetFontStyle("ContentBrowser.AssetTileViewNameFont"))
				.Text(GetNameText())
				.OnBeginTextEdit(this, &SAssetListItem::HandleBeginNameChange)
				.OnTextCommitted(this, &SAssetListItem::HandleNameCommitted)
				.OnVerifyTextChanged(this, &SAssetListItem::HandleVerifyNameChanged)
				.HighlightText(HighlightText)
				.IsSelected(IsSelectedExclusively)
				.IsReadOnly(this, &SAssetListItem::IsNameReadOnly)
				.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
				.ColorAndOpacity(this, &SAssetListItem::GetColumnTextColor, InIsSelected)
			]
		];

		if(AssetItem.IsValid())
		{
			AssetItem->OnRenameRequested().BindSP(InlineRenameWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);
			AssetItem->OnRenameCanceled().BindSP(InlineRenameWidget.Get(), &SInlineEditableTextBlock::ExitEditingMode);
		}

	}
	else if (ColumnName == "RevisionControl")
	{
		Content = SNew(SBox)
			.WidthOverride(16.0f)
			.HeightOverride(16.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				GenerateSourceControlIconWidget()
			];
	}
	else if (ColumnName == "Class")
	{
		Content = SNew(SBox)
			.Padding(DefaultColumnPadding)
			.VAlign(VAlign_Center)
			[
				SAssignNew(ClassTextWidget, STextBlock)
				.Text(GetAssetClassText())
				.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
				.HighlightText(HighlightText)
			];
	}
	else if (ColumnName == "Path")
	{
		Content = SNew(SBox)
			.Padding(DefaultColumnPadding)
			.VAlign(VAlign_Center)
			[
				SAssignNew(PathTextWidget, STextBlock)
				.Text(GetAssetPathText())
				.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
				.HighlightText(HighlightText)
			];
	}
	else if (ColumnName == "AssetAccessSpecifier")
	{
		if(bShowAssetAccessSpecifier)
		{
			Content = SNew(SBox)
				.Padding(DefaultColumnPadding)
				.VAlign(VAlign_Center)
				[
					SAssignNew(AssetAccessSpecifierText, STextBlock)
					.Text(GetAssetAccessSpecifierText())
					.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
					.HighlightText(HighlightText)
				];
		}
	}
	else
	{
		Content = SNew(SBox)
			.Padding(DefaultColumnPadding)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
				.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SAssetListItem::GetAssetTagText, ColumnName)))
			];
	}
	HandleSourceControlStateChanged();

	return Content.ToSharedRef();
}


float SAssetListItem::GetExtraStateIconWidth() const
{
	// Each individual extra state icon will also simply be limited to the max width so things like long text show up properly
	return GetExtraStateIconMaxSize().Get();
}

FOptionalSize SAssetListItem::GetExtraStateIconMaxSize() const
{
	return GetThumbnailBoxSize().Get() * 0.7f;
}

FOptionalSize SAssetListItem::GetStateIconImageSize() const
{
	float IconSize = FMath::TruncToFloat(GetThumbnailBoxSize().Get() * 0.3f);
	return IconSize > 12 ? IconSize : 12;
}

FOptionalSize SAssetListItem::GetThumbnailBoxSize() const
{
	return FOptionalSize( ItemHeight.Get() );
}

///////////////////////////////
// SAssetTileItem
///////////////////////////////

SAssetTileItem::~SAssetTileItem()
{

}

void SAssetTileItem::Construct( const FArguments& InArgs )
{
	SAssetViewItem::Construct( SAssetViewItem::FArguments()
		.AssetItem(InArgs._AssetItem)
		.OnRenameBegin(InArgs._OnRenameBegin)
		.OnRenameCommit(InArgs._OnRenameCommit)
		.OnVerifyRenameCommit(InArgs._OnVerifyRenameCommit)
		.OnItemDestroyed(InArgs._OnItemDestroyed)
		.ShouldAllowToolTip(InArgs._ShouldAllowToolTip)
		.ThumbnailEditMode(InArgs._ThumbnailEditMode)
		.HighlightText(InArgs._HighlightText)
		.OnIsAssetValidForCustomToolTip(InArgs._OnIsAssetValidForCustomToolTip)
		.OnGetCustomAssetToolTip(InArgs._OnGetCustomAssetToolTip)
		.OnVisualizeAssetToolTip(InArgs._OnVisualizeAssetToolTip)
		.OnAssetToolTipClosing( InArgs._OnAssetToolTipClosing )
		.IsSelected(InArgs._IsSelected)
		.ShowAssetAccessSpecifier(InArgs._ShowAssetAccessSpecifier)
		);

	bShowType = InArgs._ShowType;
	AssetThumbnail = InArgs._AssetThumbnail;
	ItemWidth = InArgs._ItemWidth;
	ThumbnailPadding = InArgs._ThumbnailPadding;
	ThumbnailDimension = InArgs._ThumbnailDimension;

	CurrentThumbnailSize = InArgs._CurrentThumbnailSize;

	InitializeAssetNameHeights();

	if ( AssetItem.IsValid() && AssetThumbnail.IsValid() )
	{
		FAssetThumbnailConfig ThumbnailConfig;
		ThumbnailConfig.bAllowFadeIn = true;
		ThumbnailConfig.bAllowHintText = InArgs._AllowThumbnailHintLabel;
		ThumbnailConfig.bAllowRealTimeOnHovered = false; // we use our own OnMouseEnter/Leave for logical asset item
		ThumbnailConfig.bForceGenericThumbnail = AssetItem->GetItem().GetItemTemporaryReason() == EContentBrowserItemFlags::Temporary_Creation;
		ThumbnailConfig.AllowAssetSpecificThumbnailOverlay = !ThumbnailConfig.bForceGenericThumbnail;
		ThumbnailConfig.ThumbnailLabel = InArgs._ThumbnailLabel;
		ThumbnailConfig.HighlightedText = InArgs._HighlightText;
		ThumbnailConfig.HintColorAndOpacity = InArgs._ThumbnailHintColorAndOpacity;
		ThumbnailConfig.GenericThumbnailSize = MakeAttributeSP(this, &SAssetTileItem::GetGenericThumbnailSize);

		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			ThumbnailConfig.AdditionalTooltipInSmallView = MakeAttributeSP(this, &SAssetTileItem::GenerateExtraStateTooltipWidgetNewStyle);
			ThumbnailConfig.AssetSystemInfoProvider = MakeShared<FAssetSystemContentBrowserInfoProvider>(AssetItem);
			ThumbnailConfig.AllowAssetStatusThumbnailOverlay = true;
			ThumbnailConfig.ShowAssetColor = true;
			ThumbnailConfig.AssetBorderImageOverride = TAttribute<const FSlateBrush*>::CreateSP(this, &SAssetTileItem::GetAssetAreaOverlayBackgroundImage);
			ThumbnailConfig.bCanDisplayEditModePrimitiveTools = CanDisplayPrimitiveTools();
			ThumbnailConfig.AlwaysExpandTooltip = MakeAttributeSP(this, &SAssetTileItem::ShouldExpandTooltip);
			ThumbnailConfig.IsEditModeVisible = MakeAttributeSP(this, &SAssetTileItem::GetThumbnailEditModeUIVisibility);
		}

		if(!AssetItem->GetItem().IsSupported())
		{
			ThumbnailConfig.ClassThumbnailBrushOverride = FName("Icons.WarningWithColor.Thumbnail");
		}
		
		{
			FContentBrowserItemDataAttributeValue ColorAttributeValue = AssetItem->GetItem().GetItemAttribute(ContentBrowserItemAttributes::ItemColor);
			if (ColorAttributeValue.IsValid())
			{
				const FString ColorStr = ColorAttributeValue.GetValue<FString>();

				FLinearColor Color;
				if (Color.InitFromString(ColorStr))
				{
					ThumbnailConfig.AssetTypeColorOverride = Color;
				}
			}
		}

		Thumbnail = AssetThumbnail->MakeThumbnailWidget(ThumbnailConfig);

		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			// Use the same tooltip as the Thumbnail
			if (const TSharedPtr<IToolTip>& ThumbnailTooltip = Thumbnail->GetToolTip())
			{
				ThumbnailWidgetITooltip = ThumbnailTooltip;
			}
			// reset the thumbnail tooltip to avoid calling that, we handle our tooltip
			Thumbnail->SetToolTip(TSharedPtr<IToolTip>());
		}
	}
	else
	{
		Thumbnail = SNew(SImage).Image(FAppStyle::GetDefaultBrush());
	}

	FName ItemShadowBorderName;
	TSharedRef<SWidget> ItemContents = FAssetViewItemHelper::CreateTileItemContents(this, Thumbnail.ToSharedRef(), ItemShadowBorderName, FMargin(5.f), InArgs._ShowAssetAccessSpecifier);

	if (bShowType)
	{
		SAssignNew(InlineRenameWidget, SInlineEditableTextBlock)
			.Font(this, &SAssetTileItem::GetThumbnailFont)
			.Text(GetNameText())
			.OnBeginTextEdit(this, &SAssetTileItem::HandleBeginNameChange)
			.OnTextCommitted(this, &SAssetTileItem::HandleNameCommitted)
			.OnVerifyTextChanged(this, &SAssetTileItem::HandleVerifyNameChanged)
			.HighlightText(InArgs._HighlightText)
			.IsSelected(InArgs._IsSelectedExclusively)
			.IsReadOnly(this, &SAssetTileItem::IsNameReadOnly)
			.LineBreakPolicy(FBreakIterator::CreateCamelCaseBreakIterator())
			.OverflowPolicy(ETextOverflowPolicy::MultilineEllipsis)
			.ColorAndOpacity(this, &SAssetTileItem::GetNameAreaTextColor);
	}
	else
	{
		SAssignNew(InlineRenameWidget, SInlineEditableTextBlock)
			.Font(this, &SAssetTileItem::GetThumbnailFont)
			.Text(GetNameText())
			.OnBeginTextEdit(this, &SAssetTileItem::HandleBeginNameChange)
			.OnTextCommitted(this, &SAssetTileItem::HandleNameCommitted)
			.OnVerifyTextChanged(this, &SAssetTileItem::HandleVerifyNameChanged)
			.HighlightText(InArgs._HighlightText)
			.IsSelected(InArgs._IsSelectedExclusively)
			.IsReadOnly(this, &SAssetTileItem::IsNameReadOnly)
			.OverflowPolicy(ETextOverflowPolicy::MultilineEllipsis)
			.ColorAndOpacity(this, &SAssetTileItem::GetNameAreaTextColor);
	}

	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		constexpr float BorderPadding = 1.f;
		constexpr float ShadowLeftTopPadding = 3.f;
		constexpr float ShadowRightBotPadding = 4.f;
		constexpr float ThumbnailBorderPadding = 0.f;
		constexpr float NameAreaBoxLeftRightBotPadding = 4.f;
		constexpr float NameAreaBoxTopPadding = 6.f;
		constexpr float ClassNameMaxHeight = 14.f;
		const FMargin FolderImagePadding = FMargin(2.f, 2.f, 2.f, 2.f);
		const FMargin FolderOverlayNamePaddingFromLeft = FMargin(9.f, 0.f, 0.f, 0.f);

		ChildSlot
		.Padding(FMargin(BorderPadding, BorderPadding, 0.f, 1.f))
		[
			// Drop shadow border
			SNew(SBorder)
			.Padding(FMargin(ShadowLeftTopPadding, ShadowLeftTopPadding, ShadowRightBotPadding, ShadowRightBotPadding))
			.BorderImage(IsFolder() ? TAttribute<const FSlateBrush*>(this, &SAssetTileItem::GetFolderBackgroundShadowImage) : FAppStyle::Get().GetBrush(ItemShadowBorderName))
			[
				SNew(SOverlay)
				.AddMetaData<FTagMetaData>(FTagMetaData(AssetItem->GetItem().GetVirtualPath()))
				+SOverlay::Slot()
				[
					SNew(SBorder)
					.Padding(ThumbnailBorderPadding)
					.BorderImage(IsFolder() ? TAttribute<const FSlateBrush*>(this, &SAssetTileItem::GetFolderBackgroundImage) : TAttribute<const FSlateBrush*>(this, &SAssetTileItem::GetNameAreaBackgroundImage))
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SOverlay)

							// Thumbnail
							+ SOverlay::Slot()
							.Padding(IsFolder() ? FolderImagePadding : 0.f)
							[
								SNew(SBorder)
								.Padding(0.f)
								.BorderImage(IsFolder() ? TAttribute<const FSlateBrush*>(this, &SAssetTileItem::GetFolderSlotBackgroundImage) : FAppStyle::GetNoBrush())
								[
									// The remainder of the space is reserved for the name.
									SNew(SBox)
									.WidthOverride(this, &SAssetTileItem::GetThumbnailBoxSize)
									.HeightOverride(this, &SAssetTileItem::GetThumbnailBoxSize)
									[
										ItemContents
									]
								]
							]

							// Overlay name for tiny folders
							+ SOverlay::Slot()
							.Padding(FolderOverlayNamePaddingFromLeft)
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SNew(SBox)
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.WidthOverride(this, &SAssetTileItem::GetTinyFolderNameWidth)
								[
									SNew(SBorder)
									.Padding(4.f, 2.f)
									.BorderImage(FAppStyle::GetBrush("ContentBrowser.AssetTileItem.TinyFolderTextBorder"))
									.Visibility(this, &SAssetTileItem::GetTinyFolderNameOverlayVisibility)
									[
										SNew(STextBlock)
										.Font(FStyleFonts::Get().Small)
										.ColorAndOpacity(FStyleColors::White)
										.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
										.Text(GetNameText())
									]
								]
							]
						]

						+SVerticalBox::Slot()
						.FillHeight(1.f)
						[
							SNew(SBox)
							.Padding(FMargin(NameAreaBoxLeftRightBotPadding, NameAreaBoxTopPadding, NameAreaBoxLeftRightBotPadding, NameAreaBoxLeftRightBotPadding))
							.Visibility(this, &SAssetTileItem::GetNameAreaVisibility)
							[
								SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.HAlign(IsFolder() ? HAlign_Center : HAlign_Fill)
								[
									SNew(SBox)
									.VAlign(VAlign_Top)
									.HeightOverride(this, &SAssetTileItem::GetNameAreaMaxDesiredHeight)
									[
										InlineRenameWidget.ToSharedRef()
									]
								]
								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(SBox)
									.HeightOverride(ClassNameMaxHeight)
									.VAlign(VAlign_Bottom)
									[
										SAssignNew(ClassTextWidget, STextBlock)
										.Font(FAppStyle::GetFontStyle("ContentBrowser.AssetTileViewClassNameFont"))
										.Visibility(this, &SAssetTileItem::GetAssetClassLabelVisibility)
										.TextStyle(UE::ContentBrowser::Private::FContentBrowserStyle::Get().Get(), "ContentBrowser.ClassFont")
										.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
										.Text(this, &SAssetTileItem::GetAssetClassText)
										.ColorAndOpacity(this, &SAssetTileItem::GetAssetClassLabelTextColor)
										.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("AssetTileClassTextBlock")))
									]
								]
							]
						]
					]
				]
			]
		];
	}
	else
	{
		constexpr float AssetViewWidgetsBorderPadding = 4.f;
		constexpr float AssetViewWidgetsShadowPadding = 5.f;
		constexpr float AssetViewWidgetsRenameWidgetPadding = 2.f;

		ChildSlot
		.Padding(FMargin(0.0f, 0.0f, AssetViewWidgetsBorderPadding, AssetViewWidgetsBorderPadding))
		[
			// Drop shadow border
			SNew(SBorder)
			.Padding(FMargin(0.0f, 0.0f, AssetViewWidgetsShadowPadding, AssetViewWidgetsShadowPadding))
			.BorderImage(IsFolder() ? TAttribute<const FSlateBrush*>(this, &SAssetTileItem::GetFolderBackgroundShadowImage) : FAppStyle::Get().GetBrush(ItemShadowBorderName))
			[
				SNew(SOverlay)
				.AddMetaData<FTagMetaData>(FTagMetaData(AssetItem->GetItem().GetVirtualPath()))
				+SOverlay::Slot()
				[
					SNew(SBorder)
					.Padding(0)
					.BorderImage(IsFolder() ? TAttribute<const FSlateBrush*>(this, &SAssetTileItem::GetFolderBackgroundImage) : UE::ContentBrowser::Private::FContentBrowserStyle::Get().Get().GetBrush("ContentBrowser.AssetTileItem.ThumbnailAreaBackground"))
					[
						SNew(SVerticalBox)
						// Thumbnail
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							// The remainder of the space is reserved for the name.
							SNew(SBox)
							.Padding(0)
							.WidthOverride(this, &SAssetTileItem::GetThumbnailBoxSize)
							.HeightOverride(this, &SAssetTileItem::GetThumbnailBoxSize)
							[
								ItemContents
							]
						]

						+SVerticalBox::Slot()
						[
							SNew(SBorder)
							.Padding(FMargin(2.0f, 3.0f))
							.BorderImage(this, &SAssetTileItem::GetNameAreaBackgroundImage)
							[
								SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.Padding(AssetViewWidgetsRenameWidgetPadding, AssetViewWidgetsRenameWidgetPadding, 0.0f, 0.0f)
								.VAlign(VAlign_Top)
								.HAlign(IsFolder() ? HAlign_Center : HAlign_Left)
								[
									SNew(SBox)
									.MaxDesiredHeight(this, &SAssetTileItem::GetNameAreaMaxDesiredHeight)
									[
										InlineRenameWidget.ToSharedRef()
									]
								]
								+ SVerticalBox::Slot()
								.VAlign(VAlign_Bottom)
								.AutoHeight()
								.Padding(0.0f, 0.0f, 0.0f, 0.0f)
								[
									SNew(SHorizontalBox)
									+SHorizontalBox::Slot()
									[
										SAssignNew(ClassTextWidget, STextBlock)
										.Visibility(this, &SAssetTileItem::GetAssetClassLabelVisibility)
										.TextStyle(UE::ContentBrowser::Private::FContentBrowserStyle::Get().Get(), "ContentBrowser.ClassFont")
										.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
										.Text(this, &SAssetTileItem::GetAssetClassText)
										.ColorAndOpacity(this, &SAssetTileItem::GetAssetClassLabelTextColor)
										.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("AssetTileClassTextBlock")))
									]

									+SHorizontalBox::Slot()
									.Padding(FMargin(3.0f, 0.0f, 0.0f, 1.0f))
									.AutoWidth()
									.HAlign(HAlign_Right)
									[
										SNew(SBox)
										.WidthOverride(this, &SAssetTileItem::GetSourceControlIconSize)
										.HeightOverride(this, &SAssetTileItem::GetSourceControlIconSize)
										.Visibility(this, &SAssetTileItem::GetSCCIconVisibility)
										[
											GenerateSourceControlIconWidget()
										]
									]
								]

							]
						]
					]
				]
				+SOverlay::Slot()
				[
					SNew(SImage)
					.Image(this, &SAssetViewItem::GetBorderImage)
					.Visibility(EVisibility::HitTestInvisible)
				]
			]
		];
	}

	HandleSourceControlStateChanged();

	if(AssetItem.IsValid())
	{
		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			AssetItem->OnRenameRequested().BindSP(this, &SAssetTileItem::OnRequestingRename);
		}
		else
		{
			AssetItem->OnRenameRequested().BindSP(InlineRenameWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);
		}
		AssetItem->OnRenameCanceled().BindSP(InlineRenameWidget.Get(), &SInlineEditableTextBlock::ExitEditingMode);
	}
}

void SAssetTileItem::OnAssetDataChanged()
{
	SAssetViewItem::OnAssetDataChanged();

	if (AssetThumbnail)
	{
		bool bSetThumbnail = false;
		if (AssetItem)
		{
			bSetThumbnail = AssetItem->GetItem().UpdateThumbnail(*AssetThumbnail);
		}
		if (!bSetThumbnail)
		{
			AssetThumbnail->SetAsset(FAssetData());
		}
	}

	UpdateAssetAccessSpecifierIcon();
}

void SAssetTileItem::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SAssetViewItem::OnMouseEnter(MyGeometry, MouseEvent);

	if (AssetThumbnail.IsValid())
	{
		AssetThumbnail->SetRealTime( true );
	}
}

void SAssetTileItem::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SAssetViewItem::OnMouseLeave(MouseEvent);

	if (AssetThumbnail.IsValid())
	{
		AssetThumbnail->SetRealTime( false );
	}
}

const FSlateBrush* SAssetTileItem::GetBorderImage() const
{
	const bool bIsSelected = IsSelected.IsBound() ? IsSelected.Execute() : false;
	const bool bIsHoveredOrDraggedOver = IsHovered() || bDraggedOver || (Thumbnail.IsValid() && Thumbnail->IsHovered());
	if (bIsSelected && bIsHoveredOrDraggedOver)
	{
		static const FName SelectedHover("ContentBrowser.AssetTileItem.SelectedHoverBorder");
		return FAppStyle::Get().GetBrush(SelectedHover);
	}
	else if (bIsSelected)
	{
		static const FName Selected("ContentBrowser.AssetTileItem.SelectedBorder");
		return FAppStyle::Get().GetBrush(Selected);
	}
	else if (bIsHoveredOrDraggedOver && !IsFolder())
	{
		static const FName Hovered("ContentBrowser.AssetTileItem.HoverBorder");
		return FAppStyle::Get().GetBrush(Hovered);
	}

	return FStyleDefaults::GetNoBrush();
}

void SAssetTileItem::OnRequestingRename()
{
	if (InlineRenameWidget.IsValid() && CurrentThumbnailSize.Get() != EThumbnailSize::Tiny)
	{
		InlineRenameWidget->EnterEditingMode();
		return;
	}

	const FGeometry& WidgetGeometry = GetCachedGeometry();
	FVector2f Position = WidgetGeometry.GetAbsolutePosition();

	// Lower the Y Position of the Widget by Half the Height of the TileItem minus half of the RenameWindow to keep it centered.
	const float AppScale = FSlateApplication::Get().GetApplicationScale();
	constexpr float RenameWindowHalfSize = 15.f;
	Position.Y += (WidgetGeometry.GetAbsoluteSize().Y * 0.5f) - (RenameWindowHalfSize * AppScale);

	// DPI scale and AppScale is handled by the SRenameWindow
	FSlateApplication::Get().AddWindow(
		SNew(SRenameWindow)
		.ScreenPosition(Position)
		.OnBeginTextEdit(this, &SAssetTileItem::HandleBeginNameChange)
		.OnVerifyTextChanged(this, &SAssetTileItem::HandleVerifyNameChanged)
		.InitialText(InlineRenameWidget->GetText())
		.OnTextCommitted(this, &SAssetTileItem::HandleNameCommitted));
}

float SAssetTileItem::GetExtraStateIconWidth() const
{
	// Each individual extra state icon will also simply be limited to the max width so things like long text show up properly
	return GetExtraStateIconMaxSize().Get();
}

FOptionalSize SAssetTileItem::GetExtraStateIconMaxSize() const
{
	return GetThumbnailBoxSize().Get() * 0.7f;
}

FOptionalSize SAssetTileItem::GetStateIconImageSize() const
{
	constexpr int32 SourceControlImageMinSize = 12;
	float IconSize = FMath::TruncToFloat(GetThumbnailBoxSize().Get() * 0.2f);
	return IconSize > SourceControlImageMinSize ? IconSize : SourceControlImageMinSize;
}

FOptionalSize SAssetTileItem::GetThumbnailBoxSize() const
{
	const float ThumbnailSizeOffset = IsFolder() ? -4.f : 0.f;
	return UE::Editor::ContentBrowser::IsNewStyleEnabled() ? FOptionalSize(ThumbnailDimension.Get() + ThumbnailSizeOffset) : FOptionalSize(ItemWidth.Get()- ThumbnailPadding);
}

FOptionalSize SAssetTileItem::GetTinyFolderNameWidth() const
{
	// Folder offset on each side
	constexpr float FolderOffset = 5.f;
	// Folder slot padding on each side
	constexpr float FolderSlotPadding = 2.f;
	// Folder shadow offset on the right and bottom side
	constexpr float FolderShadowOffset = 2.f;
	// Folder padding between image and actual tiny overlay folder name overlay
	constexpr float FolderTinyNameOverlayPadding = 4.f;

	const float FolderFullOffset = (FolderOffset * 2.f) + (FolderSlotPadding * 2.f) + FolderShadowOffset + FolderTinyNameOverlayPadding;
	return FOptionalSize(ThumbnailDimension.Get() - FolderFullOffset);
}

EVisibility SAssetTileItem::GetTinyFolderNameOverlayVisibility() const
{
	if (!UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		return EVisibility::Collapsed;
	}
	// Use the name area visibility so that if the logic change we will still show the overlay only when that's not visible
	return GetNameAreaVisibility() != EVisibility::Visible && IsFolder() ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
}

EVisibility SAssetTileItem::GetAssetClassLabelVisibility() const
{
	return (!IsFolder() && bShowType)
		? EVisibility::Visible
		: EVisibility::Collapsed;
}

FSlateColor SAssetTileItem::GetAssetClassLabelTextColor() const
{
	const bool bIsSelected = IsSelected.IsBound() ? IsSelected.Execute() : false;
	const bool bIsHoveredOrDraggedOver = IsHovered() || bDraggedOver || (Thumbnail.IsValid() && Thumbnail->IsHovered());
	if (bIsSelected || bIsHoveredOrDraggedOver)
	{
		return FStyleColors::White;
	}

	return UE::Editor::ContentBrowser::IsNewStyleEnabled() ? FStyleColors::Hover2 : FSlateColor::UseSubduedForeground();
}

FSlateFontInfo SAssetTileItem::GetThumbnailFont() const
{
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		const static FLazyName RegularFont("ContentBrowser.AssetTileViewNameFont");
		return FAppStyle::GetFontStyle(RegularFont);
	}
	else
	{
		FOptionalSize ThumbSize = GetThumbnailBoxSize();
		if ( ThumbSize.IsSet() )
		{
			float Size = ThumbSize.Get();
			if ( Size < 50 )
			{
				const static FName SmallFontName("ContentBrowser.AssetTileViewNameFontVerySmall");
				return FAppStyle::GetFontStyle(SmallFontName);
			}
			else if ( Size < 85 )
			{
				const static FName SmallFontName("ContentBrowser.AssetTileViewNameFontSmall");
				return FAppStyle::GetFontStyle(SmallFontName);
			}
		}

		const static FName RegularFont("ContentBrowser.AssetTileViewNameFont");
		return FAppStyle::GetFontStyle(RegularFont);
	}
}

const FSlateBrush* SAssetTileItem::GetFolderSlotBackgroundImage() const
{
	const bool bIsSelected = IsSelected.IsBound() ? IsSelected.Execute() : false;
	const bool bIsHoveredOrDraggedOver = IsHovered() || bDraggedOver;

	if (bIsSelected || bIsHoveredOrDraggedOver)
	{
		static const FLazyName SelectedOrHovered("ContentBrowser.AssetTileItem.FolderAreaBackground"); // Panel
		return FAppStyle::Get().GetBrush(SelectedOrHovered);
	}

	return FStyleDefaults::GetNoBrush();
}

const FSlateBrush* SAssetTileItem::GetFolderBackgroundImage() const
{
	const bool bIsSelected = IsSelected.IsBound() ? IsSelected.Execute() : false;
	const bool bIsHoveredOrDraggedOver = IsHovered() || bDraggedOver;

	if (bIsSelected && bIsHoveredOrDraggedOver)
	{
		static const FName SelectedHoverBackground("ContentBrowser.AssetTileItem.FolderAreaSelectedHoverBackground");
		return FAppStyle::Get().GetBrush(SelectedHoverBackground);
	}
	else if (bIsSelected)
	{
		static const FName SelectedBackground("ContentBrowser.AssetTileItem.FolderAreaSelectedBackground");
		return FAppStyle::Get().GetBrush(SelectedBackground);
	}
	else if (bIsHoveredOrDraggedOver)
	{
		static const FName HoveredBackground("ContentBrowser.AssetTileItem.FolderAreaHoveredBackground");
		return FAppStyle::Get().GetBrush(HoveredBackground);
	}

	return FStyleDefaults::GetNoBrush();
}

const FSlateBrush* SAssetTileItem::GetFolderBackgroundShadowImage() const
{
	const bool bIsSelected = IsSelected.IsBound() ? IsSelected.Execute() : false;
	const bool bIsHoveredOrDraggedOver = IsHovered() || bDraggedOver;

	if (bIsSelected || bIsHoveredOrDraggedOver)
	{
		static const FName DropShadowName("ContentBrowser.AssetTileItem.DropShadow");
		return FAppStyle::Get().GetBrush(DropShadowName);
	}

	return FStyleDefaults::GetNoBrush();
}

const FSlateBrush* SAssetTileItem::GetNameAreaBackgroundImage() const
{
	if (CurrentThumbnailSize.Get() == EThumbnailSize::Tiny)
	{
		return FStyleDefaults::GetNoBrush();
	}

	const FName SelectedHover = UE::Editor::ContentBrowser::IsNewStyleEnabled() ? "ContentBrowser.AssetTileItem.AssetContentSelectedHoverBackground" : "ContentBrowser.AssetTileItem.NameAreaSelectedHoverBackground";
	const FName Selected = UE::Editor::ContentBrowser::IsNewStyleEnabled() ? "ContentBrowser.AssetTileItem.AssetContentSelectedBackground" : "ContentBrowser.AssetTileItem.NameAreaSelectedBackground";
	const FName Hovered = UE::Editor::ContentBrowser::IsNewStyleEnabled() ? "ContentBrowser.AssetTileItem.AssetContentHoverBackground" : "ContentBrowser.AssetTileItem.NameAreaHoverBackground";
	const FName Normal = UE::Editor::ContentBrowser::IsNewStyleEnabled() ? "ContentBrowser.AssetTileItem.AssetContent" : "ContentBrowser.AssetTileItem.NameAreaBackground";

	const bool bIsSelected = IsSelected.IsBound() ? IsSelected.Execute() : false;
	const bool bIsHoveredOrDraggedOver = IsHovered() || bDraggedOver || (Thumbnail.IsValid() && Thumbnail->IsHovered());
	if (bIsSelected && bIsHoveredOrDraggedOver)
	{
		return FAppStyle::Get().GetBrush(SelectedHover);
	}
	else if (bIsSelected)
	{
		return FAppStyle::Get().GetBrush(Selected);
	}
	else if (bIsHoveredOrDraggedOver && !IsFolder())
	{
		return FAppStyle::Get().GetBrush(Hovered);
	}
	else if (!IsFolder())
	{
		return FAppStyle::Get().GetBrush(Normal);
	}
	return FStyleDefaults::GetNoBrush();
}

const FSlateBrush* SAssetTileItem::GetAssetAreaOverlayBackgroundImage() const
{
	const bool bIsSelected = IsSelected.IsBound() ? IsSelected.Execute() : false;
	const bool bIsHoveredOrDraggedOver = IsHovered() || bDraggedOver || (Thumbnail.IsValid() && Thumbnail->IsHovered());
	if (bIsSelected && bIsHoveredOrDraggedOver)
	{
		static const FLazyName SelectedHover("ContentBrowser.AssetTileItem.AssetBorderSelectedHoverBackground");
		return FAppStyle::Get().GetBrush(SelectedHover);
	}
	else if (bIsSelected)
	{
		static const FLazyName Selected("ContentBrowser.AssetTileItem.AssetBorderSelectedBackground");
		return FAppStyle::Get().GetBrush(Selected);
	}
	else if (bIsHoveredOrDraggedOver && !IsFolder())
	{
		static const FLazyName Hovered("ContentBrowser.AssetTileItem.AssetBorderHoverBackground");
		return FAppStyle::Get().GetBrush(Hovered);
	}
	else if (!IsFolder())
	{
		static const FLazyName Normal("AssetThumbnail.AssetBorder");
		return FAppStyle::Get().GetBrush(Normal);
	}

	return FStyleDefaults::GetNoBrush();
}

FSlateColor SAssetTileItem::GetNameAreaTextColor() const
{
	const bool bIsSelected = IsSelected.IsBound() ? IsSelected.Execute() : false;
	const bool bIsHoveredOrDraggedOver = IsHovered() || bDraggedOver || (Thumbnail.IsValid() && Thumbnail->IsHovered());
	if (bIsSelected || bIsHoveredOrDraggedOver)
	{
		return FStyleColors::White;
	}

	return FSlateColor::UseForeground();
}

FOptionalSize SAssetTileItem::GetNameAreaMaxDesiredHeight() const
{
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		constexpr int32 MaxHeightNameArea = 42;
		return MaxHeightNameArea;
	}
	else
	{
		return AssetNameHeights[(int32)CurrentThumbnailSize.Get()];
	}
}

int32 SAssetTileItem::GetGenericThumbnailSize() const
{
	const int32 ThumbnailSizeIdx = (int32)CurrentThumbnailSize.Get();
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		static int32 GenericThumbnailSizes[(int32)EThumbnailSize::MAX] = { 80, 96, 112, 128, 136, 160 };
		return GenericThumbnailSizes[ThumbnailSizeIdx];
	}
	else
	{
		static int32 GenericThumbnailSizes[(int32)EThumbnailSize::MAX] = { 24, 32, 64, 128, 160, 200 };
		return GenericThumbnailSizes[ThumbnailSizeIdx];
	}
}

EVisibility SAssetTileItem::GetSCCIconVisibility() const
{
	// Hide the scc state icon when there is no brush or in tiny size since there isn't enough space
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		return bHasCCStateBrush && ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable() ? EVisibility::Visible : EVisibility::Collapsed;
	}
	else
	{
		return bHasCCStateBrush &&  CurrentThumbnailSize.Get() != EThumbnailSize::Tiny && ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable() ? EVisibility::Visible : EVisibility::Collapsed;
	}
}

EVisibility SAssetTileItem::GetNameAreaVisibility() const
{
	return CurrentThumbnailSize.Get() == EThumbnailSize::Tiny ? EVisibility::Collapsed : EVisibility::Visible;
}

FOptionalSize SAssetTileItem::GetSourceControlIconSize() const
{
	return CurrentThumbnailSize.Get() == EThumbnailSize::Small ? 11.f : 16.0f;
}

void SAssetTileItem::InitializeAssetNameHeights()
{
	// The height of the asset name field for each thumbnail size
	static bool bInitializedHeights = false;

	if (!bInitializedHeights)
	{
		AssetNameHeights[(int32)EThumbnailSize::Tiny] = 0;

		{
			const static FName SmallFontName("ContentBrowser.AssetTileViewNameFontSmall");
			FSlateFontInfo Font = FAppStyle::GetFontStyle(SmallFontName);
			TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			SmallFontHeight = FontMeasureService->GetMaxCharacterHeight(Font);

			constexpr float SmallSizeMultiplier = 2;
			AssetNameHeights[(int32)EThumbnailSize::Small] = SmallFontHeight * SmallSizeMultiplier;
		}


		{
			const static FName SmallFontName("ContentBrowser.AssetTileViewNameFont");
			FSlateFontInfo Font = FAppStyle::GetFontStyle(SmallFontName);
			TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			RegularFontHeight = FontMeasureService->GetMaxCharacterHeight(Font);

			constexpr float MediumSizeMultiplier = 3;
			constexpr float LargeSizeMultiplier = 4;
			constexpr float XLargeSizeMultiplier = 5;
			constexpr float HugeSizeMultiplier = 6;

			AssetNameHeights[(int32)EThumbnailSize::Medium] = RegularFontHeight * MediumSizeMultiplier;
			AssetNameHeights[(int32)EThumbnailSize::Large] = RegularFontHeight * LargeSizeMultiplier;

			if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
			{
				AssetNameHeights[(int32)EThumbnailSize::XLarge] = RegularFontHeight * XLargeSizeMultiplier;
				AssetNameHeights[(int32)EThumbnailSize::Huge] = RegularFontHeight * HugeSizeMultiplier;
			}
			else
			{
				AssetNameHeights[(int32)EThumbnailSize::Huge] = RegularFontHeight * XLargeSizeMultiplier;
			}
		}

		bInitializedHeights = true;
	}

}

float SAssetTileItem::AssetNameHeights[(int32)EThumbnailSize::MAX];
float SAssetTileItem::RegularFontHeight(0);
float SAssetTileItem::SmallFontHeight(0);

///////////////////////////////
// SAssetColumnItem
///////////////////////////////

template <>
struct TWidgetTypeTraits<class SAssetColumnItemNameBox>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** Custom box for the Name column of an asset */
class SAssetColumnItemNameBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SAssetColumnItemNameBox ) {}

		/** The color of the asset  */
		SLATE_ATTRIBUTE( FMargin, Padding )

		/** The widget content presented in the box */
		SLATE_DEFAULT_SLOT(FArguments, Content)

	SLATE_END_ARGS()

	~SAssetColumnItemNameBox() {}

	void Construct( const FArguments& InArgs, const TSharedRef<SAssetColumnItem>& InOwnerAssetColumnItem )
	{
		OwnerAssetColumnItem = InOwnerAssetColumnItem;

		ChildSlot
		[
			SNew(SBox)
			.Padding(InArgs._Padding)
			[
				InArgs._Content.Widget
			]
		];
	}

	virtual TSharedPtr<IToolTip> GetToolTip() override
	{
		if ( OwnerAssetColumnItem.IsValid() )
		{
			return OwnerAssetColumnItem.Pin()->GetToolTip();
		}

		return nullptr;
	}

	/** Forward the event to the view item that this name box belongs to */
	virtual void OnToolTipClosing() override
	{
		if ( OwnerAssetColumnItem.IsValid() )
		{
			OwnerAssetColumnItem.Pin()->OnToolTipClosing();
		}
	}

private:
	TWeakPtr<SAssetViewItem> OwnerAssetColumnItem;
};

void SAssetColumnItem::Construct( const FArguments& InArgs )
{
	SAssetViewItem::Construct( SAssetViewItem::FArguments()
		.AssetItem(InArgs._AssetItem)
		.OnRenameBegin(InArgs._OnRenameBegin)
		.OnRenameCommit(InArgs._OnRenameCommit)
		.OnVerifyRenameCommit(InArgs._OnVerifyRenameCommit)
		.OnItemDestroyed(InArgs._OnItemDestroyed)
		.HighlightText(InArgs._HighlightText)
		.OnIsAssetValidForCustomToolTip(InArgs._OnIsAssetValidForCustomToolTip)
		.OnGetCustomAssetToolTip(InArgs._OnGetCustomAssetToolTip)
		.OnVisualizeAssetToolTip(InArgs._OnVisualizeAssetToolTip)
		.OnAssetToolTipClosing(InArgs._OnAssetToolTipClosing)
		.ShowAssetAccessSpecifier(InArgs._ShowAssetAccessSpecifier)
		);
	
	HighlightText = InArgs._HighlightText;
}

FSlateColor SAssetColumnItem::GetColumnTextColor(FIsSelected InIsSelected) const
{
	const bool bIsSelected = InIsSelected.IsBound() ? InIsSelected.Execute() : false;
	const bool bIsHoveredOrDraggedOver = IsHovered() || bDraggedOver;
	if (bIsSelected || bIsHoveredOrDraggedOver)
	{
		return FStyleColors::White;
	}

	return FSlateColor::UseForeground();
}

TSharedRef<SWidget> SAssetColumnItem::GenerateWidgetForColumn( const FName& ColumnName, FIsSelected InIsSelected )
{
	TSharedPtr<SWidget> Content;

	// A little right padding so text from this column does not run directly into text from the next.
	static const FMargin ColumnItemPadding( 5, 0, 5, 0 );

	if ( ColumnName == "Name" )
	{
		const FSlateBrush* IconBrush;
		if(IsFolder())
		{
			if(ContentBrowserUtils::IsItemDeveloperContent(AssetItem->GetItem()))
			{
				IconBrush = UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.ColumnViewDeveloperFolderIcon");
			}
			else
			{
				IconBrush = UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.ColumnViewFolderIcon");
			}
		}
		else
		{
			if(!AssetItem->GetItem().IsSupported())
			{
				IconBrush = FAppStyle::GetBrush("Icons.WarningWithColor");
			}
			else
			{
				IconBrush = UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.ColumnViewAssetIcon");
			}
		}

		// Make icon overlays (eg, SCC and dirty status) a reasonable size in relation to the icon size (note: it is assumed this icon is square)
		const float IconOverlaySize = IconBrush->ImageSize.X * 0.6f;

		Content = SNew(SHorizontalBox)
			.AddMetaData<FTagMetaData>(FTagMetaData(AssetItem->GetItem().GetVirtualPath()))
			// Icon
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 4, 0)
			[
				SNew(SOverlay)
				
				// The actual icon
				+SOverlay::Slot()
				[
					SNew(SImage)
					.Image( IconBrush )
					.ColorAndOpacity(this, &SAssetColumnItem::GetAssetColor)
				]

				// Extra external state hook
				+ SOverlay::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Top)
				[
					SNew(SBox)
					.HeightOverride(IconOverlaySize)
					.MaxDesiredWidth(IconOverlaySize)
					[
						GenerateExtraStateIconWidget(IconOverlaySize)
					]
				]

				// Dirty state
				+SOverlay::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Bottom)
				[
					SNew(SBox)
					.WidthOverride(IconOverlaySize)
					.HeightOverride(IconOverlaySize)
					[
						SNew(SImage)
						.Image(this, &SAssetColumnItem::GetDirtyImage)
					]
				]
			]

			// Editable Name
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(InlineRenameWidget, SInlineEditableTextBlock)
				.Text( GetNameText() )
				.OnBeginTextEdit(this, &SAssetColumnItem::HandleBeginNameChange)
				.OnTextCommitted(this, &SAssetColumnItem::HandleNameCommitted)
				.OnVerifyTextChanged(this, &SAssetColumnItem::HandleVerifyNameChanged)
				.HighlightText(HighlightText)
				.IsSelected(InIsSelected)
				.IsReadOnly(this, &SAssetColumnItem::IsNameReadOnly)
				.ColorAndOpacity(this, &SAssetColumnItem::GetColumnTextColor, InIsSelected)
			];

		if(AssetItem.IsValid())
		{
			AssetItem->OnRenameRequested().BindSP(InlineRenameWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);
			AssetItem->OnRenameCanceled().BindSP(InlineRenameWidget.Get(), &SInlineEditableTextBlock::ExitEditingMode);
		}

		return SNew(SBorder)
			.BorderImage(this, &SAssetViewItem::GetBorderImage)
			.Padding(0)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew( SAssetColumnItemNameBox, SharedThis(this) )
				.Padding( ColumnItemPadding )
				[
					Content.ToSharedRef()
				]
			];
	}
	else if ( ColumnName == "Class" )
	{
		Content = SAssignNew(ClassText, STextBlock)
			.ToolTipText( this, &SAssetColumnItem::GetAssetClassText )
			.Text( GetAssetClassText() )
			.HighlightText( HighlightText );
	}
	else if ( ColumnName == "Path" )
	{
		Content = SAssignNew(PathText, STextBlock)
			.ToolTipText( this, &SAssetColumnItem::GetAssetPathText )
			.Text( GetAssetPathText() )
			.HighlightText( HighlightText );
	}
	else if ( ColumnName == "AssetAccessSpecifier")
	{
		if (bShowAssetAccessSpecifier)
		{
			Content = SAssignNew(AssetAccessSpecifierText, STextBlock)
				.ToolTipText( this, &SAssetColumnItem::GetAssetAccessSpecifierText )
				.Text( GetAssetAccessSpecifierText() )
				.HighlightText( HighlightText );
		}
	}
	else if ( ColumnName == "RevisionControl" )
	{
		Content = SNew(SBox)
			.WidthOverride(16.0f)
			.HeightOverride(16.0f)
			[
				GenerateSourceControlIconWidget()
			];
	}
	else
	{
		Content = SNew(STextBlock)
			.ToolTipText( TAttribute<FText>::Create( TAttribute<FText>::FGetter::CreateSP(this, &SAssetColumnItem::GetAssetTagText, ColumnName) ) )
			.Text( TAttribute<FText>::Create( TAttribute<FText>::FGetter::CreateSP(this, &SAssetColumnItem::GetAssetTagText, ColumnName) ) );
	}

	HandleSourceControlStateChanged();

	return SNew(SBox)
		.Padding( ColumnItemPadding )
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			Content.ToSharedRef()
		];
}


void SAssetColumnItem::OnAssetDataChanged()
{
	SAssetViewItem::OnAssetDataChanged();

	if ( ClassText.IsValid() )
	{
		ClassText->SetText( GetAssetClassText() );
	}

	if ( PathText.IsValid() )
	{
		PathText->SetText( GetAssetPathText() );
	}

	if ( AssetAccessSpecifierText.IsValid() )
	{
		AssetAccessSpecifierText->SetText( GetAssetAccessSpecifierText() );
	}
}

FText SAssetColumnItem::GetAssetPathText() const
{
	return AssetItem ? AssetItem->GetAssetPathText() : FText::GetEmpty();
}

FText SAssetColumnItem::GetAssetTagText(FName AssetTag) const
{
	if (AssetItem)
	{
		// Check custom type
		{
			FText TagText;
			if (AssetItem->GetCustomColumnDisplayValue(AssetTag, TagText))
			{
				return TagText;
			}
		}

		// Check display tags
		{
			const FTagDisplayItem* FoundTagItem = CachedDisplayTags.FindByPredicate([AssetTag](const FTagDisplayItem& TagItem)
			{
				return TagItem.TagKey == AssetTag;
			});
			
			if (FoundTagItem)
			{
				return FoundTagItem->DisplayValue;
			}
		}

	}
	
	return FText();
}

#undef LOCTEXT_NAMESPACE
