// Copyright Epic Games, Inc. All Rights Reserved.

#include "CollectionAssetRegistryBridge.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "Containers/Set.h"
#include "ContentBrowserLog.h"
#include "Delegates/Delegate.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/Platform.h"
#include "ICollectionManager.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/CoreRedirects.h"
#include "UObject/LinkerLoad.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

/** The collection manager doesn't know how to follow redirectors, this class provides it with that knowledge */
class FCollectionRedirectorFollower : public ICollectionRedirectorFollower
{
public:
	FCollectionRedirectorFollower()
		: AssetRegistry(IAssetRegistry::Get())
	{
	}

	virtual bool FixupObject(const FSoftObjectPath& InObjectPath, FSoftObjectPath& OutNewObjectPath) override
	{
		// Most of the time it will be in the asset registry so early return
		FAssetData ObjectAssetData = AssetRegistry->GetAssetByObjectPath(InObjectPath, true);

		if (ObjectAssetData.IsValid())
		{
			if (!ObjectAssetData.IsRedirector())
			{
				OutNewObjectPath = InObjectPath;
				return false;
			}
			else
			{
				OutNewObjectPath = AssetRegistry->GetRedirectedObjectPath(InObjectPath);
			}
		}
		else
		{
			FString InObjectPathString = InObjectPath.ToString();
			if (!InObjectPathString.StartsWith(TEXT("/")))
			{
				if (!UE::IsSavingPackage())
				{
					FTopLevelAssetPath FullPath = UClass::TryConvertShortTypeNameToPathName(UClass::StaticClass(), InObjectPathString);
					if (FullPath.IsValid())
					{
						FString ClassPathStr = FullPath.ToString();
						const FString NewClassName = FLinkerLoad::FindNewPathNameForClass(ClassPathStr, false);
						if (!NewClassName.IsEmpty())
						{
							check(FPackageName::IsValidObjectPath(NewClassName));
							// Our new class name might be lacking the path, so try and find it so we can use the full path in the collection
							UClass* FoundClass = FindObject<UClass>(nullptr, *NewClassName);
							if (FoundClass)
							{
								OutNewObjectPath = *FoundClass->GetPathName();
							}
						}
					}
				}
			}
			else if (InObjectPathString.StartsWith(TEXT("/Script/")))
			{
				// We can't use FindObject while we're saving
				if (!UE::IsSavingPackage())
				{
					check(FPackageName::IsValidObjectPath(InObjectPathString));
					UClass* FoundClass = FindObject<UClass>(nullptr, *InObjectPathString);
					if (!FoundClass)
					{
						// Use the linker to search for class name redirects (from the loaded ActiveClassRedirects)
						const FString NewClassName = FLinkerLoad::FindNewPathNameForClass(InObjectPathString, false);

						if (!NewClassName.IsEmpty())
						{
							check(FPackageName::IsValidObjectPath(NewClassName));
							// Our new class name might be lacking the path, so try and find it so we can use the full path in the collection
							FoundClass = FindObject<UClass>(nullptr, *NewClassName);
							if (FoundClass)
							{
								OutNewObjectPath = *FoundClass->GetPathName();
							}
						}
					}
				}
			}
			else
			{
				// Check whether there is a GeneralizedObject redirector entry in FCoreRedirects
				// This would come from the process of converting on-disk UObjectRedirectors into 
				// core redirectors
				const FCoreRedirectObjectName OldObjectName(InObjectPathString);
				
				FCoreRedirectObjectName NewObjectName = 
					FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Asset, OldObjectName);
				OutNewObjectPath.SetPath(NewObjectName.ToString());
			}
		}

		return OutNewObjectPath.IsValid() && InObjectPath != OutNewObjectPath;
	}

private:
	IAssetRegistry* AssetRegistry;
};

FCollectionAssetRegistryBridge::FCollectionAssetRegistryBridge()
{
	// Load the asset registry module to listen for updates
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	AssetRegistry.OnAssetsRemoved().AddRaw(this, &FCollectionAssetRegistryBridge::OnAssetsRemoved);
	AssetRegistry.OnAssetRenamed().AddRaw(this, &FCollectionAssetRegistryBridge::OnAssetRenamed);
	
	if (AssetRegistry.IsLoadingAssets())
	{
		AssetRegistry.OnFilesLoaded().AddRaw(this, &FCollectionAssetRegistryBridge::OnAssetRegistryLoadComplete);
	}
	else
	{
		OnAssetRegistryLoadComplete();
	}
}

FCollectionAssetRegistryBridge::~FCollectionAssetRegistryBridge()
{
	// Load the asset registry module to unregister delegates
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetRegistry->OnAssetsRemoved().RemoveAll(this);
		AssetRegistry->OnAssetRenamed().RemoveAll(this);
		AssetRegistry->OnFilesLoaded().RemoveAll(this);
	}
}

void FCollectionAssetRegistryBridge::OnAssetRegistryLoadComplete()
{
	LLM_SCOPE_BYNAME(TEXT("CollectionManager"));
	FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();

	// We've found all the assets, let the collections manager fix up its references now so that it doesn't reference any redirectors
	FCollectionRedirectorFollower RedirectorFollower;
	CollectionManagerModule.Get().HandleFixupRedirectors(RedirectorFollower);
}

void FCollectionAssetRegistryBridge::OnAssetRenamed(const FAssetData& AssetData, const FString& OldObjectPath)
{
	FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();

	// Notify the collections manager that an asset has been renamed
	CollectionManagerModule.Get().HandleObjectRenamed(FSoftObjectPath(OldObjectPath), AssetData.GetSoftObjectPath());
}

void FCollectionAssetRegistryBridge::OnAssetsRemoved(TConstArrayView<FAssetData> AssetDatas)
{
	FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();

	TArray<FSoftObjectPath> RedirectorObjectsRemoved;
	TArray<FSoftObjectPath> ObjectsRemoved;
	ObjectsRemoved.Reserve(AssetDatas.Num());

	for (const FAssetData& AssetData : AssetDatas)
	{
		if (AssetData.IsRedirector())
		{
			RedirectorObjectsRemoved.Add(AssetData.GetSoftObjectPath());
		}
		else
		{
			ObjectsRemoved.Add(AssetData.GetSoftObjectPath());
		}
	}

	// Notify the collections manager that a redirector has been removed
	// This will attempt to re-save any collections that still have a reference to this redirector in their on-disk collection data
	if (!RedirectorObjectsRemoved.IsEmpty())
	{
		CollectionManagerModule.Get().HandleRedirectorsDeleted(RedirectorObjectsRemoved);
	}

	// Notify the collections manager that an asset has been removed
	if (!ObjectsRemoved.IsEmpty())
	{
		CollectionManagerModule.Get().HandleObjectsDeleted(ObjectsRemoved);
	}
}

#undef LOCTEXT_NAMESPACE
