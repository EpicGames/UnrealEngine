// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

struct FAssetData;

/**
 * The bridge between the asset registry and the collections manager - used to update collections as certain asset events happen
 */
class FCollectionAssetRegistryBridge
{
public:
	FCollectionAssetRegistryBridge();
	~FCollectionAssetRegistryBridge();

private:
	/** Called when the asset registry initial load has completed */
	void OnAssetRegistryLoadComplete();

	/** Handler for when an asset was renamed in the asset registry */
	void OnAssetRenamed(const FAssetData& AssetData, const FString& OldObjectPath);

	/** Handler for when assets were removed from the asset registry */
	void OnAssetsRemoved(TConstArrayView<FAssetData> AssetDatas);
};
