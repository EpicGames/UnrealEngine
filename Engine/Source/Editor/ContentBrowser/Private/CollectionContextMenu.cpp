// Copyright Epic Games, Inc. All Rights Reserved.

#include "CollectionContextMenu.h"

#include "CollectionViewTypes.h"
#include "CollectionViewUtils.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "ContentBrowserUtils.h"
#include "Delegates/Delegate.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "ICollectionContainer.h"
#include "ICollectionSource.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Modules/ModuleManager.h"
#include "SlotBase.h"
#include "TelemetryRouter.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/STreeView.h"

class SWidget;

#define LOCTEXT_NAMESPACE "ContentBrowser"


FCollectionContextMenu::FCollectionContextMenu(const TWeakPtr<SCollectionView>& InCollectionView)
	: CollectionView(InCollectionView)
	, bCollectionsUnderSourceControl(false)
{
}

void FCollectionContextMenu::BindCommands(TSharedPtr< FUICommandList > InCommandList)
{
	InCommandList->MapAction( FGenericCommands::Get().Rename, FUIAction(
		FExecuteAction::CreateSP( this, &FCollectionContextMenu::ExecuteRenameCollection ),
		FCanExecuteAction::CreateSP( this, &FCollectionContextMenu::CanExecuteRenameCollection )
		));
}

TSharedPtr<SWidget> FCollectionContextMenu::MakeCollectionTreeContextMenu(TSharedPtr< FUICommandList > InCommandList)
{
	// Get all menu extenders for this context menu from the content browser module
	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>( TEXT("ContentBrowser") );
	TArray<FContentBrowserMenuExtender> MenuExtenderDelegates = ContentBrowserModule.GetAllCollectionListContextMenuExtenders();

	TArray<TSharedPtr<FExtender>> Extenders;
	for (int32 i = 0; i < MenuExtenderDelegates.Num(); ++i)
	{
		if (MenuExtenderDelegates[i].IsBound())
		{
			Extenders.Add(MenuExtenderDelegates[i].Execute());
		}
	}
	TSharedPtr<FExtender> MenuExtender = FExtender::Combine(Extenders);

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, InCommandList, MenuExtender);

	UpdateProjectSourceControl();

	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();
	TArray<TSharedPtr<FCollectionItem>> SelectedCollections = CollectionViewPtr->CollectionTreePtr->GetSelectedItems();

	bool bAnyManagedBySCC = false;
	bool bAnyNeedSCCUpdate = false;
	bool bAnyNeedSave = false;
	bool bAnyWritable = false;
	
	if (CollectionViewPtr->GetCollectionContainer().IsValid())
	{
		for (int32 CollectionIdx = 0; CollectionIdx < SelectedCollections.Num(); ++CollectionIdx)
		{
			if (CollectionViewPtr->GetCollectionContainer()->IsReadOnly(SelectedCollections[CollectionIdx]->CollectionType))
			{
				continue;
			}

			bAnyManagedBySCC |= SelectedCollections[CollectionIdx]->CollectionType != ECollectionShareType::CST_Local;
			bAnyNeedSCCUpdate |= SelectedCollections[CollectionIdx]->CurrentStatus == ECollectionItemStatus::IsOutOfDate;
			bAnyNeedSave |= SelectedCollections[CollectionIdx]->CurrentStatus == ECollectionItemStatus::HasLocalChanges;
			bAnyWritable = true;

			if (bAnyManagedBySCC && bAnyNeedSCCUpdate && bAnyNeedSave)
			{
				// Found collections to turn all options on, break now
				break;
			}
		}
	}

	MenuBuilder.BeginSection("CollectionOptions", LOCTEXT("CollectionListOptionsMenuHeading", "Collection Options"));
	{
		const bool bHasSingleSelectedCollection = SelectedCollections.Num() == 1;
		const bool bIsFirstSelectedCollectionReadOnly = SelectedCollections.Num() > 0 &&
			(!CollectionViewPtr->GetCollectionContainer().IsValid() || CollectionViewPtr->GetCollectionContainer()->IsReadOnly(SelectedCollections[0]->CollectionType));
		const bool bIsFirstSelectedCollectionStatic = SelectedCollections.Num() > 0 && SelectedCollections[0]->StorageMode == ECollectionStorageMode::Static;

		{
			TOptional<FCollectionNameType> ParentCollection;
			if (SelectedCollections.Num() > 0) 
			{
				ParentCollection = FCollectionNameType(SelectedCollections[0]->CollectionName, SelectedCollections[0]->CollectionType);
			}

			// New... (submenu)
			MenuBuilder.AddSubMenu(
				LOCTEXT("NewChildCollection", "New..."),
				LOCTEXT("NewChildCollectionTooltip", "Create a child collection."),
				FNewMenuDelegate::CreateSP( this, &FCollectionContextMenu::MakeNewCollectionSubMenu, ECollectionStorageMode::Static, SCollectionView::FCreateCollectionPayload( ParentCollection ) ),
				FUIAction(
					FExecuteAction(),
					FCanExecuteAction::CreateLambda( [=]{ return bHasSingleSelectedCollection && !bIsFirstSelectedCollectionReadOnly && bIsFirstSelectedCollectionStatic; } )
					),
				NAME_None,
				EUserInterfaceActionType::Button
				);
		}

		// Rename
		MenuBuilder.AddMenuEntry( FGenericCommands::Get().Rename, NAME_None, LOCTEXT("RenameCollection", "Rename"), LOCTEXT("RenameCollectionTooltip", "Rename this collection."));

		// Set Share Type
		MenuBuilder.AddSubMenu(
			LOCTEXT("SetCollectionShareType", "Set Share Type"),
			LOCTEXT("SetCollectionShareTypeTooltip", "Change the share type of this collection."),
			FNewMenuDelegate::CreateSP( this, &FCollectionContextMenu::MakeCollectionShareTypeSubMenu ),
			FUIAction(
				FExecuteAction(),
				FCanExecuteAction::CreateLambda( [=]{ return bHasSingleSelectedCollection && !bIsFirstSelectedCollectionReadOnly; } )
				),
			NAME_None,
			EUserInterfaceActionType::Button
			);

		// If any colors have already been set, display color options as a sub menu
		if ( CanExecuteColorChange() && CollectionViewUtils::HasCustomColors(*CollectionViewPtr->GetCollectionContainer()))
		{
			// Set Color (submenu)
			MenuBuilder.AddSubMenu(
				LOCTEXT("SetColor", "Set Color"),
				LOCTEXT("SetCollectionColorTooltip", "Sets the color this collection should appear as."),
				FNewMenuDelegate::CreateSP( this, &FCollectionContextMenu::MakeSetColorSubMenu )
				);
		}
		else
		{
			// Set Color
			MenuBuilder.AddMenuEntry(
				LOCTEXT("SetColor", "Set Color"),
				LOCTEXT("SetCollectionColorTooltip", "Sets the color this collection should appear as."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP( this, &FCollectionContextMenu::ExecutePickColor ), 
					FCanExecuteAction::CreateSP( this, &FCollectionContextMenu::CanExecuteColorChange )
					)
				);
		}
	}
	MenuBuilder.EndSection();

	if ( SelectedCollections.Num() > 0 )
	{
		MenuBuilder.BeginSection("CollectionBulkOperations", LOCTEXT("CollectionListBulkOperationsMenuHeading", "Bulk Operations"));
		{
			// Save
			MenuBuilder.AddMenuEntry(
				LOCTEXT("SaveCollection", "Save"),
				LOCTEXT("SaveCollectionTooltip", "Save this collection."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP( this, &FCollectionContextMenu::ExecuteSaveCollection ),
					FCanExecuteAction::CreateLambda( [=]{ return bAnyNeedSave; } )
					)
				);

			// Delete
			MenuBuilder.AddMenuEntry(
				LOCTEXT("DestroyCollection", "Delete"),
				LOCTEXT("DestroyCollectionTooltip", "Delete this collection."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP( this, &FCollectionContextMenu::ExecuteDestroyCollection ),
					FCanExecuteAction::CreateSP( this, &FCollectionContextMenu::CanExecuteDestroyCollection, bAnyManagedBySCC, bAnyWritable )
					)
				);

			// Update
			MenuBuilder.AddMenuEntry(
				LOCTEXT("UpdateCollection", "Update"),
				LOCTEXT("UpdateCollectionTooltip", "Update this collection to make sure it's using the latest version from revision control."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP( this, &FCollectionContextMenu::ExecuteUpdateCollection ),
					FCanExecuteAction::CreateLambda( [=]{ return bAnyNeedSCCUpdate; } )
					)
				);

			// Refresh
			MenuBuilder.AddMenuEntry(
				LOCTEXT("RefreshCollection", "Refresh"),
				LOCTEXT("RefreshCollectionTooltip", "Refresh the revision control status of this collection."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP( this, &FCollectionContextMenu::ExecuteRefreshCollection ),
					FCanExecuteAction::CreateLambda( [=]{ return bAnyManagedBySCC; } )
					)
				);
		}
		MenuBuilder.EndSection();
	}
	
	return MenuBuilder.MakeWidget();
}

void FCollectionContextMenu::MakeNewCollectionSubMenu(FMenuBuilder& MenuBuilder, ECollectionStorageMode::Type StorageMode, SCollectionView::FCreateCollectionPayload InCreationPayload)
{
	const FText MenuHeading = FText::Format(
		(InCreationPayload.ParentCollection.IsSet()) ? LOCTEXT("NewXChildCollectionMenuHeading", "New {0} Child Collection") : LOCTEXT("NewXCollectionMenuHeading", "New {0} Collection"),
		ECollectionStorageMode::ToText(StorageMode)
		);

	MenuBuilder.BeginSection("CollectionNewCollection", MenuHeading);
	{
		const bool bCanCreateSharedChildren = !InCreationPayload.ParentCollection.IsSet() || ECollectionShareType::IsValidChildType( InCreationPayload.ParentCollection->Type, ECollectionShareType::CST_Shared );
		MenuBuilder.AddMenuEntry(
			LOCTEXT("NewCollection_Shared", "Shared Collection"),
			LOCTEXT("NewCollection_SharedTooltip", "Create a collection that can be seen by anyone."),
			FSlateIcon( FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName( ECollectionShareType::CST_Shared ) ),
			FUIAction(
				FExecuteAction::CreateSP( this, &FCollectionContextMenu::ExecuteNewCollection, ECollectionShareType::CST_Shared, StorageMode, InCreationPayload ),
				FCanExecuteAction::CreateSP( this, &FCollectionContextMenu::CanExecuteNewCollection, ECollectionShareType::CST_Shared, bCanCreateSharedChildren )
				)
			);

		const bool bCanCreatePrivateChildren = !InCreationPayload.ParentCollection.IsSet() || ECollectionShareType::IsValidChildType( InCreationPayload.ParentCollection->Type, ECollectionShareType::CST_Private );
		MenuBuilder.AddMenuEntry(
			LOCTEXT("NewCollection_Private", "Private Collection"),
			LOCTEXT("NewCollection_PrivateTooltip", "Create a collection that can only be seen by you."),
			FSlateIcon( FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName( ECollectionShareType::CST_Private ) ),
			FUIAction(
				FExecuteAction::CreateSP( this, &FCollectionContextMenu::ExecuteNewCollection, ECollectionShareType::CST_Private, StorageMode, InCreationPayload ),
				FCanExecuteAction::CreateSP( this, &FCollectionContextMenu::CanExecuteNewCollection, ECollectionShareType::CST_Private, bCanCreatePrivateChildren )
				)
			);

		const bool bCanCreateLocalChildren = !InCreationPayload.ParentCollection.IsSet() || ECollectionShareType::IsValidChildType( InCreationPayload.ParentCollection->Type, ECollectionShareType::CST_Local );
		MenuBuilder.AddMenuEntry(
			LOCTEXT("NewCollection_Local", "Local Collection"),
			LOCTEXT("NewCollection_LocalTooltip", "Create a collection that is not in revision control and can only be seen by you."),
			FSlateIcon( FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName( ECollectionShareType::CST_Local ) ),
			FUIAction(
				FExecuteAction::CreateSP( this, &FCollectionContextMenu::ExecuteNewCollection, ECollectionShareType::CST_Local, StorageMode, InCreationPayload ),
				FCanExecuteAction::CreateSP( this, &FCollectionContextMenu::CanExecuteNewCollection, ECollectionShareType::CST_Local, bCanCreateLocalChildren )
				)
			);
	}
	MenuBuilder.EndSection();
}

void FCollectionContextMenu::MakeSaveDynamicCollectionSubMenu(FMenuBuilder& MenuBuilder, FText InSearchQuery)
{
	auto OnCollectionCreated = FCollectionItem::FCollectionCreatedEvent::CreateSP(this, &FCollectionContextMenu::ExecuteSaveDynamicCollection, InSearchQuery);

	// Create new root level collection
	MakeNewCollectionSubMenu(MenuBuilder, ECollectionStorageMode::Dynamic, SCollectionView::FCreateCollectionPayload(OnCollectionCreated));

	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();

	if (!CollectionViewPtr->GetCollectionContainer())
	{
		return;
	}
	
	TArray<FCollectionNameType> AvailableCollections;
	CollectionViewPtr->GetCollectionContainer()->GetCollections(AvailableCollections);

	AvailableCollections.Sort([](const FCollectionNameType& One, const FCollectionNameType& Two) -> bool
	{
		return One.Name.LexicalLess(Two.Name);
	});

	if (AvailableCollections.Num() > 0)
	{
		MenuBuilder.BeginSection("CollectionReplaceCollection", LOCTEXT("OverwriteDynamicCollectionMenuHeading", "Overwrite Dynamic Collection"));

		for (const FCollectionNameType& AvailableCollection : AvailableCollections)
		{
			// Never display system collections
			if (AvailableCollection.Type == ECollectionShareType::CST_System)
			{
				continue;
			}

			// Can only overwrite dynamic collections
			ECollectionStorageMode::Type StorageMode = ECollectionStorageMode::Static;
			CollectionViewPtr->GetCollectionContainer()->GetCollectionStorageMode(AvailableCollection.Name, AvailableCollection.Type, StorageMode);
			if (StorageMode != ECollectionStorageMode::Dynamic)
			{
				continue;
			}

			MenuBuilder.AddMenuEntry(
				FText::FromName(AvailableCollection.Name), 
				FText::Format(LOCTEXT("SaveDynamicCollection_OverwriteExistingCollectionToolTip", "Overwrite '{0}' with the current search query"), FText::FromName(AvailableCollection.Name)),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(AvailableCollection.Type)),
				FUIAction(
					FExecuteAction::CreateSP( this, &FCollectionContextMenu::ExecuteSaveDynamicCollection, AvailableCollection, InSearchQuery ),
					FCanExecuteAction::CreateSP( this, &FCollectionContextMenu::CanExecuteSaveDynamicCollection, AvailableCollection )
					)
				);
		}

		MenuBuilder.EndSection();
	}
}

void FCollectionContextMenu::MakeCollectionShareTypeSubMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("CollectionShareType", LOCTEXT("CollectionShareTypeMenuHeading", "Collection Share Type"));
	{
		MenuBuilder.AddMenuEntry(
			ECollectionShareType::ToText( ECollectionShareType::CST_Shared ),
			ECollectionShareType::GetDescription( ECollectionShareType::CST_Shared ),
			FSlateIcon( FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName( ECollectionShareType::CST_Shared ) ),
			FUIAction(
				FExecuteAction::CreateSP( this, &FCollectionContextMenu::ExecuteSetCollectionShareType, ECollectionShareType::CST_Shared ),
				FCanExecuteAction::CreateSP( this, &FCollectionContextMenu::CanExecuteSetCollectionShareType, ECollectionShareType::CST_Shared ),
				FIsActionChecked::CreateSP( this, &FCollectionContextMenu::IsSetCollectionShareTypeChecked, ECollectionShareType::CST_Shared )
				),
			NAME_None,
			EUserInterfaceActionType::Check
			);

		MenuBuilder.AddMenuEntry(
			ECollectionShareType::ToText( ECollectionShareType::CST_Private ),
			ECollectionShareType::GetDescription( ECollectionShareType::CST_Private ),
			FSlateIcon( FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName( ECollectionShareType::CST_Private ) ),
			FUIAction(
				FExecuteAction::CreateSP( this, &FCollectionContextMenu::ExecuteSetCollectionShareType, ECollectionShareType::CST_Private ),
				FCanExecuteAction::CreateSP( this, &FCollectionContextMenu::CanExecuteSetCollectionShareType, ECollectionShareType::CST_Private ),
				FIsActionChecked::CreateSP( this, &FCollectionContextMenu::IsSetCollectionShareTypeChecked, ECollectionShareType::CST_Private )
				),
			NAME_None,
			EUserInterfaceActionType::Check
			);

		MenuBuilder.AddMenuEntry(
			ECollectionShareType::ToText( ECollectionShareType::CST_Local ),
			ECollectionShareType::GetDescription( ECollectionShareType::CST_Local ),
			FSlateIcon( FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName( ECollectionShareType::CST_Local ) ),
			FUIAction(
				FExecuteAction::CreateSP( this, &FCollectionContextMenu::ExecuteSetCollectionShareType, ECollectionShareType::CST_Local ),
				FCanExecuteAction::CreateSP( this, &FCollectionContextMenu::CanExecuteSetCollectionShareType, ECollectionShareType::CST_Local ),
				FIsActionChecked::CreateSP( this, &FCollectionContextMenu::IsSetCollectionShareTypeChecked, ECollectionShareType::CST_Local )
				),
			NAME_None,
			EUserInterfaceActionType::Check
			);
	}
	MenuBuilder.EndSection();
}

void FCollectionContextMenu::MakeSetColorSubMenu(FMenuBuilder& MenuBuilder)
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();

	// New Color
	MenuBuilder.AddMenuEntry(
		LOCTEXT("NewColor", "New Color"),
		LOCTEXT("NewCollectionColorTooltip", "Changes the color this collection should appear as."),
		FSlateIcon(),
		FUIAction( FExecuteAction::CreateSP( this, &FCollectionContextMenu::ExecutePickColor ) )
		);

	// Clear Color (only required if any of the selection has one)
	if ( SelectedHasCustomColors() )
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ClearColor", "Clear Color"),
			LOCTEXT("ClearCollectionColorTooltip", "Resets the color this collection appears as."),
			FSlateIcon(),
			FUIAction( FExecuteAction::CreateSP( this, &FCollectionContextMenu::ExecuteResetColor ) )
			);
	}

	// Add all the custom colors the user has chosen so far
	TArray< FLinearColor > CustomColors;
	if ( CollectionViewUtils::HasCustomColors( *CollectionViewPtr->GetCollectionContainer(), &CustomColors ) )
	{	
		MenuBuilder.BeginSection("PathContextCustomColors", LOCTEXT("CustomColorsExistingColors", "Existing Colors") );
		{
			for ( int32 ColorIndex = 0; ColorIndex < CustomColors.Num(); ColorIndex++ )
			{
				const FLinearColor& Color = CustomColors[ ColorIndex ];
				MenuBuilder.AddWidget(
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2, 0, 0, 0)
						[
							SNew(SButton)
							.ButtonStyle( FAppStyle::Get(), "Menu.Button" )
							.OnClicked( this, &FCollectionContextMenu::OnColorClicked, Color )
							[
								SNew(SColorBlock)
								.Color( Color )
								.Size( FVector2D(77,16) )
							]
						],
					FText::GetEmpty(),
					/*bNoIndent=*/true
				);
			}
		}
		MenuBuilder.EndSection();
	}
}

void FCollectionContextMenu::UpdateProjectSourceControl()
{
	// Force update of source control so that we're always showing the valid options
	bCollectionsUnderSourceControl = false;
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();
	if (CollectionViewPtr &&
		CollectionViewPtr->GetCollectionContainer() &&
		ISourceControlModule::Get().IsEnabled() &&
		ISourceControlModule::Get().GetProvider().IsAvailable())
	{
		FString HintFilename = CollectionViewPtr->GetCollectionContainer()->GetCollectionSource()->GetSourceControlStatusHintFilename();
		if (!HintFilename.IsEmpty())
		{
			FSourceControlStatePtr SourceControlState = ISourceControlModule::Get().GetProvider().GetState(HintFilename, EStateCacheUsage::ForceUpdate);
			bCollectionsUnderSourceControl = (SourceControlState->IsSourceControlled() && !SourceControlState->IsIgnored() && !SourceControlState->IsUnknown());
		}
	}
}

bool FCollectionContextMenu::CanRenameSelectedCollections() const
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();
	if (!CollectionViewPtr.IsValid())
	{
		return false;
	}

	if (!CollectionViewPtr->GetCollectionContainer().IsValid())
	{
		return false;
	}

	TArray<TSharedPtr<FCollectionItem>> SelectedCollections = CollectionViewPtr->CollectionTreePtr->GetSelectedItems();
	
	if(SelectedCollections.Num() == 1)
	{
		if (CollectionViewPtr->GetCollectionContainer()->IsReadOnly(SelectedCollections[0]->CollectionType))
		{
			return false;
		}

		return !(SelectedCollections[0]->CollectionType != ECollectionShareType::CST_Local) || (bCollectionsUnderSourceControl && ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable());
	}
	
	return false;
}

void FCollectionContextMenu::ExecuteNewCollection(ECollectionShareType::Type CollectionType, ECollectionStorageMode::Type StorageMode, SCollectionView::FCreateCollectionPayload InCreationPayload)
{
	if ( !ensure(CollectionView.IsValid()) )
	{
		return;
	}

	const double BeginTimeSec = FPlatformTime::Seconds();
	
	CollectionView.Pin()->CreateCollectionItem(CollectionType, StorageMode, InCreationPayload);

	// Telemetry Event
	{
		FCollectionCreatedTelemetryEvent AssetAdded;
		AssetAdded.DurationSec = FPlatformTime::Seconds() - BeginTimeSec;
		AssetAdded.CollectionShareType = CollectionType;
		FTelemetryRouter::Get().ProvideTelemetry(AssetAdded);
	}
}

void FCollectionContextMenu::ExecuteSetCollectionShareType(ECollectionShareType::Type CollectionType)
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();

	if ( !ensure(CollectionViewPtr.IsValid()) )
	{
		return;
	}

	TArray<TSharedPtr<FCollectionItem>> SelectedCollections = CollectionViewPtr->CollectionTreePtr->GetSelectedItems();

	if ( !ensure(SelectedCollections.Num() == 1) )
	{
		return;
	}

	TSharedPtr<ICollectionContainer> CollectionContainer = CollectionViewPtr->GetCollectionContainer();

	CollectionContainer->RenameCollection(SelectedCollections[0]->CollectionName, SelectedCollections[0]->CollectionType, SelectedCollections[0]->CollectionName, CollectionType);
}

void FCollectionContextMenu::ExecuteSaveDynamicCollection(FCollectionNameType InCollection, FText InSearchQuery)
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();

	if (!ensure(CollectionViewPtr.IsValid()))
	{
		return;
	}

	CollectionViewPtr->GetCollectionContainer()->SetDynamicQueryText(InCollection.Name, InCollection.Type, InSearchQuery.ToString());
}

void FCollectionContextMenu::ExecuteRenameCollection()
{
	if ( !ensure(CollectionView.IsValid()) )
	{
		return;
	}

	TArray<TSharedPtr<FCollectionItem>> SelectedCollections = CollectionView.Pin()->CollectionTreePtr->GetSelectedItems();

	if ( !ensure(SelectedCollections.Num() == 1) )
	{
		return;
	}

	CollectionView.Pin()->RenameCollectionItem(SelectedCollections[0]);
}

void FCollectionContextMenu::ExecuteUpdateCollection()
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();

	if ( !ensure(CollectionViewPtr.IsValid()) )
	{
		return;
	}

	TSharedPtr<ICollectionContainer> CollectionContainer = CollectionViewPtr->GetCollectionContainer();
	TArray<TSharedPtr<FCollectionItem>> SelectedCollections = CollectionViewPtr->CollectionTreePtr->GetSelectedItems();

	for (const auto& SelectedCollection : SelectedCollections)
	{
		if (!CollectionContainer->IsReadOnly(SelectedCollection->CollectionType))
		{
			CollectionContainer->UpdateCollection(SelectedCollection->CollectionName, SelectedCollection->CollectionType);
		}
	}
}

void FCollectionContextMenu::ExecuteRefreshCollection()
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();

	if ( !ensure(CollectionViewPtr.IsValid()) )
	{
		return;
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	TSharedPtr<ICollectionContainer> CollectionContainer = CollectionViewPtr->GetCollectionContainer();
	TArray<TSharedPtr<FCollectionItem>> SelectedCollections = CollectionViewPtr->CollectionTreePtr->GetSelectedItems();

	TArray<FString> CollectionFilesToRefresh;
	for (const auto& SelectedCollection : SelectedCollections)
	{
		FCollectionStatusInfo StatusInfo;
		if (CollectionContainer->GetCollectionStatusInfo(SelectedCollection->CollectionName, SelectedCollection->CollectionType, StatusInfo))
		{
			if (StatusInfo.bUseSCC && StatusInfo.SCCState.IsValid() && StatusInfo.SCCState->IsSourceControlled())
			{
				// Forcing a status update will refresh the collection state
				SourceControlProvider.GetState(StatusInfo.SCCState->GetFilename(), EStateCacheUsage::ForceUpdate);
			}
		}
	}
}

void FCollectionContextMenu::ExecuteSaveCollection()
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();

	if ( !ensure(CollectionViewPtr.IsValid()) )
	{
		return;
	}

	TSharedPtr<ICollectionContainer> CollectionContainer = CollectionViewPtr->GetCollectionContainer();
	TArray<TSharedPtr<FCollectionItem>> SelectedCollections = CollectionViewPtr->CollectionTreePtr->GetSelectedItems();

	for (const auto& SelectedCollection : SelectedCollections)
	{
		if (!CollectionContainer->IsReadOnly(SelectedCollection->CollectionType))
		{
			CollectionContainer->SaveCollection(SelectedCollection->CollectionName, SelectedCollection->CollectionType);
		}
	}
}

void FCollectionContextMenu::ExecuteDestroyCollection()
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();

	if ( !ensure(CollectionViewPtr.IsValid()) )
	{
		return;
	}

	TSharedPtr<ICollectionContainer> CollectionContainer = CollectionViewPtr->GetCollectionContainer();
	TArray<TSharedPtr<FCollectionItem>> SelectedCollections = CollectionViewPtr->CollectionTreePtr->GetSelectedItems();

	SelectedCollections.RemoveAll([&CollectionContainer](const TSharedPtr<FCollectionItem>& CollectionItem)
		{
			return CollectionContainer->IsReadOnly(CollectionItem->CollectionType);
		});

	FText Prompt;
	if ( SelectedCollections.Num() == 1 )
	{
		Prompt = FText::Format(LOCTEXT("CollectionDestroyConfirm_Single", "Delete {0}?"), FText::FromName(SelectedCollections[0]->CollectionName));
	}
	else
	{
		Prompt = FText::Format(LOCTEXT("CollectionDestroyConfirm_Multiple", "Delete {0} Collections?"), FText::AsNumber(SelectedCollections.Num()));
	}

	if (ContentBrowserUtils::DisplayConfirmationDialog(LOCTEXT("CollectionDestroyConfirm_Yes", "Delete"), Prompt))
	{
		ExecuteDestroyCollectionConfirmed(SelectedCollections);
	}
}

FReply FCollectionContextMenu::ExecuteDestroyCollectionConfirmed(TArray<TSharedPtr<FCollectionItem>> CollectionList)
{
	const double BeginEventSec = FPlatformTime::Seconds();
	
	CollectionView.Pin()->DeleteCollectionItems(CollectionList);

	{
		FCollectionsDeletedTelemetryEvent CollectionDeleted;
		CollectionDeleted.DurationSec = FPlatformTime::Seconds() - BeginEventSec;
		CollectionDeleted.CollectionsDeleted = CollectionList.Num();
		FTelemetryRouter::Get().ProvideTelemetry(CollectionDeleted);
	}
	
	return FReply::Handled();
}

void FCollectionContextMenu::ExecuteResetColor()
{
	ResetColors();
}

void FCollectionContextMenu::ExecutePickColor()
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();
	TArray<FCollectionNameType> SelectedCollections = CollectionViewPtr->GetSelectedCollections();

	FLinearColor InitialColor = FLinearColor::White;
	for (const FCollectionNameType& SelectedCollection : SelectedCollections)
	{
		if (!CollectionViewPtr->GetCollectionContainer()->IsReadOnly(SelectedCollection.Type))
		{
			InitialColor = CollectionViewUtils::ResolveColor(*CollectionViewPtr->GetCollectionContainer(), SelectedCollection.Name, SelectedCollection.Type);
		}
	}

	FColorPickerArgs PickerArgs;
	PickerArgs.bIsModal = true; // TODO: Allow live color updates via a proxy?
	PickerArgs.ParentWidget = CollectionViewPtr;
	PickerArgs.InitialColor = InitialColor;
	PickerArgs.OnColorCommitted.BindSP(this, &FCollectionContextMenu::OnColorCommitted);

	OpenColorPicker(PickerArgs);
}

bool FCollectionContextMenu::CanExecuteNewCollection(ECollectionShareType::Type CollectionType, bool bIsValidChildType) const
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();
	if (!CollectionViewPtr.IsValid())
	{
		return false;
	}

	if (!CollectionViewPtr->GetCollectionContainer().IsValid() || CollectionViewPtr->GetCollectionContainer()->IsReadOnly(CollectionType))
	{
		return false;
	}

	return bIsValidChildType && (CollectionType == ECollectionShareType::CST_Local || (bCollectionsUnderSourceControl && ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable()));
}

bool FCollectionContextMenu::CanExecuteSetCollectionShareType(ECollectionShareType::Type CollectionType) const
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();
	if ( !ensure(CollectionViewPtr.IsValid()) )
	{
		return false;
	}

	if (!CollectionViewPtr->GetCollectionContainer().IsValid() || CollectionViewPtr->GetCollectionContainer()->IsReadOnly(CollectionType))
	{
		return false;
	}

	TArray<TSharedPtr<FCollectionItem>> SelectedCollections = CollectionViewPtr->CollectionTreePtr->GetSelectedItems();

	if ( !ensure(SelectedCollections.Num() == 1) )
	{
		return false;
	}

	if (CollectionViewPtr->GetCollectionContainer()->IsReadOnly(SelectedCollections[0]->CollectionType))
	{
		return false;
	}

	const bool bIsSourceControlAvailable = bCollectionsUnderSourceControl && ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable();
	const bool bIsCurrentTypeLocal = SelectedCollections[0]->CollectionType == ECollectionShareType::CST_Local;
	const bool bIsNewTypeLocal = CollectionType == ECollectionShareType::CST_Local;
	const bool bIsNewShareTypeDifferent = SelectedCollections[0]->CollectionType != CollectionType;

	return bIsNewShareTypeDifferent && ((bIsCurrentTypeLocal && bIsNewTypeLocal) || bIsSourceControlAvailable);
}

bool FCollectionContextMenu::IsSetCollectionShareTypeChecked(ECollectionShareType::Type CollectionType) const
{
	if ( !ensure(CollectionView.IsValid()) )
	{
		return false;
	}

	TArray<TSharedPtr<FCollectionItem>> SelectedCollections = CollectionView.Pin()->CollectionTreePtr->GetSelectedItems();

	if ( !ensure(SelectedCollections.Num() == 1) )
	{
		return false;
	}

	return SelectedCollections[0]->CollectionType == CollectionType;
}

bool FCollectionContextMenu::CanExecuteSaveDynamicCollection(FCollectionNameType InCollection) const
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();
	if (!CollectionViewPtr.IsValid())
	{
		return false;
	}

	if (!CollectionViewPtr->GetCollectionContainer().IsValid() || CollectionViewPtr->GetCollectionContainer()->IsReadOnly(InCollection.Type))
	{
		return false;
	}

	return InCollection.Type == ECollectionShareType::CST_Local || (bCollectionsUnderSourceControl && ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable());
}

bool FCollectionContextMenu::CanExecuteRenameCollection() const
{
	return CanRenameSelectedCollections();
}

bool FCollectionContextMenu::CanExecuteDestroyCollection(bool bAnyManagedBySCC, bool bAnyWritable) const
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();
	if (!CollectionViewPtr.IsValid())
	{
		return false;
	}

	if (!CollectionViewPtr->GetCollectionContainer().IsValid() || !bAnyWritable)
	{
		return false;
	}

	return !bAnyManagedBySCC || (bCollectionsUnderSourceControl && ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable());
}

bool FCollectionContextMenu::SelectedHasCustomColors() const
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();
	TArray<FCollectionNameType> SelectedCollections = CollectionViewPtr->GetSelectedCollections();

	for(const FCollectionNameType& SelectedCollection : SelectedCollections)
	{
		// Ignore any that are the default color
		const TOptional<FLinearColor> Color = CollectionViewUtils::GetCustomColor(CollectionViewPtr->GetCollectionContainer().Get(), SelectedCollection.Name, SelectedCollection.Type);
		if (Color)
		{
			return true;
		}
	}
	return false;
}

bool FCollectionContextMenu::CanExecuteColorChange() const
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();
	if (!CollectionViewPtr.IsValid())
	{
		return false;
	}

	if (!CollectionViewPtr->GetCollectionContainer().IsValid())
	{
		return false;
	}

	TArray<TSharedPtr<FCollectionItem>> SelectedCollections = CollectionViewPtr->CollectionTreePtr->GetSelectedItems();
	const bool bIsSourceControlValid = bCollectionsUnderSourceControl && ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable();

	for (TSharedPtr<FCollectionItem> SelectedCollection : SelectedCollections)
	{
		if (CollectionViewPtr->GetCollectionContainer()->IsReadOnly(SelectedCollection->CollectionType))
		{
			continue;
		}

		if (SelectedCollection->CollectionType != ECollectionShareType::CST_Local && !bIsSourceControlValid)
		{
			continue;
		}

		return true;
	}

	return false;
}

FReply FCollectionContextMenu::OnColorClicked( const FLinearColor InColor )
{
	OnColorCommitted(InColor);

	// Dismiss the menu here, as we can't make the 'clear' option appear if a folder has just had a color set for the first time
	FSlateApplication::Get().DismissAllMenus();

	return FReply::Handled();
}

void FCollectionContextMenu::OnColorCommitted( const FLinearColor InColor )
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();
	TArray<FCollectionNameType> SelectedCollections = CollectionViewPtr->GetSelectedCollections();

	// Make sure an color entry exists for all the collections, otherwise it can't save correctly
	for (const FCollectionNameType& SelectedCollection : SelectedCollections)
	{
		if (!CollectionViewPtr->GetCollectionContainer()->IsReadOnly(SelectedCollection.Type))
		{
			CollectionViewUtils::SetCustomColor(*CollectionViewPtr->GetCollectionContainer(), SelectedCollection.Name, SelectedCollection.Type, InColor);
		}
	}
}

void FCollectionContextMenu::ResetColors()
{
	TSharedPtr<SCollectionView> CollectionViewPtr = CollectionView.Pin();
	TArray<FCollectionNameType> SelectedCollections = CollectionViewPtr->GetSelectedCollections();

	// Clear the custom colors for all the selected collections
	for(const FCollectionNameType& SelectedCollection : SelectedCollections)
	{
		if (!CollectionViewPtr->GetCollectionContainer()->IsReadOnly(SelectedCollection.Type))
		{
			CollectionViewUtils::SetCustomColor(*CollectionViewPtr->GetCollectionContainer(), SelectedCollection.Name, SelectedCollection.Type, TOptional<FLinearColor>());
		}
	}
}

#undef LOCTEXT_NAMESPACE
