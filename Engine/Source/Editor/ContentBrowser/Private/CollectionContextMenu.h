// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CollectionManagerTypes.h"
#include "Containers/Array.h"
#include "Input/Reply.h"
#include "Math/Color.h"
#include "SCollectionView.h"
#include "Templates/SharedPointer.h"

class FMenuBuilder;
class FText;
class FUICommandList;
class SWidget;
struct FCollectionItem;

class FCollectionContextMenu : public TSharedFromThis<FCollectionContextMenu>
{
public:
	/** Constructor */
	FCollectionContextMenu(const TWeakPtr<SCollectionView>& InCollectionView);

	/** Bind menu selection commands to the command list */
	void BindCommands(TSharedPtr< FUICommandList > InCommandList);

	/** Makes the collection tree context menu widget */
	TSharedPtr<SWidget> MakeCollectionTreeContextMenu(TSharedPtr< FUICommandList > InCommandList);

	/** Makes the new collection submenu */
	void MakeNewCollectionSubMenu(FMenuBuilder& MenuBuilder, ECollectionStorageMode::Type StorageMode, SCollectionView::FCreateCollectionPayload InCreationPayload);

	/** Makes the save dynamic collection submenu */
	void MakeSaveDynamicCollectionSubMenu(FMenuBuilder& MenuBuilder, FText InSearchQuery);

	/** Makes the collection share type submenu */
	void MakeCollectionShareTypeSubMenu(FMenuBuilder& MenuBuilder);

	/** Update the source control flag the 'CanExecute' functions rely on */
	void UpdateProjectSourceControl();

	/** Can the selected collections be renamed? */
	bool CanRenameSelectedCollections() const;

protected:
	/** Makes the set color submenu */
	void MakeSetColorSubMenu(FMenuBuilder& MenuBuilder);

private:
	/** Handler for when a collection is selected in the "New" menu */
	void ExecuteNewCollection(ECollectionShareType::Type CollectionType, ECollectionStorageMode::Type StorageMode, SCollectionView::FCreateCollectionPayload InCreationPayload);

	/** Handler for when a collection share type is changed in the "Share Type" menu */
	void ExecuteSetCollectionShareType(ECollectionShareType::Type CollectionType);

	/** Handler for when a dynamic collection is selected in the "Save" menu */
	void ExecuteSaveDynamicCollection(FCollectionNameType InCollection, FText InSearchQuery);

	/** Handler for when "Rename Collection" is selected */
	void ExecuteRenameCollection();

	/** Handler for when "Update Collection" is selected */
	void ExecuteUpdateCollection();

	/** Handler for when "Refresh Collection" is selected */
	void ExecuteRefreshCollection();

	/** Handler for when "Save Collection" is selected */
	void ExecuteSaveCollection();

	/** Handler for when "Destroy Collection" is selected */
	void ExecuteDestroyCollection();

	/** Handler for when "Destroy Collection" is confirmed */
	FReply ExecuteDestroyCollectionConfirmed(TArray<TSharedPtr<FCollectionItem>> CollectionList);

	/** Handler for when reset color is selected */
	void ExecuteResetColor();

	/** Handler for when new or set color is selected */
	void ExecutePickColor();

	/** Handler to check to see if "New Collection" can be executed */
	bool CanExecuteNewCollection(ECollectionShareType::Type CollectionType, bool bIsValidChildType) const;

	/** Handler to check to see if a entry in the "Share Type" menu can be executed */
	bool CanExecuteSetCollectionShareType(ECollectionShareType::Type CollectionType) const;

	/** Handler to check to see if an entry in the "Share Type" menu should be checked */
	bool IsSetCollectionShareTypeChecked(ECollectionShareType::Type CollectionType) const;

	/** Handler to check to see if "Save Dynamic Collection" can be executed */
	bool CanExecuteSaveDynamicCollection(FCollectionNameType InCollection) const;

	/** Handler to check to see if "Rename Collection" can be executed */
	bool CanExecuteRenameCollection() const;

	/** Handler to check to see if "Destroy Collection" can be executed */
	bool CanExecuteDestroyCollection(bool bAnyManagedBySCC, bool bAnyWritable) const;

	/** Checks to see if any of the selected collections use custom colors */
	bool SelectedHasCustomColors() const;

	/** Handler to check to see if "Set/Clear Color" can be executed */
	bool CanExecuteColorChange() const;

	/** Callback when the color is picked from the set color submenu */
	FReply OnColorClicked( const FLinearColor InColor );

	/** Callback when the color is picked from the set color submenu or picker */
	void OnColorCommitted( const FLinearColor InColor );

	/** Resets the colors of the selected collections */
	void ResetColors();

	TWeakPtr<SCollectionView> CollectionView;

	/** Flag caching whether the shared collections folder is under source control */
	bool bCollectionsUnderSourceControl;
};
