// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CollectionManagerTypes.h"

class ICollectionContainer;

/** Status states that a collection item can be in */
enum class ECollectionItemStatus : uint8
{
	/** The collection is up-to-date in source control, and isn't empty */
	IsUpToDateAndPopulated,
		
	/** The collection is up-to-date in source control, but is empty */
	IsUpToDateAndEmpty,

	/** The collection is out-of-date in source control */
	IsOutOfDate,

	/** The collection is checked out by another source control user, so can't be modified at this time */
	IsCheckedOutByAnotherUser,

	/** The collection is conflicted in source control, so can't be modified at this time */
	IsConflicted,

	/** The collection is under source control but the SCC provider is currently unavailable, so can't be modified at this time */
	IsMissingSCCProvider,

	/** The collection has local changes that either haven't been saved, or haven't been committed to source control */
	HasLocalChanges,
};

/** A list item representing a collection */
struct FCollectionItem
{
	struct FCompareFCollectionItemByName
	{
		FORCEINLINE bool operator()( const TSharedPtr<FCollectionItem>& A, const TSharedPtr<FCollectionItem>& B ) const
		{
			return A->CollectionName.LexicalLess(B->CollectionName);
		}
	};

	/** The container of this item */
	TSharedPtr<ICollectionContainer> CollectionContainer;

	/** The name of the collection */
	FName CollectionName;

	/** The type of the collection */
	ECollectionShareType::Type CollectionType;

	/** How does this collection store its objects? (static or dynamic) */
	ECollectionStorageMode::Type StorageMode;

	/** Pointer to our parent collection (if any) */
	TWeakPtr<FCollectionItem> ParentCollection;

	/** Array of pointers to our child collections (if any) */
	TArray<TWeakPtr<FCollectionItem>> ChildCollections;

	/** If true, will set up an inline rename after next ScrollIntoView */
	bool bRenaming;

	/** If true, this item will be created the next time it is renamed */
	bool bNewCollection;

	/** The number of objects in this collection (static collections only) */
	int32 NumObjects;

	/** Current status of this collection item */
	ECollectionItemStatus CurrentStatus;

	/** The color of the collection */
	FLinearColor CollectionColor;

	/** Called once after the collection is created (see bNewCollection) */
	DECLARE_DELEGATE_OneParam( FCollectionCreatedEvent, FCollectionNameType )

	/** Broadcasts whenever renaming a collection item is requested */
	DECLARE_MULTICAST_DELEGATE( FRenamedRequestEvent )

	/** Broadcasts once after the collection is created (see bNewCollection) */
	FCollectionCreatedEvent OnCollectionCreatedEvent;

	/** Broadcasts whenever a rename is requested */
	FRenamedRequestEvent OnRenamedRequestEvent;

	/** Constructor */
	FCollectionItem(const TSharedRef<ICollectionContainer>& InCollectionContainer, const FName& InCollectionName, const ECollectionShareType::Type InCollectionType)
		: CollectionContainer(InCollectionContainer)
		, CollectionName(InCollectionName)
		, CollectionType(InCollectionType)
		, StorageMode(ECollectionStorageMode::Static)
		, ParentCollection()
		, ChildCollections()
		, bRenaming(false)
		, bNewCollection(false)
		, NumObjects(0)
		, CurrentStatus(ECollectionItemStatus::IsUpToDateAndEmpty)
		, CollectionColor(FLinearColor::White)
	{}
};
