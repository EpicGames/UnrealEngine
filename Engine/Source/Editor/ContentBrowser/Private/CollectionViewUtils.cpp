// Copyright Epic Games, Inc. All Rights Reserved.


#include "CollectionViewUtils.h"

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "ICollectionContainer.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"

#define LOCTEXT_NAMESPACE "CollectionView"

namespace CollectionViewUtils
{

// Keep a map of all the collections that have custom colors, so updating the color in one location updates them all
static TMap<FString, TOptional<FLinearColor>> LegacyCollectionColors;

FLinearColor GetLegacyDefaultColor()
{
	// The default tint the folder should appear as
	return FLinearColor::Gray;
}

/** Create a string of the form "CollectionName:CollectionType */
FString ToConfigKey(const FString& InCollectionName, const ECollectionShareType::Type& InCollectionType)
{
	static_assert(ECollectionShareType::CST_All == 4, "Update CollectionViewUtils::ToConfigKey for the updated ECollectionShareType values");

	check(InCollectionType != ECollectionShareType::CST_All);

	FString CollectionTypeStr;
	switch(InCollectionType)
	{
	case ECollectionShareType::CST_System:
		CollectionTypeStr = "System";
		break;
	case ECollectionShareType::CST_Local:
		CollectionTypeStr = "Local";
		break;
	case ECollectionShareType::CST_Private:
		CollectionTypeStr = "Private";
		break;
	case ECollectionShareType::CST_Shared:
		CollectionTypeStr = "Shared";
		break;
	default:
		break;
	}

	return InCollectionName + ":" + CollectionTypeStr;
}

/** Convert a string of the form "CollectionName:CollectionType back into its individual elements */
bool FromConfigKey(const FString& InKey, FString& OutCollectionName, ECollectionShareType::Type& OutCollectionType)
{
	static_assert(ECollectionShareType::CST_All == 4, "Update CollectionViewUtils::FromConfigKey for the updated ECollectionShareType values");

	FString CollectionTypeStr;
	if(InKey.Split(":", &OutCollectionName, &CollectionTypeStr, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
	{
		if(CollectionTypeStr == "System")
		{
			OutCollectionType = ECollectionShareType::CST_System;
		}
		else if(CollectionTypeStr == "Local")
		{
			OutCollectionType = ECollectionShareType::CST_Local;
		}
		else if(CollectionTypeStr == "Private")
		{
			OutCollectionType = ECollectionShareType::CST_Private;
		}
		else if(CollectionTypeStr == "Shared")
		{
			OutCollectionType = ECollectionShareType::CST_Shared;
		}
		else
		{
			return false;
		}

		return !OutCollectionName.IsEmpty();
	}

	return false;
}

const TOptional<FLinearColor> LoadLegacyColor(const FName InCollectionName, const ECollectionShareType::Type& InCollectionType)
{
	check(InCollectionType != ECollectionShareType::CST_All);

	const FString ColorKeyStr = ToConfigKey(InCollectionName.ToString(), InCollectionType);

	// See if we have a value cached first
	{
		TOptional<FLinearColor>* const CachedColor = LegacyCollectionColors.Find(ColorKeyStr);
		if(CachedColor)
		{
			return *CachedColor;
		}
	}
		
	// Loads the color of collection at the given path from the config
	if(FPaths::FileExists(GEditorPerProjectIni))
	{
		// Create a new entry from the config, skip if it's default
		FString ColorStr;
		if(GConfig->GetString(TEXT("CollectionColor"), *ColorKeyStr, ColorStr, GEditorPerProjectIni))
		{
			FLinearColor Color;
			if(Color.InitFromString(ColorStr) && !Color.Equals(CollectionViewUtils::GetLegacyDefaultColor()))
			{
				return LegacyCollectionColors.Add(ColorKeyStr, Color);
			}
		}
	}

	// Cache an empty color to avoid hitting the disk again
	return LegacyCollectionColors.Add(ColorKeyStr, TOptional<FLinearColor>());
}

void ClearLegacyColor(const FName InCollectionName, const ECollectionShareType::Type& InCollectionType)
{
	check(InCollectionType != ECollectionShareType::CST_All);

	const FString ColorKeyStr = ToConfigKey(InCollectionName.ToString(), InCollectionType);

	// Saves the color of the collection to the config
	if(FPaths::FileExists(GEditorPerProjectIni))
	{
		// If this is no longer custom, remove it
		GConfig->RemoveKey(TEXT("CollectionColor"), *ColorKeyStr, GEditorPerProjectIni);
	}

	// Update the map too
	LegacyCollectionColors.Remove(ColorKeyStr);
}

bool HasLegacyCustomColors(const ICollectionContainer& InCollectionContainer, TArray<FLinearColor>* OutColors)
{
	if(!FPaths::FileExists(GEditorPerProjectIni))
	{
		return false;
	}

	// Read individual entries from a config file.
	TArray<FString> Section;
	GConfig->GetSection(TEXT("CollectionColor"), Section, GEditorPerProjectIni);

	bool bHasCustom = false;

	for(FString& EntryStr : Section)
	{
		EntryStr.TrimStartInline();

		FString ColorKeyStr;
		FString ColorStr;
		if(!EntryStr.Split("=", &ColorKeyStr, &ColorStr))
		{
			continue;
		}

		// Ignore any that have invalid or default colors
		FLinearColor CurrentColor;
		if(!CurrentColor.InitFromString(ColorStr) || CurrentColor.Equals(CollectionViewUtils::GetLegacyDefaultColor()))
		{
			continue;
		}

		// Ignore any that reference old collections
		FString CollectionName;
		ECollectionShareType::Type CollectionType;
		if(!FromConfigKey(ColorKeyStr, CollectionName, CollectionType) || !InCollectionContainer.CollectionExists(*CollectionName, CollectionType))
		{
			continue;
		}

		bHasCustom = true;
		if(OutColors)
		{
			// Only add if not already present (ignores near matches too)
			bool bAdded = false;
			for(const FLinearColor& Color : *OutColors)
			{
				if(CurrentColor.Equals(Color))
				{
					bAdded = true;
					break;
				}
			}
			if(!bAdded)
			{
				OutColors->Add(CurrentColor);
			}
		}
		else
		{
			break;
		}
	}

	return bHasCustom;
}

FLinearColor ResolveColor(const ICollectionContainer& InCollectionContainer, const FName InCollectionName, const ECollectionShareType::Type& InCollectionType)
{
	TOptional<FLinearColor> CollectionColor = GetCustomColor(&InCollectionContainer, InCollectionName, InCollectionType);
	if (!CollectionColor)
	{
		CollectionColor = GetDefaultColor();
	}
	check(CollectionColor);
	return CollectionColor.GetValue();
}

TOptional<FLinearColor> GetCustomColor(const ICollectionContainer* InCollectionContainer, const FName InCollectionName, const ECollectionShareType::Type& InCollectionType)
{
	// First try and use the color set on the collection itself
	TOptional<FLinearColor> CollectionColor;
	if (InCollectionContainer != nullptr)
	{
		InCollectionContainer->GetCollectionColor(InCollectionName, InCollectionType, CollectionColor);
	}

	// Failing that, try and use the legacy local color set for this collection
	if (!CollectionColor)
	{
		CollectionColor = LoadLegacyColor(InCollectionName, InCollectionType);
	}

	return CollectionColor;
}

void SetCustomColor(ICollectionContainer& InCollectionContainer, const FName InCollectionName, const ECollectionShareType::Type& InCollectionType, const TOptional<FLinearColor>& CollectionColor)
{
	if (InCollectionContainer.SetCollectionColor(InCollectionName, InCollectionType, CollectionColor))
	{
		// Set correctly, so clear any legacy color
		ClearLegacyColor(InCollectionName, InCollectionType);
	}
}

bool HasCustomColors(const ICollectionContainer& InCollectionContainer, TArray<FLinearColor>* OutColors)
{
	bool bHasColors = false;

	// First get the colors from the collection manager
	bHasColors |= InCollectionContainer.HasCollectionColors(OutColors);

	// Then add in any legacy colors
	bHasColors |= HasLegacyCustomColors(InCollectionContainer, OutColors);

	return bHasColors;
}

FLinearColor GetDefaultColor()
{
	// Use the selection accent color as the default
	const FSlateColor NewSlateColor = FAppStyle::GetSlateColor("SelectionColor");
	return NewSlateColor.IsColorSpecified() ? NewSlateColor.GetSpecifiedColor() : FLinearColor::White;
}

} // namespace CollectionViewUtils

#undef LOCTEXT_NAMESPACE
