// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CollectionManagerTypes.h"
#include "Containers/Array.h"
#include "Math/Color.h"
#include "Misc/Optional.h"
#include "UObject/NameTypes.h"

class ICollectionContainer;

namespace CollectionViewUtils
{
	/**
	 * Resolve the color of the collection, falling back to the default if no custom color is set
	 *
	 * @param InCollectionName - The name of the collection to resolve the color for
	 * @param InCollectionType - The type of the collection to resolve the color for
	 * @return The color the collection should appear as
	 */
	FLinearColor ResolveColor(const ICollectionContainer& InCollectionContainer, const FName InCollectionName, const ECollectionShareType::Type& InCollectionType);

	/**
	 * Gets the color of the collection
	 *
	 * @param InCollectionName - The name of the collection to get the color for
	 * @param InCollectionType - The type of the collection to get the color for
	 * @return The color the collection should appear as, will be empty if not customized
	 */
	TOptional<FLinearColor> GetCustomColor(const ICollectionContainer* InCollectionContainer, const FName InCollectionName, const ECollectionShareType::Type& InCollectionType);

	/**
	 * Sets the color of the collection
	 *
	 * @param InCollectionName - The name of the collection to set the color for
	 * @param InCollectionType - The type of the collection to set the color for
	 * @param CollectionColor - The color the collection should appear as, or an empty color to clear it
	 */
	void SetCustomColor(ICollectionContainer& InCollectionContainer, const FName InCollectionName, const ECollectionShareType::Type& InCollectionType, const TOptional<FLinearColor>& CollectionColor);

	/**
	 * Checks to see if any collection has a custom color, optionally outputs them to a list
	 *
	 * @param OutColors - If specified, returns all the custom colors being used
	 * @return true, if custom colors are present
	 */
	bool HasCustomColors(const ICollectionContainer& InCollectionContainer, TArray<FLinearColor>* OutColors = nullptr);

	/** Gets the default color the collection should appear as */
	FLinearColor GetDefaultColor();
}
