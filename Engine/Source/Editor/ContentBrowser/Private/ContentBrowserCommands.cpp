// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"
#include "Misc/EnumerateRange.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

const FName FContentBrowserCommands::GenericCommandsName = TEXT("GenericCommands");

void FContentBrowserCommands::RegisterCommands()
{
	// ContentBrowser commands
	UI_COMMAND(OpenAssetsOrFolders, "Open Assets or Folders", "Opens the selected assets or folders, depending on the selection", EUserInterfaceActionType::Button, FInputChord(EKeys::Enter), FInputChord(EModifierKey::Control, EKeys::E));
	UI_COMMAND(PreviewAssets, "Preview Assets", "Loads the selected assets and previews them if possible", EUserInterfaceActionType::Button, FInputChord(EKeys::SpaceBar));
	UI_COMMAND(CreateNewFolder, "Create New Folder", "Creates new folder in selected path", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::N));
	UI_COMMAND(GoUpToParentFolder, "Go Up to Parent Folder", "Opens the folder that contains the currently open one", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control , EKeys::BackSpace));
	UI_COMMAND(SaveSelectedAsset, "Save Selected Item", "Save the selected item", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::S));
	UI_COMMAND(SaveAllCurrentFolder, "Save All", "Save All in current folder", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ResaveAllCurrentFolder, "Resave All", "Resave all assets contained in the current folder", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(EditPath, "Edit Path", "Edit the current content browser path", EUserInterfaceActionType::Button, FInputChord(EKeys::F4), FInputChord(EModifierKey::Control, EKeys::L));
	UI_COMMAND(FilterBySelection, "Filter By Selection", "Apply a filter based on the asset types of the current selection of the content browser", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Alt, EKeys::F));

	// AssetView commands
	UI_COMMAND(AssetViewCopyObjectPath, "Copy Selected Object Path", "Copy the selected object path", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::C));
	UI_COMMAND(AssetViewCopyPackageName, "Copy Selected Package Name", "Copy the selected package name", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Alt, EKeys::C));
	UI_COMMAND(GridViewShortcut, "Grid", "View assets as tiles in a grid.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ListViewShortcut, "List", "View assets in a list with thumbnails.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ColumnViewShortcut, "Column", "View assets in a list with columns of details.", EUserInterfaceActionType::RadioButton, FInputChord());
	
	// Favorites commands
	const TArray<FInputChord> InputChords
	{
		FInputChord(EModifierKey::Control, EKeys::F1),
		FInputChord(EModifierKey::Control, EKeys::F2),
		FInputChord(EModifierKey::Control, EKeys::F3),
		FInputChord(EModifierKey::Control, EKeys::F4),
		FInputChord(EModifierKey::Control, EKeys::F5),
		FInputChord(EModifierKey::Control, EKeys::F6),
		FInputChord(EModifierKey::Control, EKeys::F7),
		FInputChord(EModifierKey::Control, EKeys::F8),
		FInputChord(EModifierKey::Control, EKeys::F9),
		FInputChord(EModifierKey::Control, EKeys::F10),
	};

	const FText FavoriteShortcutFormat = LOCTEXT("FavoriteShortcutText", "FavoriteShortcut{0}");
	for (TConstEnumerateRef<FInputChord> InputChord : EnumerateRange(InputChords))
	{
		TSharedPtr<FUICommandInfo>& Command = FavoriteShortcuts.AddDefaulted_GetRef();
		const FText CommandText = FText::Format(FavoriteShortcutFormat, FText::AsNumber(InputChord.GetIndex()));

		FUICommandInfo::MakeCommandInfo(
			AsShared(),
			Command,
			FName(TEXT("FavoriteShortcut") + FString::FromInt(InputChord.GetIndex())),
			CommandText,
			LOCTEXT("FavoriteShortcutTooltip", "Shortcut for navigation that can be assigned to a favorite item"),
			FSlateIcon(),
			EUserInterfaceActionType::Button,
			*InputChord
		);
	}
}


#undef LOCTEXT_NAMESPACE
