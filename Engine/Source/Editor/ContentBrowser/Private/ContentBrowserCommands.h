// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

class FContentBrowserCommands
	: public TCommands<FContentBrowserCommands>
{
public:
	static const FName GenericCommandsName;

	/** Default constructor. */
	FContentBrowserCommands()
		: TCommands<FContentBrowserCommands>(TEXT("ContentBrowser"), NSLOCTEXT( "ContentBrowser", "ContentBrowser", "Content Browser" ), (FInputBindingManager::Get().GetContextByName(GenericCommandsName).IsValid() ? GenericCommandsName : NAME_None), FAppStyle::GetAppStyleSetName())
	{ }

	//~ Begin TCommands
	virtual void RegisterCommands() override;
	//~ End TCommands

	TSharedPtr<FUICommandInfo> CreateNewFolder;
	TSharedPtr<FUICommandInfo> OpenAssetsOrFolders;
	TSharedPtr<FUICommandInfo> GoUpToParentFolder;
	TSharedPtr<FUICommandInfo> PreviewAssets;
	TSharedPtr<FUICommandInfo> SaveSelectedAsset;
	TSharedPtr<FUICommandInfo> SaveAllCurrentFolder;
	TSharedPtr<FUICommandInfo> ResaveAllCurrentFolder;
	TSharedPtr<FUICommandInfo> EditPath;
	TSharedPtr<FUICommandInfo> AssetViewCopyObjectPath;
	TSharedPtr<FUICommandInfo> AssetViewCopyPackageName;
	TSharedPtr<FUICommandInfo> GridViewShortcut;
	TSharedPtr<FUICommandInfo> ListViewShortcut;
	TSharedPtr<FUICommandInfo> ColumnViewShortcut;
	TSharedPtr<FUICommandInfo> FilterBySelection;
	TArray<TSharedPtr<FUICommandInfo>> FavoriteShortcuts;
};
