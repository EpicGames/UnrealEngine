// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserConfig.h"

#include "ContentBrowserLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBrowserConfig)

TObjectPtr<UContentBrowserConfig> UContentBrowserConfig::Instance = nullptr;

UContentBrowserFavoriteProjectSettings* UContentBrowserFavoriteProjectSettings::Get()
{
	return GetMutableDefault<UContentBrowserFavoriteProjectSettings>();
}

void UContentBrowserConfig::Initialize()
{
	if (!Instance)
	{
		Instance = NewObject<UContentBrowserConfig>(); 
		Instance->AddToRoot();
	}
}
