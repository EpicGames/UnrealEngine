// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetThumbnail.h"
#include "EditorConfigBase.h"
#include "Engine/DeveloperSettings.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "ContentBrowserConfig.generated.h"

USTRUCT()
struct FPathViewConfig
{
	GENERATED_BODY()

	UPROPERTY()
	bool bExpanded = false;

	UPROPERTY()
	TArray<FString> PluginFilters;
};

USTRUCT()
struct FContentBrowserInstanceConfig
{
	GENERATED_BODY()

	UPROPERTY()
	FPathViewConfig PathView;

	UPROPERTY()
	bool bShowFavorites = true;

	UPROPERTY()
	bool bFavoritesExpanded = true;

	UPROPERTY()
	bool bSourcesExpanded = true;

	UPROPERTY()
	bool bFilterRecursively = false;
	
	UPROPERTY()
	bool bShowFolders = true;

	UPROPERTY()
	bool bShowEmptyFolders = true;

	UPROPERTY()
	bool bShowEngineContent = false;

	UPROPERTY()
	bool bShowDeveloperContent = false;

	UPROPERTY()
	bool bShowLocalizedContent = false;

	UPROPERTY()
	bool bShowPluginContent = false;

	UPROPERTY()
	bool bShowCppFolders = false;

	UPROPERTY()
	bool bSearchClasses = false;

	UPROPERTY()
	bool bSearchAssetPaths = false;

	UPROPERTY()
	bool bSearchCollections = false;

	UPROPERTY()
	bool bShowAssetAccessSpecifier = false;
};

USTRUCT()
struct FContentBrowserFavoriteItem
{
	GENERATED_BODY()

	/** Internal path to the folder/item */
	UPROPERTY()
	FString Path;

	/** Friendly name given to this item */
	UPROPERTY()
	FString Alias;

	/** Name of the command this item should respond to */
	UPROPERTY()
	FName ShortcutCommandName;

	bool operator==(const FContentBrowserFavoriteItem& Other) const
	{
		return Path.Equals(Other.Path);
	}

	bool operator!=(const FContentBrowserFavoriteItem& Other) const
	{
		return !(*this == Other);
	}
	
	friend uint32 GetTypeHash(const FContentBrowserFavoriteItem& InItem)
	{
		return GetTypeHash(InItem.Path);
	}
};

UCLASS(EditorConfig="ContentBrowser")
class UContentBrowserConfig : public UEditorConfigBase
{
	GENERATED_BODY()
	
public:
	static void Initialize();
	static UContentBrowserConfig* Get() { return Instance; }

	UPROPERTY(meta=(EditorConfig))
	TMap<FName, FContentBrowserInstanceConfig> Instances;

private:
	static TObjectPtr<UContentBrowserConfig> Instance;
};

UCLASS(minimalapi, Config=Editor, DisplayName="Content Browser")
class UContentBrowserCollectionProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()
public:

	UPROPERTY(config, EditAnywhere, Category=Collections, meta = (DisplayName="Exclude these collections from Content Browser"))
	TArray<FName> ExcludedCollectionsFromView;
};

UCLASS(MinimalApi, Config = EditorPerProjectUserSettings)
class UContentBrowserFavoriteProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	static UContentBrowserFavoriteProjectSettings* Get();

	/** Favorite items for a specific project */
	UPROPERTY(Config)
	TArray<FContentBrowserFavoriteItem> FavoriteItems;
};