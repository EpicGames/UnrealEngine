// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserInstanceUtils.h"
#include "ContentBrowserUtils.h"
#include "ContentBrowserConfig.h"

namespace ContentBrowserInstanceUtils
{
	template<typename T>
	T GetConfigValue(const FName& InstanceName, const T FContentBrowserInstanceConfig::* Member)
	{
		if (!InstanceName.IsNone())
		{
			const FContentBrowserInstanceConfig* Config = ContentBrowserUtils::GetConstInstanceConfig(InstanceName);
			if (Config != nullptr)
			{
				return Config->*Member;
			}
		}
		return T();
	}

	template<typename T>
	void SetConfigValue(const FName& InstanceName, T FContentBrowserInstanceConfig::* Member, const T Value, const bool bInSaveConfig)
	{
		if (!InstanceName.IsNone())
		{
			FContentBrowserInstanceConfig* Config = ContentBrowserUtils::GetContentBrowserConfig(InstanceName);
			if (Config != nullptr)
			{
				Config->*Member = Value;
				if (bInSaveConfig)
				{
					SaveAllConfigs();
				}
			}
		}
	}
};

TArray<FName> ContentBrowserInstanceUtils::GetConfigNames()
{
	TArray<FName> ConfigNames;

	UContentBrowserConfig* Config = UContentBrowserConfig::Get();
	if (Config != nullptr)
	{
		Config->Instances.GenerateKeyArray(ConfigNames);
	}

	return ConfigNames;
}

void ContentBrowserInstanceUtils::SaveAllConfigs()
{
	UContentBrowserConfig* Config = UContentBrowserConfig::Get();
	if (Config != nullptr)
	{
		Config->SaveEditorConfig();
	}
}

bool ContentBrowserInstanceUtils::GetShowPluginContent(const FName& InstanceName)
{
	return GetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowPluginContent);
}

void ContentBrowserInstanceUtils::SetShowPluginContent(const FName& InstanceName, const bool bInShowPluginContent, const bool bInSaveConfig)
{
	SetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowPluginContent, bInShowPluginContent, bInSaveConfig);
}

bool ContentBrowserInstanceUtils::GetShowLocalizedContent(const FName& InstanceName)
{
	return GetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowLocalizedContent);
}

void ContentBrowserInstanceUtils::SetShowLocalizedContent(const FName& InstanceName, const bool bInShowLocalizedContent, const bool bInSaveConfig)
{
	SetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowLocalizedContent, bInShowLocalizedContent, bInSaveConfig);
}

bool ContentBrowserInstanceUtils::GetShowEngineContent(const FName& InstanceName)
{
	return GetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowEngineContent);
}

void ContentBrowserInstanceUtils::SetShowEngineContent(const FName& InstanceName, const bool bInShowEngineContent, const bool bInSaveConfig)
{
	SetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowEngineContent, bInShowEngineContent, bInSaveConfig);
}

bool ContentBrowserInstanceUtils::GetShowDeveloperContent(const FName& InstanceName)
{
	return GetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowDeveloperContent);
}

void ContentBrowserInstanceUtils::SetShowDeveloperContent(const FName& InstanceName, const bool bInShowDeveloperContent, const bool bInSaveConfig)
{
	SetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowDeveloperContent, bInShowDeveloperContent, bInSaveConfig);
}

bool ContentBrowserInstanceUtils::GetShowCppFolders(const FName& InstanceName)
{
	return GetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowCppFolders);
}

void ContentBrowserInstanceUtils::SetShowCppFolders(const FName& InstanceName, const bool bInShowCppFolders, const bool bInSaveConfig)
{
	SetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowCppFolders, bInShowCppFolders, bInSaveConfig);
}

bool ContentBrowserInstanceUtils::GetShowFavorites(const FName& InstanceName)
{
	return GetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowFavorites);
}

void ContentBrowserInstanceUtils::SetShowFavorites(const FName& InstanceName, const bool bInShowFavorites, const bool bInSaveConfig)
{
	SetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowFavorites, bInShowFavorites, bInSaveConfig);
}

bool ContentBrowserInstanceUtils::GetFavoritesExpanded(const FName& InstanceName)
{
	return GetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bFavoritesExpanded);
}

void ContentBrowserInstanceUtils::SetFavoritesExpanded(const FName& InstanceName, const bool bInFavoritesExpanded, const bool bInSaveConfig)
{
	SetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bFavoritesExpanded, bInFavoritesExpanded, bInSaveConfig);
}

bool ContentBrowserInstanceUtils::GetSourcesExpanded(const FName& InstanceName)
{
	return GetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bSourcesExpanded);
}

void ContentBrowserInstanceUtils::SetSourcesExpanded(const FName& InstanceName, const bool bInSourcesExpanded, const bool bInSaveConfig)
{
	SetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bSourcesExpanded, bInSourcesExpanded, bInSaveConfig);
}

bool ContentBrowserInstanceUtils::GetFilterRecursively(const FName& InstanceName)
{
	return GetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bFilterRecursively);
}

void ContentBrowserInstanceUtils::SetFilterRecursively(const FName& InstanceName, const bool bInFilterRecursively, const bool bInSaveConfig)
{
	SetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bFilterRecursively, bInFilterRecursively, bInSaveConfig);
}

bool ContentBrowserInstanceUtils::GetShowFolders(const FName& InstanceName)
{
	return GetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowFolders);
}

void ContentBrowserInstanceUtils::SetShowFolders(const FName& InstanceName, const bool bInShowFolders, const bool bInSaveConfig)
{
	SetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowFolders, bInShowFolders, bInSaveConfig);
}

bool ContentBrowserInstanceUtils::GetShowEmptyFolders(const FName& InstanceName)
{
	return GetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowEmptyFolders);
}

void ContentBrowserInstanceUtils::SetShowEmptyFolders(const FName& InstanceName, const bool bInShowEmptyFolders, const bool bInSaveConfig)
{
	SetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bShowEmptyFolders, bInShowEmptyFolders, bInSaveConfig);
}

bool ContentBrowserInstanceUtils::GetSearchClasses(const FName& InstanceName)
{
	return GetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bSearchClasses);
}

void ContentBrowserInstanceUtils::SetSearchClasses(const FName& InstanceName, const bool bInSearchClasses, const bool bInSaveConfig)
{
	SetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bSearchClasses, bInSearchClasses, bInSaveConfig);
}

bool ContentBrowserInstanceUtils::GetSearchAssetPaths(const FName& InstanceName)
{
	return GetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bSearchAssetPaths);
}

void ContentBrowserInstanceUtils::SetSearchAssetPaths(const FName& InstanceName, const bool bInSearchAssetPaths, const bool bInSaveConfig)
{
	SetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bSearchAssetPaths, bInSearchAssetPaths, bInSaveConfig);
}

bool ContentBrowserInstanceUtils::GetSearchCollections(const FName& InstanceName)
{
	return GetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bSearchCollections);
}

void ContentBrowserInstanceUtils::SetSearchCollections(const FName& InstanceName, const bool bInSearchCollections, const bool bInSaveConfig)
{
	SetConfigValue(InstanceName, &FContentBrowserInstanceConfig::bSearchCollections, bInSearchCollections, bInSaveConfig);
}
