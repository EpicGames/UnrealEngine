// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserMenuContexts.h"

#include "Containers/UnrealString.h"
#include "ContentBrowserDataSubsystem.h"
#include "SContentBrowser.h"
#include "UObject/UnrealNames.h"
#include "ToolMenus.h"
#include "Algo/Copy.h"
#include "AssetViewUtils.h"
#include "AssetContextMenu.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBrowserMenuContexts)

FName UContentBrowserToolbarMenuContext::GetCurrentPath() const
{
	if (TSharedPtr<SContentBrowser> Browser = ContentBrowser.Pin())
	{
		return *Browser->GetCurrentPath(EContentBrowserPathType::Virtual);
	}

	return NAME_None;
}

bool UContentBrowserToolbarMenuContext::CanWriteToCurrentPath() const
{
	if (TSharedPtr<SContentBrowser> Browser = ContentBrowser.Pin())
	{
		return Browser->CanWriteToCurrentPath();
	}

	return false;
}

TArray<UObject*> UContentBrowserAssetContextMenuContext::LoadSelectedObjectsIfNeeded() const
{
	TArray<UObject*> Objects;
	AssetViewUtils::FLoadAssetsSettings Settings{
		// Default settings
	};
	AssetViewUtils::LoadAssetsIfNeeded(SelectedAssets, Objects, Settings);

	return Objects;
}

TArray<FAssetData> UContentBrowserAssetContextMenuContext::GetSelectedAssetsOfType(const UClass* AssetClass, EIncludeSubclasses IncludeSubclasses) const
{
	TArray<FAssetData> TypedSelectedAssets;
	if (IncludeSubclasses == EIncludeSubclasses::Yes)
	{
		Algo::CopyIf(SelectedAssets, TypedSelectedAssets, [AssetClass](const FAssetData& AssetData) { return AssetData.IsInstanceOf(AssetClass); });
	}
	else
	{
		Algo::CopyIf(SelectedAssets, TypedSelectedAssets, [AssetClass](const FAssetData& AssetData) { return AssetData.GetClass() == AssetClass; });
	}

	return TypedSelectedAssets;
}

const FAssetData* UContentBrowserAssetContextMenuContext::GetSingleSelectedAssetOfType(const UClass* AssetClass, EIncludeSubclasses IncludeSubclasses) const
{
	if (SelectedAssets.Num() == 1)
	{
		if (IncludeSubclasses == EIncludeSubclasses::Yes)
		{
			if (SelectedAssets[0].IsInstanceOf(AssetClass))
			{
				return &SelectedAssets[0];
			}
		}
		else
		{
			if (SelectedAssets[0].GetClass() == AssetClass)
			{
				return &SelectedAssets[0];
			}
		}
	}

	return nullptr;
}

const TArray<FContentBrowserItem>& UContentBrowserAssetContextMenuContext::GetSelectedItems() const
{
	if (TSharedPtr<FAssetContextMenu> PinnedAssetContextMenu = AssetContextMenu.Pin())
	{
		return PinnedAssetContextMenu->GetSelectedItems();
	}
	static const TArray<FContentBrowserItem> Empty;
	return Empty;
}

namespace UE::ContentBrowser
{
	TArray<UToolMenu*> ExtendToolMenu_AssetContextMenus(TConstArrayView<UClass*> AssetClasses)
	{
		TArray<UToolMenu*> ToolMenus;
		for (UClass* AssetClass : AssetClasses)
		{
			ToolMenus.Add(ExtendToolMenu_AssetContextMenu(TSoftClassPtr<UObject>(AssetClass)));
		}
		return ToolMenus;
	}
	
	UToolMenu* ExtendToolMenu_AssetContextMenu(UClass* AssetClass)
	{
		return ExtendToolMenu_AssetContextMenu(TSoftClassPtr<UObject>(AssetClass));
	}
	
	UToolMenu* ExtendToolMenu_AssetContextMenu(TSoftClassPtr<UObject> AssetSoftClass)
	{
		FNameBuilder Builder;
		Builder << TEXT("ContentBrowser.AssetContextMenu.");
		Builder << AssetSoftClass.GetAssetName();
		const FName MenuName(Builder.ToView());

		return UToolMenus::Get()->ExtendMenu(MenuName);
	}
}
