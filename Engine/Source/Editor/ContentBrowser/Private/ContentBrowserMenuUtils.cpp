// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserMenuUtils.h"
#include "ContentBrowserConfig.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserStyle.h"
#include "Settings/ContentBrowserSettings.h"
#include "ToolMenu.h"

#define LOCTEXT_NAMESPACE "ContentBrowserMenuUtils"

namespace ContentBrowserMenuUtils::Filters
{
	FContentBrowserInstanceConfig* GetContentBrowserConfig(FName InOwningContentBrowserName)
	{
		if (InOwningContentBrowserName.IsNone())
		{
			return nullptr;
		}

		if (UContentBrowserConfig* EditorConfig = UContentBrowserConfig::Get())
		{
			return EditorConfig->Instances.Find(InOwningContentBrowserName);
		}

		return nullptr;
	}

	bool IsToggleAllowed(FCanExecuteAction InCanExecuteAction)
	{
		if (InCanExecuteAction.IsBound())
		{
			return InCanExecuteAction.Execute();
		}

		return true;
	}

	bool IsShowingCppContent(FName InOwningContentBrowserName, FCanExecuteAction InCanExecuteAction)
	{
		if (!IsToggleAllowed(InCanExecuteAction))
		{
			return false;
		}

		if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig(InOwningContentBrowserName))
		{
			return Config->bShowCppFolders;
		}
	
		return GetDefault<UContentBrowserSettings>()->GetDisplayCppFolders();
	}

	/** Toggle whether localized content should be shown or not */
	void ToggleShowCppContent(FName InOwningContentBrowserName, FCanExecuteAction InCanExecuteAction)
	{
		check(IsToggleAllowed(InCanExecuteAction));

		bool bNewState = !GetDefault<UContentBrowserSettings>()->GetDisplayCppFolders();

		if (FContentBrowserInstanceConfig* Config = GetContentBrowserConfig(InOwningContentBrowserName))
		{
			bNewState = !Config->bShowCppFolders;
			Config->bShowCppFolders = bNewState;
			UContentBrowserConfig::Get()->SaveEditorConfig();
		}

		GetMutableDefault<UContentBrowserSettings>()->SetDisplayCppFolders(bNewState);
		GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
	}

	bool IsShowingDevelopersContent(FName InOwningContentBrowserName, FCanExecuteAction InCanExecuteAction)
	{
		if (!IsToggleAllowed(InCanExecuteAction))
		{
			return false;
		}

		if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig(InOwningContentBrowserName))
		{
			return Config->bShowDeveloperContent;
		}

		return GetDefault<UContentBrowserSettings>()->GetDisplayDevelopersFolder();
	}

	void ToggleShowDevelopersContent(FName InOwningContentBrowserName, FCanExecuteAction InCanExecuteAction)
	{
		check(IsToggleAllowed(InCanExecuteAction));

		bool bNewState = !GetDefault<UContentBrowserSettings>()->GetDisplayDevelopersFolder();

		if (FContentBrowserInstanceConfig* EditorConfig = GetContentBrowserConfig(InOwningContentBrowserName))
		{
			bNewState = !EditorConfig->bShowDeveloperContent;
			EditorConfig->bShowDeveloperContent = bNewState;
			UContentBrowserConfig::Get()->SaveEditorConfig();
		}

		GetMutableDefault<UContentBrowserSettings>()->SetDisplayDevelopersFolder(bNewState);
		GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
	}
	
	bool IsShowingEngineContent(FName InOwningContentBrowserName, FCanExecuteAction InCanExecuteAction)
	{
		if (!IsToggleAllowed(InCanExecuteAction))
		{
			// Engine toggle is allowed to change if not forced to always be shown
			// Return true in case the IsToggleAllowed for it is false since it means that it is shown
			return true;
		}

		if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig(InOwningContentBrowserName))
		{
			return Config->bShowEngineContent;
		}

		return GetDefault<UContentBrowserSettings>()->GetDisplayEngineFolder();
	}

	void ToggleShowEngineContent(FName InOwningContentBrowserName, FCanExecuteAction InCanExecuteAction)
	{
		check(IsToggleAllowed(InCanExecuteAction));

		bool bNewState = !GetDefault<UContentBrowserSettings>()->GetDisplayEngineFolder();

		if (FContentBrowserInstanceConfig* EditorConfig = GetContentBrowserConfig(InOwningContentBrowserName))
		{
			bNewState = !EditorConfig->bShowEngineContent;
			EditorConfig->bShowEngineContent = bNewState;
			UContentBrowserConfig::Get()->SaveEditorConfig();
		}

		GetMutableDefault<UContentBrowserSettings>()->SetDisplayEngineFolder(bNewState);
		GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
	}

	bool IsShowingPluginContent(FName InOwningContentBrowserName, FCanExecuteAction InCanExecuteAction)
	{
		if (!IsToggleAllowed(InCanExecuteAction))
		{
			// Plugin toggle is allowed to change if not forced to always be shown
			// Return true in case the IsToggleAllowed for it is false since it means that it is shown
			return true;
		}

		if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig(InOwningContentBrowserName))
		{
			return Config->bShowPluginContent;
		}

		return GetDefault<UContentBrowserSettings>()->GetDisplayPluginFolders();
	}

	void ToggleShowPluginContent(FName InOwningContentBrowserName, FCanExecuteAction InCanExecuteAction)
	{
		check(IsToggleAllowed(InCanExecuteAction));

		bool bNewState = !GetDefault<UContentBrowserSettings>()->GetDisplayPluginFolders();

		if (FContentBrowserInstanceConfig* EditorConfig = GetContentBrowserConfig(InOwningContentBrowserName))
		{
			bNewState = !EditorConfig->bShowPluginContent;
			EditorConfig->bShowPluginContent = bNewState;
			UContentBrowserConfig::Get()->SaveEditorConfig();
		}

		GetMutableDefault<UContentBrowserSettings>()->SetDisplayPluginFolders(bNewState);
		GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
	}
	
	bool IsShowingLocalizedContent(FName InOwningContentBrowserName, FCanExecuteAction InCanExecuteAction)
	{
		if (!IsToggleAllowed(InCanExecuteAction))
		{
			return false;
		}

		if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig(InOwningContentBrowserName))
		{
			return Config->bShowLocalizedContent;
		}

		return GetDefault<UContentBrowserSettings>()->GetDisplayL10NFolder();
	}

	void ToggleShowLocalizedContent(FName InOwningContentBrowserName, FCanExecuteAction InCanExecuteAction)
	{
		check(IsToggleAllowed(InCanExecuteAction));

		bool bNewState = !GetDefault<UContentBrowserSettings>()->GetDisplayL10NFolder();

		if (FContentBrowserInstanceConfig* Config = GetContentBrowserConfig(InOwningContentBrowserName))
		{
			bNewState = !Config->bShowLocalizedContent;
			Config->bShowLocalizedContent = bNewState;
			UContentBrowserConfig::Get()->SaveEditorConfig();
		}

		GetMutableDefault<UContentBrowserSettings>()->SetDisplayL10NFolder(bNewState);
		GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
	}
}

void ContentBrowserMenuUtils::AddFiltersToMenu(UToolMenu* InMenu, const FName& InOwningContentBrowserName, FFiltersAdditionalParams InFiltersAdditionalParams)
{
	FToolMenuInsert ContentMenuInsert = FToolMenuInsert();
	if (InMenu->ContainsSection("View"))
	{
		ContentMenuInsert.Name = TEXT("View");
		ContentMenuInsert.Position = EToolMenuInsertType::After;
	}

	FToolMenuSection& Section = InMenu->FindOrAddSection("Content", LOCTEXT("ContentHeading", "Content"), ContentMenuInsert);

	// New style omits the "Show" prefix
	const FText ShowCppLabel = UE::Editor::ContentBrowser::IsNewStyleEnabled() ?  LOCTEXT("ShowCppClassesOption_NewStyle", "C++ Classes") : LOCTEXT("ShowCppClassesOption", "Show C++ Classes");
	const FText ShowDevelopersContentLabel = UE::Editor::ContentBrowser::IsNewStyleEnabled() ? LOCTEXT("ShowDevelopersContentOption_NewStyle", "Developers Content") : LOCTEXT("ShowDevelopersContentOption", "Show Developers Content");
	const FText ShowEngineContentLabel = UE::Editor::ContentBrowser::IsNewStyleEnabled() ? LOCTEXT("ShowEngineFolderOption_NewStyle", "Engine Content") : LOCTEXT("ShowEngineFolderOption", "Show Engine Content");
	const FText ShowPluginContentLabel = UE::Editor::ContentBrowser::IsNewStyleEnabled() ? LOCTEXT("ShowPluginFolderOption_NewStyle", "Plugin Content") : LOCTEXT("ShowPluginFolderOption", "Show Plugin Content");
	const FText ShowLocalizedContentLabel = UE::Editor::ContentBrowser::IsNewStyleEnabled() ? LOCTEXT("ShowLocalizedContentOption_NewStyle", "Localized Content") : LOCTEXT("ShowLocalizedContentOption", "Show Localized Content");

	Section.AddMenuEntry(
		"ShowCppClasses",
		ShowCppLabel,
		LOCTEXT("ShowCppClassesOptionToolTip", "Show C++ classes in the view?"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(&Filters::ToggleShowCppContent, InOwningContentBrowserName, InFiltersAdditionalParams.CanShowCPPClasses),
			FCanExecuteAction::CreateStatic(&Filters::IsToggleAllowed, InFiltersAdditionalParams.CanShowCPPClasses),
			FIsActionChecked::CreateStatic(&Filters::IsShowingCppContent, InOwningContentBrowserName, InFiltersAdditionalParams.CanShowCPPClasses)
		),
		EUserInterfaceActionType::ToggleButton
	);

	Section.AddMenuEntry(
		"ShowDevelopersContent",
		ShowDevelopersContentLabel,
		LOCTEXT("ShowDevelopersContentOptionToolTip", "Show developers content in the view?"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(&Filters::ToggleShowDevelopersContent, InOwningContentBrowserName, InFiltersAdditionalParams.CanShowDevelopersContent),
			FCanExecuteAction::CreateStatic(&Filters::IsToggleAllowed, InFiltersAdditionalParams.CanShowDevelopersContent),
			FIsActionChecked::CreateStatic(&Filters::IsShowingDevelopersContent, InOwningContentBrowserName, InFiltersAdditionalParams.CanShowDevelopersContent)
			),
		EUserInterfaceActionType::ToggleButton
	);

	Section.AddMenuEntry(
		"ShowEngineFolder",
		ShowEngineContentLabel,
		LOCTEXT("ShowEngineFolderOptionToolTip", "Show engine content in the view?"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(&Filters::ToggleShowEngineContent, InOwningContentBrowserName, InFiltersAdditionalParams.CanShowEngineFolder),
			FCanExecuteAction::CreateStatic(&Filters::IsToggleAllowed, InFiltersAdditionalParams.CanShowEngineFolder),
			FIsActionChecked::CreateStatic(&Filters::IsShowingEngineContent, InOwningContentBrowserName, InFiltersAdditionalParams.CanShowEngineFolder)
		),
		EUserInterfaceActionType::ToggleButton
	);

	Section.AddMenuEntry(
		"ShowPluginFolder",
		ShowPluginContentLabel,
		LOCTEXT("ShowPluginFolderOptionToolTip", "Show plugin content in the view?"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(&Filters::ToggleShowPluginContent, InOwningContentBrowserName, InFiltersAdditionalParams.CanShowPluginFolder),
			FCanExecuteAction::CreateStatic(&Filters::IsToggleAllowed, InFiltersAdditionalParams.CanShowPluginFolder),
			FIsActionChecked::CreateStatic(&Filters::IsShowingPluginContent, InOwningContentBrowserName, InFiltersAdditionalParams.CanShowPluginFolder)
		),
		EUserInterfaceActionType::ToggleButton
	);

	Section.AddMenuEntry(
		"ShowLocalizedContent",
		ShowLocalizedContentLabel,
		LOCTEXT("ShowLocalizedContentOptionToolTip", "Show localized content in the view?"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateStatic(&Filters::ToggleShowLocalizedContent, InOwningContentBrowserName, InFiltersAdditionalParams.CanShowLocalizedContent),
			FCanExecuteAction::CreateStatic(&Filters::IsToggleAllowed, InFiltersAdditionalParams.CanShowLocalizedContent),
			FIsActionChecked::CreateStatic(&Filters::IsShowingLocalizedContent, InOwningContentBrowserName, InFiltersAdditionalParams.CanShowLocalizedContent)
			),
		EUserInterfaceActionType::ToggleButton
	);
}

#undef LOCTEXT_NAMESPACE
