// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/UIAction.h"
#include "Templates/SharedPointer.h"

class FName;
class SContentBrowser;
class UToolMenu;

/**
 * Additional parameter for the content filters
 */
struct FFiltersAdditionalParams
{
	FFiltersAdditionalParams()
	{
		CanShowCPPClasses = FCanExecuteAction();
		CanShowDevelopersContent = FCanExecuteAction();
		CanShowEngineFolder = FCanExecuteAction();
		CanShowPluginFolder = FCanExecuteAction();
		CanShowLocalizedContent = FCanExecuteAction();
	}

	FCanExecuteAction CanShowCPPClasses;
	FCanExecuteAction CanShowDevelopersContent;
	FCanExecuteAction CanShowEngineFolder;
	FCanExecuteAction CanShowPluginFolder;
	FCanExecuteAction CanShowLocalizedContent;
};

namespace ContentBrowserMenuUtils
{
	/**
	 * Add content browser filters to the given InMenu
	 * @param InMenu Menu to add filter to
	 * @param InOwningContentBrowserName ContentBrowser owner name
	 * @param InFiltersAdditionalParams Additional filters params
	 */
	void AddFiltersToMenu(UToolMenu* InMenu, const FName& InOwningContentBrowserName, FFiltersAdditionalParams InFiltersAdditionalParams = FFiltersAdditionalParams());
}
