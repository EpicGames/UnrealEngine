// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserModule.h"

#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserLog.h"
#include "ContentBrowserSingleton.h"
#include "HAL/PlatformMath.h"
#include "IContentBrowserDataModule.h"
#include "IContentBrowserSingleton.h"
#include "Logging/LogMacros.h"
#include "MRUFavoritesList.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "Settings/ContentBrowserSettings.h"
#include "UObject/UObjectGlobals.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"

IMPLEMENT_MODULE( FContentBrowserModule, ContentBrowser );
DEFINE_LOG_CATEGORY(LogContentBrowser);

const FName FContentBrowserModule::NumberOfRecentAssetsName(TEXT("NumObjectsInRecentList"));

void FContentBrowserModule::StartupModule()
{
	// Ensure the data module is loaded
	IContentBrowserDataModule::Get();

	ContentBrowserSingleton = new FContentBrowserSingleton();
	
	UContentBrowserSettings::OnSettingChanged().AddRaw(this, &FContentBrowserModule::ContentBrowserSettingChanged);
}

void FContentBrowserModule::ShutdownModule()
{	
	if ( ContentBrowserSingleton )
	{
		delete ContentBrowserSingleton;
		ContentBrowserSingleton = NULL;
	}
	UContentBrowserSettings::OnSettingChanged().RemoveAll(this);
}

IContentBrowserSingleton& FContentBrowserModule::Get() const
{
	check(ContentBrowserSingleton);
	return *ContentBrowserSingleton;
}

void FContentBrowserModule::SetContentBrowserViewExtender(const FCreateViewExtender& InViewExtender)
{
	ContentBrowserViewExtender = InViewExtender;
}

FContentBrowserModule::FCreateViewExtender FContentBrowserModule::GetContentBrowserViewExtender()
{
	return ContentBrowserViewExtender;
}

FMainMRUFavoritesList* FContentBrowserModule::GetRecentlyOpenedAssets() const
{
	return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->GetRecentlyOpenedAssets();
}

FDelegateHandle FContentBrowserModule::AddAssetViewExtraStateGenerator(const FAssetViewExtraStateGenerator& Generator)
{
	AssetViewExtraStateGenerators.Add(Generator);
	return Generator.Handle;
}

void FContentBrowserModule::RemoveAssetViewExtraStateGenerator(const FDelegateHandle& GeneratorHandle)
{
	AssetViewExtraStateGenerators.RemoveAll([&GeneratorHandle](const FAssetViewExtraStateGenerator& Generator) { return Generator.Handle == GeneratorHandle; });
}

void FContentBrowserModule::ContentBrowserSettingChanged(FName InName)
{
	if (UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem())
	{
		ContentBrowserData->RefreshVirtualPathTreeIfNeeded();
	}

	ContentBrowserSingleton->SetPrivateContentPermissionListDirty();

	OnContentBrowserSettingChanged.Broadcast(InName);
}

void FContentBrowserModule::RegisterDynamicTagAssetClass(const FTopLevelAssetPath& InClassPathName)
{
	AssetClassesRequiringDynamicTags.AddUnique(InClassPathName);
}

void FContentBrowserModule::UnregisterDynamicTagAssetClass(const FTopLevelAssetPath& InClassPathName)
{
	AssetClassesRequiringDynamicTags.Remove(InClassPathName);
}

bool FContentBrowserModule::IsDynamicTagAssetClass(const FTopLevelAssetPath& InClassPathName) const 
{
	// Fast path if the class name matches perfectly with one of the asset classes requiring dynamic tags : 
	if (AssetClassesRequiringDynamicTags.Contains(InClassPathName))
	{
		return true;
	}

	const UClass* RequestedClass = FindObject<UClass>(InClassPathName);
	if (RequestedClass == nullptr)
	{
		return false;
	}

	// Otherwise, do the slow search and check if the requested class is a child of one of the asset classes requiring dynamic tags : 
	return AssetClassesRequiringDynamicTags.ContainsByPredicate([RequestedClass](const FTopLevelAssetPath& InDynamicTagsClassPath)
		{
			const UClass* DynamicTagsClass = FindObject<UClass>(InDynamicTagsClassPath);
			return ((DynamicTagsClass != nullptr) && RequestedClass->IsChildOf(DynamicTagsClass));
		});
}

void FContentBrowserModule::RegisterWizard(const FWizard& InWizard)
{
	Wizards.Add(InWizard);
}

void FContentBrowserModule::UnregisterWizard(const FName WizardIdentifier)
{
	Wizards.RemoveAll([WizardIdentifier](const FWizard& Wizard)
	{
		return Wizard.WizardIdentifier == WizardIdentifier;
	});
}

const TArray<FContentBrowserModule::FWizard>& FContentBrowserModule::GetWizards() const
{
	return Wizards;
}

bool FContentBrowserModule::RegisterAssetSearchOverride(TUniquePtr<UE::Editor::ContentBrowser::Extension::IAssetSearchOverride>&& InOverride)
{
	if (ActiveAssetSearchOverride.IsValid())
	{
		return false;
	}

	ActiveAssetSearchOverride = MoveTemp(InOverride);
	// Next: Refresh existing content browsers?
	return true;
}

void FContentBrowserModule::UnregisterAssetSearchOverride()
{
	ActiveAssetSearchOverride.Reset();
	// Next: Refresh existing content browsers?
}

UE::Editor::ContentBrowser::Extension::IAssetSearchOverride* FContentBrowserModule::GetActiveAssetSearchOverride() const
{
	return ActiveAssetSearchOverride.Get();
}

