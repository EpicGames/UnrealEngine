// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

#include "ContentBrowserPathViewMenuContexts.generated.h"

class SPathView;

/**
 * Context for the PathView setting combo button
 */
UCLASS()
class UContentBrowserPathViewContextMenuContext : public UObject
{
	GENERATED_BODY()

public:
	FName OwningContentBrowserName;
	TWeakPtr<SPathView> PathView;
};
