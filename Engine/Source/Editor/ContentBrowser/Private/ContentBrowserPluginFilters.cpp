// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserPluginFilters.h"

#include "Containers/Array.h"
#include "Interfaces/IPluginManager.h"
#include "PluginDescriptor.h"
#include "Templates/SharedPointer.h"


/////////////////////////////////////////
// FContentBrowserPluginFilter_ContentOnlyPlugins
/////////////////////////////////////////
bool FContentBrowserPluginFilter_ContentOnlyPlugins::PassesFilter(FPluginFilterType InItem) const
{
	return InItem->GetDescriptor().Modules.Num() == 0;
}
