// Copyright Epic Games, Inc. All Rights Reserved.


#include "ContentBrowserSingleton.h"

#include "Algo/Transform.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "CollectionAssetRegistryBridge.h"
#include "Containers/Set.h"
#include "Containers/StringView.h"
#include "ContentBrowserCommands.h"
#include "ContentBrowserConfig.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserModule.h"
#include "ContentBrowserStyle.h"
#include "ContentBrowserUtils.h"
#include "ContentSources/IContentSource.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorDirectories.h"
#include "EngineLogs.h"
#include "Filters/CustomTextFilters.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IAssetTools.h"
#include "IContentBrowserDataModule.h"
#include "IDocumentation.h"
#include "Interfaces/IMainFrameModule.h"
#include "Internationalization/Internationalization.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/NamePermissionList.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/PackageName.h"
#include "Misc/StringBuilder.h"
#include "Modules/ModuleManager.h"
#include "SAssetDialog.h"
#include "SAssetPicker.h"
#include "SCollectionPicker.h"
#include "SContentBrowser.h"
#include "SPathPicker.h"
#include "StatusBarSubsystem.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "ToolMenuMisc.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Trace/Detail/Channel.h"
#include "TutorialMetaData.h"
#include "UObject/Class.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealNames.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SContentBrowserFavoriteItem.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBrowserSingleton)

class UObject;
struct FContentBrowserItem;

#define LOCTEXT_NAMESPACE "ContentBrowser"

static const FName ContentBrowserDrawerInstanceName("ContentBrowserDrawer");

IContentBrowserSingleton& IContentBrowserSingleton::Get()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	return ContentBrowserModule.Get();
}

FContentBrowserSingleton::FContentBrowserSingleton()
	: CollectionAssetRegistryBridge(MakeShared<FCollectionAssetRegistryBridge>())
	, SettingsStringID(0)
{
	const FSlateIcon ContentBrowserIcon(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetStyleSetName(), "ContentBrowser.TabIcon");
	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();
	TSharedRef<FWorkspaceItem> ContentBrowserGroup = MenuStructure.GetLevelEditorCategory()->AddGroup(
		"ContentBrowser",
		LOCTEXT("WorkspaceMenu_ContentBrowserCategory", "Content Browser"),
		LOCTEXT("ContentBrowserMenuTooltipText", "Open a Content Browser tab."),
		ContentBrowserIcon,
		true);

	for (int32 BrowserIdx = 0; BrowserIdx < UE_ARRAY_COUNT(ContentBrowserTabIDs); BrowserIdx++)
	{
		const FName TabID = FName(*FString::Printf(TEXT("ContentBrowserTab%d"), BrowserIdx + 1));
		ContentBrowserTabIDs[BrowserIdx] = TabID;

		const FText DefaultDisplayName = GetContentBrowserLabelWithIndex(BrowserIdx);

		FTabSpawnerEntry& ContentBrowserTabSpawner = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TabID, FOnSpawnTab::CreateRaw(this, &FContentBrowserSingleton::SpawnContentBrowserTab, BrowserIdx))
			.SetDisplayName(DefaultDisplayName)
			.SetTooltipText(LOCTEXT("ContentBrowserMenuTooltipText", "Open a Content Browser tab."))
			.SetGroup(ContentBrowserGroup)
			.SetIcon(ContentBrowserIcon);

		ContentBrowserTabs.Add(ContentBrowserTabSpawner.AsSpawnerEntry());
	}

	UToolMenu* ContentMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.AddQuickMenu");
	FToolMenuSection& Section = ContentMenu->FindOrAddSection("Content");

	Section.AddSubMenu("ContentBrowser", LOCTEXT("ContentBrowserMenu", "Content Browser"), LOCTEXT("ContentBrowserTooltip", "Actions related to the Content Browser"),
						FNewToolMenuDelegate::CreateRaw(this, &FContentBrowserSingleton::GetContentBrowserSubMenu, ContentBrowserGroup), false, 
						FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.OpenContentBrowser"))
						.InsertPosition.Position = EToolMenuInsertType::Last;

	// Register to be notified when properties are edited
	FEditorDelegates::LoadSelectedAssetsIfNeeded.AddRaw(this, &FContentBrowserSingleton::OnEditorLoadSelectedAssetsIfNeeded);

	FContentBrowserCommands::Register();

	PopulateConfigValues();

	ShowPrivateContentState.InvariantPaths = MakeShared<FPathPermissionList>();
	ShowPrivateContentState.InvariantPaths->OnFilterChanged().AddRaw(this, &FContentBrowserSingleton::SetPrivateContentPermissionListDirty);
	ShowPrivateContentState.CachedVirtualPaths = MakeShared<FPathPermissionList>();

	ShowEpicInternalContentState.InvariantPaths = MakeShared<FPathPermissionList>();
}

FContentBrowserSingleton::~FContentBrowserSingleton()
{
	FEditorDelegates::LoadSelectedAssetsIfNeeded.RemoveAll(this);

	if ( FSlateApplication::IsInitialized() )
	{
		for ( int32 BrowserIdx = 0; BrowserIdx < UE_ARRAY_COUNT(ContentBrowserTabIDs); BrowserIdx++ )
		{
			FGlobalTabmanager::Get()->UnregisterNomadTabSpawner( ContentBrowserTabIDs[BrowserIdx] );
		}
	}
}

TSharedRef<SWidget> FContentBrowserSingleton::CreateAssetPicker(const FAssetPickerConfig& AssetPickerConfig)
{
	return SNew( SAssetPicker )
		.IsEnabled( FSlateApplication::Get().GetNormalExecutionAttribute() )
		.AssetPickerConfig(AssetPickerConfig);
}

TSharedPtr<SWidget> FContentBrowserSingleton::GetAssetPickerSearchBox(const TSharedRef<SWidget>& AssetPickerWidget)
{
	TSharedRef<SAssetPicker> AssetPicker = StaticCastSharedRef<SAssetPicker>(AssetPickerWidget);
	return AssetPicker->GetSearchBox();
}

TSharedRef<SWidget> FContentBrowserSingleton::CreatePathPicker(const FPathPickerConfig& PathPickerConfig)
{
	return SNew( SPathPicker )
		.IsEnabled( FSlateApplication::Get().GetNormalExecutionAttribute() )
		.PathPickerConfig(PathPickerConfig);
}

TSharedRef<class SWidget> FContentBrowserSingleton::CreateCollectionPicker(const FCollectionPickerConfig& CollectionPickerConfig)
{
	return SNew( SCollectionPicker )
		.IsEnabled( FSlateApplication::Get().GetNormalExecutionAttribute() )
		.CollectionPickerConfig(CollectionPickerConfig);
}

TSharedRef<class SWidget> FContentBrowserSingleton::CreateContentBrowserDrawer(const FContentBrowserConfig& ContentBrowserConfig, TFunction<TSharedPtr<SDockTab>()> InOnGetTabForDrawer)
{
	TSharedPtr<SContentBrowser> ContentBrowserDrawerPinned;
	if(!ContentBrowserDrawer.IsValid())
	{
		ContentBrowserDrawerPinned =
			SNew(SContentBrowser, ContentBrowserDrawerInstanceName, &ContentBrowserConfig)
			.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
			.IsDrawer(true);

		OnGetTabForDrawer = InOnGetTabForDrawer;
		ContentBrowserDrawer = ContentBrowserDrawerPinned;
	}
	else
	{
		ContentBrowserDrawerPinned = ContentBrowserDrawer.Pin();
	}

	return ContentBrowserDrawerPinned.ToSharedRef();
}

void FContentBrowserSingleton::CreateOpenAssetDialog(const FOpenAssetDialogConfig& InConfig,
													 const FOnAssetsChosenForOpen& InOnAssetsChosenForOpen,
													 const FOnAssetDialogCancelled& InOnAssetDialogCancelled)
{
	const bool bModal = false;
	TSharedRef<SAssetDialog> AssetDialog = SNew(SAssetDialog, InConfig);
	AssetDialog->SetOnAssetsChosenForOpen(InOnAssetsChosenForOpen);
	AssetDialog->SetOnAssetDialogCancelled(InOnAssetDialogCancelled);
	SharedCreateAssetDialogWindow(AssetDialog, InConfig, bModal);
}

TArray<FAssetData> FContentBrowserSingleton::CreateModalOpenAssetDialog(const FOpenAssetDialogConfig& InConfig)
{
	struct FModalResults
	{
		void OnAssetsChosenForOpen(const TArray<FAssetData>& SelectedAssets)
		{
			SavedResults = SelectedAssets;
		}

		TArray<FAssetData> SavedResults;
	};

	FModalResults ModalWindowResults;
	FOnAssetsChosenForOpen OnAssetsChosenForOpenDelegate = FOnAssetsChosenForOpen::CreateRaw(&ModalWindowResults, &FModalResults::OnAssetsChosenForOpen);

	const bool bModal = true;
	TSharedRef<SAssetDialog> AssetDialog = SNew(SAssetDialog, InConfig);
	AssetDialog->SetOnAssetsChosenForOpen(OnAssetsChosenForOpenDelegate);
	SharedCreateAssetDialogWindow(AssetDialog, InConfig, bModal);

	return ModalWindowResults.SavedResults;
}

void FContentBrowserSingleton::CreateSaveAssetDialog(const FSaveAssetDialogConfig& InConfig,
													 const FOnObjectPathChosenForSave& InOnObjectPathChosenForSave,
													 const FOnAssetDialogCancelled& InOnAssetDialogCancelled)
{
	const bool bModal = false;
	TSharedRef<SAssetDialog> AssetDialog = SNew(SAssetDialog, InConfig);
	AssetDialog->SetOnObjectPathChosenForSave(InOnObjectPathChosenForSave);
	AssetDialog->SetOnAssetDialogCancelled(InOnAssetDialogCancelled);
	SharedCreateAssetDialogWindow(AssetDialog, InConfig, bModal);
}

FString FContentBrowserSingleton::CreateModalSaveAssetDialog(const FSaveAssetDialogConfig& InConfig)
{
	struct FModalResults
	{
		void OnObjectPathChosenForSave(const FString& ObjectPath)
		{
			SavedResult = ObjectPath;
		}

		FString SavedResult;
	};

	FModalResults ModalWindowResults;
	FOnObjectPathChosenForSave OnObjectPathChosenForSaveDelegate = FOnObjectPathChosenForSave::CreateRaw(&ModalWindowResults, &FModalResults::OnObjectPathChosenForSave);

	const bool bModal = true;
	TSharedRef<SAssetDialog> AssetDialog = SNew(SAssetDialog, InConfig);
	AssetDialog->SetOnObjectPathChosenForSave(OnObjectPathChosenForSaveDelegate);
	SharedCreateAssetDialogWindow(AssetDialog, InConfig, bModal);

	return ModalWindowResults.SavedResult;
}

bool FContentBrowserSingleton::HasPrimaryContentBrowser() const
{
	if ( PrimaryContentBrowser.IsValid() )
	{
		// There is a primary content browser
		return true;
	}
	else
	{
		for (int32 BrowserIdx = 0; BrowserIdx < AllContentBrowsers.Num(); ++BrowserIdx)
		{
			if ( AllContentBrowsers[BrowserIdx].IsValid() )
			{
				// There is at least one valid content browser
				return true;
			}
		}

		// There were no valid content browsers
		return false;
	}
}

bool FContentBrowserSingleton::SetPrimaryContentBrowser(FName InstanceName)
{
	for (int32 BrowserIdx = 0; BrowserIdx < AllContentBrowsers.Num(); ++BrowserIdx)
	{
		TSharedPtr<SContentBrowser> ContentBrowser = AllContentBrowsers[BrowserIdx].Pin();
		if ( ContentBrowser && ContentBrowser->GetInstanceName() == InstanceName)
		{
			// There is at least one valid content browser
			PrimaryContentBrowser = ContentBrowser;
			return true;
		}
	}
	return false;
}

void FContentBrowserSingleton::FocusPrimaryContentBrowser(bool bFocusSearch)
{
	// See if the primary content browser is still valid
	if ( !PrimaryContentBrowser.IsValid() )
	{
		ChooseNewPrimaryBrowser();
	}

	if ( PrimaryContentBrowser.IsValid() )
	{
		FocusContentBrowser( PrimaryContentBrowser.Pin() );
	}
	else
	{
		// If we couldn't find a primary content browser, open one
		SummonNewBrowser();
	}

	// Do we also want to focus on the search box of the content browser?
	if ( bFocusSearch && PrimaryContentBrowser.IsValid() )
	{
		PrimaryContentBrowser.Pin()->SetKeyboardFocusOnSearch();
	}
}

void FContentBrowserSingleton::FocusContentBrowserSearchField(TSharedPtr<SWidget> ContentBrowserWidget)
{
	TSharedPtr<SContentBrowser> BrowserToFocus;

	if (ContentBrowserWidget.IsValid() )
	{
		for (TWeakPtr<SContentBrowser>& Browser : AllContentBrowsers)
		{
			if (Browser == ContentBrowserWidget)
			{
				BrowserToFocus = Browser.Pin();
			}
		}
	}

	if (!BrowserToFocus.IsValid() && ContentBrowserDrawer == ContentBrowserWidget)
	{
		BrowserToFocus = ContentBrowserDrawer.Pin();
	}

	if (BrowserToFocus.IsValid())
	{
		BrowserToFocus->SetKeyboardFocusOnSearch();
	}
}

void FContentBrowserSingleton::CreateNewAsset(const FString& DefaultAssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory)
{
	const bool bAllowLockedBrowsers = true;
	const FName ContentBrowserInstanceName = NAME_None;
	const bool bCreateNewContentBrowser = false;

	TSharedPtr<SContentBrowser> ContentBrowserToSync = FindContentBrowserToSync(bAllowLockedBrowsers, ContentBrowserInstanceName, bCreateNewContentBrowser);

	if (!ContentBrowserToSync.IsValid())
	{
		FocusPrimaryContentBrowser(false);
		ContentBrowserToSync = PrimaryContentBrowser.Pin();
	}

	if (ContentBrowserToSync.IsValid())
	{
		ContentBrowserToSync->CreateNewAsset(DefaultAssetName, PackagePath, AssetClass, Factory);
	}
}

TSharedPtr<SContentBrowser> FContentBrowserSingleton::FindContentBrowserToSync(bool bAllowLockedBrowsers, const FName& InstanceName, bool bNewSpawnBrowser)
{
	TSharedPtr<SContentBrowser> ContentBrowserToSync;

	if (InstanceName.IsValid() && !InstanceName.IsNone())
	{
		for (int32 BrowserIdx = 0; BrowserIdx < AllContentBrowsers.Num(); ++BrowserIdx)
		{
			if (AllContentBrowsers[BrowserIdx].IsValid() && AllContentBrowsers[BrowserIdx].Pin()->GetInstanceName() == InstanceName)
			{
				return AllContentBrowsers[BrowserIdx].Pin();
			}	
		}

		return ContentBrowserToSync;
	}

	if ( !PrimaryContentBrowser.IsValid() )
	{
		ChooseNewPrimaryBrowser();
	}

	auto CanContentBrowserBeSynced =
		[ this, bAllowLockedBrowsers, PinnedDrawer = ContentBrowserDrawer.Pin() ]( const TSharedPtr< SContentBrowser >& ContentBrowser ) -> bool
		{
			// Content browser can be synced if:
			// - It's valid.
			// - It's not locked or we allow locked browsers (bAllowLockedBrowsers).
			// - If it's the drawer, the active window needs to have a status bar so that it can show the drawer.

			bool bCanBeSynced = ContentBrowser.IsValid() && ( bAllowLockedBrowsers || !ContentBrowser->IsLocked() );

			if ( ContentBrowser == PinnedDrawer )
			{
				bCanBeSynced = bCanBeSynced && GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->ActiveWindowHasStatusBar();
			}

			return bCanBeSynced;
		};

	if ( CanContentBrowserBeSynced( PrimaryContentBrowser.Pin() ) )
	{
		// If wanting to spawn a new browser window, don't set the BrowserToSync in order to summon a new browser
		if (!bNewSpawnBrowser)
		{
			// If the primary content browser is not locked, sync it
			ContentBrowserToSync = PrimaryContentBrowser.Pin();
		}
	}
	else
	{

		// If there is no primary or it is locked, sync the content browser drawer if it's available
		if ( PrimaryContentBrowser != ContentBrowserDrawer
			&& CanContentBrowserBeSynced( ContentBrowserDrawer.Pin() ) )
		{
			ContentBrowserToSync = ContentBrowserDrawer.Pin();
		}
		else
		{
			// if the drawer isn't available, find the first non-locked valid browser
			for ( const TWeakPtr< SContentBrowser >& ContentBrowser : AllContentBrowsers )
			{
				if ( CanContentBrowserBeSynced( ContentBrowser.Pin() ) )
				{
					ContentBrowserToSync = ContentBrowser.Pin();
					break;
				}
			}
		}
	}

	if ( !ContentBrowserToSync.IsValid() )
	{
		// There are no valid, unlocked browsers, attempt to summon a new one.
		const FName NewBrowserName = SummonNewBrowser(bAllowLockedBrowsers);

		// Now try to find a non-locked valid browser again, now that a new one may exist
		for (int32 BrowserIdx = 0; BrowserIdx < AllContentBrowsers.Num(); ++BrowserIdx)
		{
			if ((AllContentBrowsers[BrowserIdx].IsValid() && (NewBrowserName == NAME_None && (bAllowLockedBrowsers || !AllContentBrowsers[BrowserIdx].Pin()->IsLocked()))) || (AllContentBrowsers[BrowserIdx].IsValid() && AllContentBrowsers[BrowserIdx].Pin()->GetInstanceName() == NewBrowserName))
			{
				ContentBrowserToSync = AllContentBrowsers[BrowserIdx].Pin();
				break;
			}
		}
	}

	if ( !ContentBrowserToSync.IsValid() )
	{
		FText NotificationText = LOCTEXT("ContentBrowserLocked", "Unable to browse to the requested asset. All Content Browsers are locked.");
		FNotificationInfo Notification(NotificationText);
		Notification.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Notification);

		UE_LOGF(LogNet, Log, "%ls", *NotificationText.ToString());
	}

	return ContentBrowserToSync;
}

void FContentBrowserSingleton::SyncBrowserToAssets(const TArray<FAssetData>& AssetDataList, bool bAllowLockedBrowsers, bool bFocusContentBrowser, const FName& InstanceName, bool bNewSpawnBrowser)
{
	TSharedPtr<SContentBrowser> ContentBrowserToSync = FindContentBrowserToSync(bAllowLockedBrowsers, InstanceName, bNewSpawnBrowser);

	if ( ContentBrowserToSync.IsValid() )
	{
		// Finally, focus and sync the browser that was found
		if (bFocusContentBrowser)
		{
			FocusContentBrowser(ContentBrowserToSync);
		}

		TSet<FName> OuterPathNames;
		TArray<FAssetData> AssetDataListToSync;
		AssetDataListToSync.Reserve(AssetDataList.Num());

		for (const FAssetData& AssetData : AssetDataList)
		{
			if (FName OuterPathName = AssetData.GetOptionalOuterPathName(); !OuterPathName.IsNone())
			{
				OuterPathNames.Add(*FSoftObjectPath(OuterPathName.ToString()).GetLongPackageName());
			}
			else
			{
				AssetDataListToSync.Add(AssetData);
			}
		}

		if (OuterPathNames.Num())
		{
			TMap<FName, FAssetData> PackagesToAssetDataMap;
			UE::AssetRegistry::GetAssetForPackages(OuterPathNames.Array(), PackagesToAssetDataMap);
			Algo::Transform(PackagesToAssetDataMap, AssetDataListToSync, [](const TPair<FName, FAssetData>& Pair) { return Pair.Value; });
		}

		ContentBrowserToSync->SyncToAssets(AssetDataListToSync);
	}
}

void FContentBrowserSingleton::SyncBrowserToAssets(const TArray<UObject*>& AssetList, bool bAllowLockedBrowsers, bool bFocusContentBrowser, const FName& InstanceName, bool bNewSpawnBrowser)
{
	// Convert UObject* array to FAssetData array
	TArray<FAssetData> AssetDataList;
	for (int32 AssetIdx = 0; AssetIdx < AssetList.Num(); ++AssetIdx)
	{
		if ( AssetList[AssetIdx] )
		{
			AssetDataList.Add( FAssetData(AssetList[AssetIdx]) );
		}
	}

	SyncBrowserToAssets(AssetDataList, bAllowLockedBrowsers, bFocusContentBrowser, InstanceName, bNewSpawnBrowser);
}

void FContentBrowserSingleton::SyncBrowserToFolders(const TArray<FString>& FolderList, bool bAllowLockedBrowsers, bool bFocusContentBrowser, const FName& InstanceName, bool bNewSpawnBrowser)
{
	TSharedPtr<SContentBrowser> ContentBrowserToSync = FindContentBrowserToSync(bAllowLockedBrowsers, InstanceName, bNewSpawnBrowser);

	if ( ContentBrowserToSync.IsValid() )
	{
		// Finally, focus and sync the browser that was found
		if (bFocusContentBrowser)
		{
			FocusContentBrowser(ContentBrowserToSync);
		}
		ContentBrowserToSync->SyncToFolders(FolderList);
	}
}

void FContentBrowserSingleton::SyncBrowserToCollections(const TArray<FCollectionRef>& CollectionsToSync, bool bAllowLockedBrowsers, bool bFocusContentBrowser, const FName& InstanceName, bool bNewSpawnBrowser)
{
	TSharedPtr<SContentBrowser> ContentBrowserToSync = FindContentBrowserToSync(bAllowLockedBrowsers, InstanceName, bNewSpawnBrowser);

	if (ContentBrowserToSync.IsValid())
	{
		// Finally, focus and sync the browser that was found
		if (bFocusContentBrowser)
		{
			FocusContentBrowser(ContentBrowserToSync);
		}
		ContentBrowserToSync->SyncToCollections(CollectionsToSync);
	}
}

void FContentBrowserSingleton::SyncBrowserToItems(const TArray<FContentBrowserItem>& ItemsToSync, bool bAllowLockedBrowsers, bool bFocusContentBrowser, const FName& InstanceName, bool bNewSpawnBrowser)
{
	TSharedPtr<SContentBrowser> ContentBrowserToSync = FindContentBrowserToSync(bAllowLockedBrowsers, InstanceName, bNewSpawnBrowser);

	if ( ContentBrowserToSync.IsValid() )
	{
		// Finally, focus and sync the browser that was found
		if (bFocusContentBrowser)
		{
			FocusContentBrowser(ContentBrowserToSync);
		}
		ContentBrowserToSync->SyncToItems(ItemsToSync);
	}
}

void FContentBrowserSingleton::SyncBrowserTo(const FContentBrowserSelection& ItemSelection, bool bAllowLockedBrowsers, bool bFocusContentBrowser, const FName& InstanceName, bool bNewSpawnBrowser)
{
	TSharedPtr<SContentBrowser> ContentBrowserToSync = FindContentBrowserToSync(bAllowLockedBrowsers, InstanceName, bNewSpawnBrowser);

	if ( ContentBrowserToSync.IsValid() )
	{
		// Finally, focus and sync the browser that was found
		if (bFocusContentBrowser)
		{
			FocusContentBrowser(ContentBrowserToSync);
		}
		ContentBrowserToSync->SyncTo(ItemSelection);
	}
}

void FContentBrowserSingleton::GetSelectedAssets(TArray<FAssetData>& SelectedAssets)
{
	if ( PrimaryContentBrowser.IsValid() )
	{
		PrimaryContentBrowser.Pin()->GetSelectedAssets(SelectedAssets);
	}
}

void FContentBrowserSingleton::GetAllSelectedAssets(TArray<FAssetData>& SelectedAssets)
{
	for (const TWeakPtr<SContentBrowser>& Browser : AllContentBrowsers)
	{
		TSharedPtr<SContentBrowser> BrowserPin = Browser.Pin();
		// Skip browsers not currently showing the legacy (asset view) source. Querying them would
		// force-switch them away from their active source (e.g. Fab) and they have no legacy selection anyway.
		if (!BrowserPin.IsValid() || !BrowserPin->IsLegacyContentSourceActive())
		{
			continue;
		}

		TArray<FAssetData> BrowserSelection;
		BrowserPin->GetSelectedAssets(BrowserSelection);
		for (FAssetData& Asset : BrowserSelection)
		{
			SelectedAssets.AddUnique(MoveTemp(Asset));
		}
	}
}

void FContentBrowserSingleton::GetSelectedFolders(TArray<FString>& SelectedFolders)
{
	if (PrimaryContentBrowser.IsValid())
	{
		PrimaryContentBrowser.Pin()->GetSelectedFolders(SelectedFolders);
	}
}

void FContentBrowserSingleton::GetSelectedPathViewFolders(TArray<FString>& SelectedFolders)
{
	if (PrimaryContentBrowser.IsValid())
	{
		SelectedFolders = PrimaryContentBrowser.Pin()->GetSelectedPathViewFolders();
	}
}

FString FContentBrowserSingleton::GetCurrentPath(const EContentBrowserPathType PathType)
{
	if (PrimaryContentBrowser.IsValid())
	{
		return PrimaryContentBrowser.Pin()->GetCurrentPath(PathType);
	}
	return FString();
}

FContentBrowserItemPath FContentBrowserSingleton::GetCurrentPath()
{
	if (PrimaryContentBrowser.IsValid())
	{
		return FContentBrowserItemPath(PrimaryContentBrowser.Pin()->GetCurrentPath(EContentBrowserPathType::Virtual), EContentBrowserPathType::Virtual);
	}

	return FContentBrowserItemPath();
}

void FContentBrowserSingleton::CaptureThumbnailFromViewport(FViewport* InViewport, TArray<FAssetData>& SelectedAssets)
{
	ContentBrowserUtils::CaptureThumbnailFromViewport(InViewport, SelectedAssets);
}


void FContentBrowserSingleton::OnEditorLoadSelectedAssetsIfNeeded()
{
	if ( PrimaryContentBrowser.IsValid() )
	{
		PrimaryContentBrowser.Pin()->LoadSelectedObjectsIfNeeded();
	}
}

FContentBrowserSingleton& FContentBrowserSingleton::Get()
{
	static const FName ModuleName = "ContentBrowser";
	FContentBrowserModule& Module = FModuleManager::GetModuleChecked<FContentBrowserModule>(ModuleName);
	return static_cast<FContentBrowserSingleton&>(Module.Get());
}

void FContentBrowserSingleton::SetPrimaryContentBrowser(const TSharedRef<SContentBrowser>& NewPrimaryBrowser)
{
	if ( PrimaryContentBrowser.IsValid() && PrimaryContentBrowser.Pin().ToSharedRef() == NewPrimaryBrowser )
	{
		// This is already the primary content browser
		return;
	}

	if ( !NewPrimaryBrowser->CanSetAsPrimaryContentBrowser() )
	{
		// This browser can not be set as primary
		return;
	}

	if ( PrimaryContentBrowser.IsValid() )
	{
		PrimaryContentBrowser.Pin()->SetIsPrimaryContentBrowser(false);
	}

	PrimaryContentBrowser = NewPrimaryBrowser;
	NewPrimaryBrowser->SetIsPrimaryContentBrowser(true);
}

void FContentBrowserSingleton::ContentBrowserClosed(const TSharedRef<SContentBrowser>& ClosedBrowser)
{
	// Find the browser in the all browsers list
	for (int32 BrowserIdx = AllContentBrowsers.Num() - 1; BrowserIdx >= 0; --BrowserIdx)
	{
		if ( !AllContentBrowsers[BrowserIdx].IsValid() || AllContentBrowsers[BrowserIdx].Pin() == ClosedBrowser )
		{
			AllContentBrowsers.RemoveAt(BrowserIdx);
		}
	}

	if ( !PrimaryContentBrowser.IsValid() || ClosedBrowser == PrimaryContentBrowser.Pin() )
	{
		ChooseNewPrimaryBrowser();
	}

	BrowserToLastKnownTabManagerMap.Add(ClosedBrowser->GetInstanceName(), ClosedBrowser->GetTabManager());
}

const FContentBrowserPluginSettings& FContentBrowserSingleton::GetPluginSettings(FName PluginName) const
{
	const FContentBrowserPluginSettings* LocalSettings = PluginSettings.FindByPredicate([PluginName](const FContentBrowserPluginSettings& Setting) { return Setting.PluginName == PluginName; });
	if (LocalSettings)
	{
		return *LocalSettings;
	}

	static FContentBrowserPluginSettings DefaultSettings;
	return DefaultSettings;
}

void FContentBrowserSingleton::OpenEditFavoriteItemWindow(const FContentBrowserFavoriteItem& InItem)
{
	// Only a single edit window should be opened
	if (const TSharedPtr<SWindow> EditFavoriteItemWindow = EditFavoriteItemWindowWeak.Pin())
	{
		EditFavoriteItemWindow->RequestDestroyWindow();
		EditFavoriteItemWindowWeak.Reset();
	}
	
	// Filter shortcuts by removing those already assigned
	TArray<TSharedPtr<FUICommandInfo>> ShortcutCommands;
	if (const UContentBrowserFavoriteProjectSettings* FavoriteSettings = UContentBrowserFavoriteProjectSettings::Get())
	{
		TSet<FName> ShortcutUsed;
		ShortcutUsed.Reserve(FavoriteSettings->FavoriteItems.Num());
		Algo::Transform(FavoriteSettings->FavoriteItems, ShortcutUsed, [](const FContentBrowserFavoriteItem& InItem)
		{
			return InItem.ShortcutCommandName;
		});
		
		// Do not remove current shortcut for this item
		ShortcutUsed.Remove(InItem.ShortcutCommandName);

		FContentBrowserCommands& ContentBrowserCommands = FContentBrowserCommands::Get();
		ShortcutCommands.Reserve(ContentBrowserCommands.FavoriteShortcuts.Num() - ShortcutUsed.Num());
		for (const TSharedPtr<FUICommandInfo>& FavoriteShortcut : ContentBrowserCommands.FavoriteShortcuts)
		{
			if (FavoriteShortcut 
				&& !ShortcutUsed.Contains(FavoriteShortcut->GetCommandName()))
			{
				ShortcutCommands.Emplace(FavoriteShortcut);
			}
		}
	}

	SContentBrowserFavoriteItem::FArguments Args;
	Args.OnConfirm_Raw(this, &FContentBrowserSingleton::OnFavoriteItemEdited);
	Args.CommandOptions(ShortcutCommands);
	EditFavoriteItemWindowWeak = SContentBrowserFavoriteItem::CreateAndShowWindow(Args, InItem);
}

void FContentBrowserSingleton::OnFavoriteItemEdited(const FContentBrowserFavoriteItem& InItem, bool bInAnyChanges)
{
	if (!bInAnyChanges)
	{
		return;
	}

	const FContentBrowserItemPath ItemPath(InItem.Path, EContentBrowserPathType::Internal);
	if (FContentBrowserFavoriteItem* FavoriteItem = ContentBrowserUtils::FindFavoriteFolder(ItemPath))
	{
		(*FavoriteItem) = InItem;

		if (UContentBrowserFavoriteProjectSettings* FavoriteSettings = UContentBrowserFavoriteProjectSettings::Get())
		{
			FavoriteSettings->SaveConfig();
		}

		BroadcastFavoritesChanged(ItemPath, /** Added */false);
	}
}

void FContentBrowserSingleton::SharedCreateAssetDialogWindow(const TSharedRef<SAssetDialog>& AssetDialog, const FSharedAssetDialogConfig& InConfig, bool bModal) const
{
	const FVector2D DefaultWindowSize(1152.0f, 648.0f);
	const FVector2D WindowSize = InConfig.WindowSizeOverride.IsZero() ? DefaultWindowSize : InConfig.WindowSizeOverride;
	const FText WindowTitle = InConfig.DialogTitleOverride.IsEmpty() ? LOCTEXT("GenericAssetDialogWindowHeader", "Asset Dialog") : InConfig.DialogTitleOverride;

	TSharedRef<SWindow> DialogWindow =
		SNew(SWindow)
		.Title(WindowTitle)
		.ClientSize(WindowSize);

	DialogWindow->SetContent(AssetDialog);

	TSharedPtr<SWindow> ParentWindow = InConfig.WindowOverride;
	if (!InConfig.WindowOverride)
	{
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		const TSharedPtr<SWindow>& MainFrameParentWindow = MainFrameModule.GetParentWindow();
		ParentWindow = MainFrameModule.GetParentWindow();
	}


	if (ParentWindow.IsValid())
	{
		if (bModal)
		{
			FSlateApplication::Get().AddModalWindow(DialogWindow, ParentWindow.ToSharedRef());
		}
		else if (FGlobalTabmanager::Get()->GetRootWindow().IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(DialogWindow, ParentWindow.ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(DialogWindow);
		}
	}
	else
	{
		if (ensureMsgf(!bModal, TEXT("Could not create asset dialog because modal windows must have a parent and this was called at a time where the mainframe window does not exist.")))
		{
			FSlateApplication::Get().AddWindow(DialogWindow);
		}
	}
}

void FContentBrowserSingleton::ChooseNewPrimaryBrowser()
{
	// The drawer is the lowest priority. If a docked content browser can become the primary browser choose it first
	// Find the first valid browser and trim any invalid browsers along the way
	for (int32 BrowserIdx = 0; BrowserIdx < AllContentBrowsers.Num(); ++BrowserIdx)
	{
		if (TSharedPtr<SContentBrowser> TestBrowser = AllContentBrowsers[BrowserIdx].Pin())
		{
			if (TestBrowser->CanSetAsPrimaryContentBrowser() && TestBrowser != ContentBrowserDrawer)
			{
				SetPrimaryContentBrowser(AllContentBrowsers[BrowserIdx].Pin().ToSharedRef());
				break;
			}
		}
		else
		{
			// Trim any invalid content browsers
			AllContentBrowsers.RemoveAt(BrowserIdx);
			BrowserIdx--;
		}
	}

	// Set the content browser drawer as the primary browser
	if (!PrimaryContentBrowser.IsValid() && ContentBrowserDrawer.IsValid())
	{
		SetPrimaryContentBrowser(ContentBrowserDrawer.Pin().ToSharedRef());
	}
}

void FContentBrowserSingleton::FocusContentBrowser(const TSharedPtr<SContentBrowser>& BrowserToFocus)
{
	if ( BrowserToFocus.IsValid() )
	{
		if (BrowserToFocus == ContentBrowserDrawer)
		{
			GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->OpenContentBrowserDrawer();
		}
		else
		{
			TSharedRef<SContentBrowser> Browser = BrowserToFocus.ToSharedRef();
			TSharedPtr<FTabManager> TabManager = Browser->GetTabManager();
			if (TabManager.IsValid())
			{
				TabManager->TryInvokeTab(Browser->GetInstanceName());
			}
		}
	}
}

void FContentBrowserSingleton::DockContentBrowserDrawer()
{
	TSharedPtr<FTabManager> ForTabManager;
	TSharedPtr<SDockTab> Tab = OnGetTabForDrawer();
	if (Tab)
	{
		ForTabManager = FGlobalTabmanager::Get()->GetTabManagerForMajorTab(Tab.ToSharedRef());
	}

	
	// Dont summon a content browser if a content browser already exists in the tab manager
	bool bExistingTab = false;
	if(ForTabManager)
	{
		for (int32 BrowserIdx = 0; BrowserIdx < UE_ARRAY_COUNT(ContentBrowserTabIDs); BrowserIdx++)
		{
			if (TSharedPtr<SDockTab> ExistingTab = ForTabManager->FindExistingLiveTab(ContentBrowserTabIDs[BrowserIdx]))
			{
				GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->ForceDismissDrawer();
				ExistingTab->ActivateInParent(ETabActivationCause::SetDirectly);
				bExistingTab = true;
				break;
			}
		}
	}

	if(!bExistingTab)
	{
		TSharedPtr<SContentBrowser> ContentBrowserDrawerPinned = ContentBrowserDrawer.Pin();

		// Make sure current content browser drawer settings are saved so we can copy them to the new browser
		ContentBrowserDrawerPinned->SaveSettings();

		FName InstanceName = SummonNewBrowser(false, ForTabManager);
		for (int32 AllBrowsersIdx = AllContentBrowsers.Num() - 1; AllBrowsersIdx >= 0; --AllBrowsersIdx)
		{
			if (TSharedPtr<SContentBrowser> Browser = AllContentBrowsers[AllBrowsersIdx].Pin())
			{
				if (Browser->GetInstanceName() == InstanceName)
				{
					Browser->CopySettingsFromBrowser(ContentBrowserDrawerPinned);
					break;
				}
			}
		}

		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->ForceDismissDrawer();
	}

}

FName FContentBrowserSingleton::SummonNewBrowser(bool bAllowLockedBrowsers, TSharedPtr<FTabManager> SpecificTabManager)
{
	TSet<FName> OpenBrowserIDs;

	// Find all currently open browsers to help find the first open slot
	for (int32 BrowserIdx = AllContentBrowsers.Num() - 1; BrowserIdx >= 0; --BrowserIdx)
	{
		const TWeakPtr<SContentBrowser>& Browser = AllContentBrowsers[BrowserIdx];
		if ( Browser.IsValid() )
		{
			OpenBrowserIDs.Add(Browser.Pin()->GetInstanceName());
		}
	}
	
	FName NewTabName;
	for ( int32 BrowserIdx = 0; BrowserIdx < UE_ARRAY_COUNT(ContentBrowserTabIDs); BrowserIdx++ )
	{
		FName TestTabID = ContentBrowserTabIDs[BrowserIdx];
		if ( !OpenBrowserIDs.Contains(TestTabID) && (bAllowLockedBrowsers || !IsLocked(TestTabID)) )
		{
			// Found the first index that is not currently open
			NewTabName = TestTabID;
			break;
		}
	}

	if ( NewTabName != NAME_None )
	{
		const TWeakPtr<FTabManager>& TabManagerToInvoke = SpecificTabManager.IsValid() ? SpecificTabManager : BrowserToLastKnownTabManagerMap.FindRef(NewTabName);
		if ( TabManagerToInvoke.IsValid() )
		{
			TabManagerToInvoke.Pin()->TryInvokeTab(NewTabName);
		}
		else
		{
			FGlobalTabmanager::Get()->TryInvokeTab(NewTabName);
		}
	}
	else
	{
		// No available slots... don't summon anything
	}

	return NewTabName;
}

TSharedRef<SWidget> FContentBrowserSingleton::CreateContentBrowser( const FName InstanceName, TSharedPtr<SDockTab> ContainingTab, const FContentBrowserConfig* ContentBrowserConfig )
{
	TSharedRef<SContentBrowser> NewBrowser =
		SNew( SContentBrowser, InstanceName, ContentBrowserConfig )
		.IsEnabled( FSlateApplication::Get().GetNormalExecutionAttribute() )
		.ContainingTab( ContainingTab );

	AllContentBrowsers.Add( NewBrowser );

	// The drawer is the lowest priority. If a new content browser is being added see if it is capable of becoming the primary browser
	if( !PrimaryContentBrowser.IsValid() || PrimaryContentBrowser == ContentBrowserDrawer)
	{
		ChooseNewPrimaryBrowser();
	}

	return NewBrowser;
}


TSharedRef<SDockTab> FContentBrowserSingleton::SpawnContentBrowserTab( const FSpawnTabArgs& SpawnTabArgs, int32 BrowserIdx )
{	
	TAttribute<FText> Label = TAttribute<FText>::Create( TAttribute<FText>::FGetter::CreateRaw( this, &FContentBrowserSingleton::GetContentBrowserTabLabel, BrowserIdx ) );

	TSharedRef<SDockTab> NewTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.Label( Label )
		.ToolTip( IDocumentation::Get()->CreateToolTip( Label, nullptr, "Shared/ContentBrowser", "Tab" ) )
		.OnExtendContextMenu_Raw(this, &FContentBrowserSingleton::ExtendContentBrowserTabContextMenu);
	
	TSharedRef<SWidget> NewBrowser = CreateContentBrowser( SpawnTabArgs.GetTabId().TabType, NewTab, nullptr );

	// Add wrapper for tutorial highlighting
	TSharedRef<SBox> Wrapper =
		SNew(SBox)
		.AddMetaData<FTutorialMetaData>(FTutorialMetaData(TEXT("ContentBrowser"), TEXT("ContentBrowserTab1")))
		[
			NewBrowser
		];

	NewTab->SetContent( Wrapper );

	return NewTab;
}

void FContentBrowserSingleton::ExtendContentBrowserTabContextMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("SummonContentBrowserTabs", LOCTEXT("ContentBrowserTabs", "Content Browser Tabs"));

	for (int32 BrowserIdx = 0; BrowserIdx < UE_ARRAY_COUNT(ContentBrowserTabIDs); BrowserIdx++)
	{
		const FName TabID = ContentBrowserTabIDs[BrowserIdx];
		const FText DefaultDisplayName = GetContentBrowserLabelWithIndex(BrowserIdx);

		InMenuBuilder.AddMenuEntry(DefaultDisplayName,
			LOCTEXT("ContentBrowserMenuTooltipText", "Open a Content Browser tab."),
			FSlateIcon(),
			FUIAction(
					FExecuteAction::CreateLambda([TabID, this]()
					{
						// Go through all the content browsers to check if the current one is open
						for (int32 BrowserIdx = 0; BrowserIdx < AllContentBrowsers.Num(); ++BrowserIdx)
						{
							const TWeakPtr<SContentBrowser>& Browser = AllContentBrowsers[BrowserIdx];

							if (Browser.IsValid() && Browser.Pin()->GetInstanceName() == TabID)
							{
								FocusContentBrowser(Browser.Pin()); // Focus it if so
							}
						}

						// If the tab was not found, try to open it
						FGlobalTabmanager::Get()->TryInvokeTab(TabID);
					}
				),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([this, TabID]()
					{
						// Go through all the content browsers to check if the current one is open
						for (int32 BrowserIdx = 0; BrowserIdx < AllContentBrowsers.Num(); ++BrowserIdx)
						{
							const TWeakPtr<SContentBrowser>& Browser = AllContentBrowsers[BrowserIdx];

							if (Browser.IsValid() && Browser.Pin()->GetInstanceName() == TabID)
							{
								return true;
							}
						}

						return false;
						
					}
				)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}

	InMenuBuilder.EndSection();
}


bool FContentBrowserSingleton::IsLocked(const FName& InstanceName) const
{
	// First try all the open browsers, as their locked state might be newer than the configs
	for (int32 AllBrowsersIdx = AllContentBrowsers.Num() - 1; AllBrowsersIdx >= 0; --AllBrowsersIdx)
	{
		const TWeakPtr<SContentBrowser>& Browser = AllContentBrowsers[AllBrowsersIdx];
		if ( Browser.IsValid() && Browser.Pin()->GetInstanceName() == InstanceName )
		{
			return Browser.Pin()->IsLocked();
		}
	}

	// Fallback to getting the locked state from the config instead
	bool bIsLocked = false;
	GConfig->GetBool(*SContentBrowser::SettingsIniSection, *(InstanceName.ToString() + TEXT(".Locked")), bIsLocked, GEditorPerProjectIni);
	return bIsLocked;
}

FText FContentBrowserSingleton::GetContentBrowserLabelWithIndex( int32 BrowserIdx )
{
	return FText::Format(LOCTEXT("ContentBrowserTabNameWithIndex", "Content Browser {0}"), FText::AsNumber(BrowserIdx + 1));
}

FText FContentBrowserSingleton::GetContentBrowserTabLabel(int32 BrowserIdx)
{
	int32 NumOpenContentBrowsers = 0;
	for (int32 AllBrowsersIdx = AllContentBrowsers.Num() - 1; AllBrowsersIdx >= 0; --AllBrowsersIdx)
	{
		const TWeakPtr<SContentBrowser>& Browser = AllContentBrowsers[AllBrowsersIdx];
		if ( Browser.IsValid() )
		{
			NumOpenContentBrowsers++;
		}
		else
		{
			AllContentBrowsers.RemoveAt(AllBrowsersIdx);
		}
	}

	if ( NumOpenContentBrowsers > 1 )
	{
		return GetContentBrowserLabelWithIndex( BrowserIdx );
	}
	else
	{
		return LOCTEXT("ContentBrowserTabName", "Content Browser");
	}
}

void FContentBrowserSingleton::SetSelectedPaths(const TArray<FString>& FolderPaths, bool bNeedsRefresh/* = false*/, bool bPathsAreVirtual/* = false*/)
{
	// Make sure we have a valid browser
	if (!PrimaryContentBrowser.IsValid())
	{
		ChooseNewPrimaryBrowser();

		if (!PrimaryContentBrowser.IsValid())
		{
			SummonNewBrowser();
		}
	}

	if (PrimaryContentBrowser.IsValid())
	{
		if (bPathsAreVirtual)
		{
			PrimaryContentBrowser.Pin()->SetSelectedPaths(FolderPaths, bNeedsRefresh);
		}
		else
		{
			if (UContentBrowserDataSubsystem* ContentBrowserDataSubsystem = IContentBrowserDataModule::Get().GetSubsystem())
			{
				PrimaryContentBrowser.Pin()->SetSelectedPaths(ContentBrowserDataSubsystem->ConvertInternalPathsToVirtual(FolderPaths), bNeedsRefresh);
			}
		}
	}
}

void FContentBrowserSingleton::ForceShowPluginContent(bool bEnginePlugin)
{
	if (!PrimaryContentBrowser.IsValid())
	{
		ChooseNewPrimaryBrowser();

		if (!PrimaryContentBrowser.IsValid())
		{
			SummonNewBrowser();
		}
	}

	if (PrimaryContentBrowser.IsValid())
	{
		PrimaryContentBrowser.Pin()->ForceShowPluginContent(bEnginePlugin);
	}
}

void FContentBrowserSingleton::SaveContentBrowserSettings(TSharedPtr<SWidget> ContentBrowserWidget)
{
	TSharedPtr<SContentBrowser> BrowserToSave;

	if (ContentBrowserWidget.IsValid())
	{
		for (TWeakPtr<SContentBrowser>& Browser : AllContentBrowsers)
		{
			if (Browser == ContentBrowserWidget)
			{
				BrowserToSave = Browser.Pin();
			}
		}
	}

	if (!BrowserToSave.IsValid() && ContentBrowserDrawer == ContentBrowserWidget)
	{
		BrowserToSave = ContentBrowserDrawer.Pin();
	}

	if (BrowserToSave.IsValid())
	{
		BrowserToSave->SaveSettings();
	}
}


void FContentBrowserSingleton::ExecuteRename(TSharedPtr<SWidget> PickerWidget)
{
	if (PickerWidget.IsValid())
	{
		if (PickerWidget->GetType() == FName(TEXT("SAssetPicker")))
		{
			TSharedPtr<SAssetPicker> AssetPicker = StaticCastSharedPtr<SAssetPicker>(PickerWidget);
			AssetPicker->ExecuteRenameCommand();
		}
		else if (PickerWidget->GetType() == FName(TEXT("SPathPicker")))
		{
			TSharedPtr<SPathPicker> PathPicker = StaticCastSharedPtr<SPathPicker>(PickerWidget);
			PathPicker->ExecuteRenameFolder();
		}
	}
}


void FContentBrowserSingleton::ExecuteAddFolder(TSharedPtr<SWidget> Widget)
{
	if (Widget.IsValid())
	{
		if (Widget->GetType() == FName(TEXT("SPathPicker")))
		{
			TSharedPtr<SPathPicker> PathPicker = StaticCastSharedPtr<SPathPicker>(Widget);
			PathPicker->ExecuteAddFolder();
		}
	}
}

void FContentBrowserSingleton::RefreshPathView(TSharedPtr<SWidget> Widget)
{
	if (Widget.IsValid())
	{
		if (Widget->GetType() == FName(TEXT("SPathPicker")))
		{
			TSharedPtr<SPathPicker> PathPicker = StaticCastSharedPtr<SPathPicker>(Widget);
			PathPicker->RefreshPathView();
		}
	}
}

bool FContentBrowserSingleton::IsShowingPrivateContent(const FStringView VirtualFolderPath)
{
	if (!ShowPrivateContentState.InvariantPaths->HasFiltering())
	{
		return false;
	}

	if (!ShowPrivateContentState.CachedVirtualPaths)
	{
		RebuildPrivateContentStateCache();
	}

	return ShowPrivateContentState.CachedVirtualPaths->PassesStartsWithFilter(VirtualFolderPath);
}

void FContentBrowserSingleton::RebuildPrivateContentStateCache()
{
	ShowPrivateContentState.CachedVirtualPaths.Reset();
	ShowPrivateContentState.CachedVirtualPaths = MakeShared<FPathPermissionList>();

	TArray<FString> InvariantAllowList = ShowPrivateContentState.InvariantPaths->GetAllowListEntries();
	for (const FString& InvariantPath : InvariantAllowList)
	{
		FName VirtualPath;
		IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual(FStringView(InvariantPath),
			VirtualPath);

		ShowPrivateContentState.CachedVirtualPaths->AddAllowListItem(TEXT("ContentBrowser"), VirtualPath);
	}
}

bool FContentBrowserSingleton::IsFolderShowPrivateContentToggleable(const FStringView VirtualFolderPath)
{
	if (IsFolderShowPrivateContentToggleableDelegate.IsBound())
	{
		return IsFolderShowPrivateContentToggleableDelegate.Execute(VirtualFolderPath);
	}

	return true;
}

const TSharedPtr<FPathPermissionList>& FContentBrowserSingleton::GetShowPrivateContentPermissionList()
{
	return ShowPrivateContentState.InvariantPaths;
}

const TSharedPtr<FPathPermissionList>& FContentBrowserSingleton::GetShowEpicInternalContentPermissionList()
{
	return ShowEpicInternalContentState.InvariantPaths;
}

void FContentBrowserSingleton::SetPrivateContentPermissionListDirty()
{
	ShowPrivateContentState.CachedVirtualPaths.Reset();
}

void FContentBrowserSingleton::RegisterIsFolderShowPrivateContentToggleableDelegate(FIsFolderShowPrivateContentToggleableDelegate InIsFolderShowPrivateContentToggleableDelegate)
{
	IsFolderShowPrivateContentToggleableDelegate = MoveTemp(InIsFolderShowPrivateContentToggleableDelegate);
}

void FContentBrowserSingleton::UnregisterIsFolderShowPrivateContentToggleableDelegate()
{
	IsFolderShowPrivateContentToggleableDelegate = FIsFolderShowPrivateContentToggleableDelegate();
}

FDelegateHandle FContentBrowserSingleton::RegisterOnFavoritesChangedHandler(FOnFavoritesChanged::FDelegate InOnFavoritesChanged)
{
	return OnFavoritesChanged.Add(InOnFavoritesChanged);
}

FDelegateHandle FContentBrowserSingleton::RegisterOnFavoritesChangedHandler(FSimpleDelegate InOnFavoritesChanged)
{
	return OnFavoritesChanged.AddLambda([InOnFavoritesChanged](const FContentBrowserItemPath&, bool)
	{
		InOnFavoritesChanged.ExecuteIfBound();
	});
}

void FContentBrowserSingleton::UnregisterOnFavoritesChangedDelegate(FDelegateHandle Handle)
{
	OnFavoritesChanged.Remove(Handle);
}

void FContentBrowserSingleton::RefreshFavoritesPathView()
{
	BroadcastFavoritesChanged(FContentBrowserItemPath(), /** Added */false);
}

void FContentBrowserSingleton::BroadcastFavoritesChanged(const FContentBrowserItemPath& ChangedItem, bool bAdded) const
{
	OnFavoritesChanged.Broadcast(ChangedItem, bAdded);
}

void FContentBrowserSingleton::PopulateConfigValues()
{
	const FString ContentBrowserSection = TEXT("ContentBrowser");

	// PluginSettings
	{
		const FString PluginSettingsName = TEXT("PluginSettings");
		TArray<FString> PluginSettingsStrings;
		GConfig->GetArray(*ContentBrowserSection, *PluginSettingsName, PluginSettingsStrings, GEditorIni);
		for (const FString& PluginSettingString : PluginSettingsStrings)
		{
			FContentBrowserPluginSettings PluginSetting;
			const TCHAR* Result = FContentBrowserPluginSettings::StaticStruct()->ImportText(*PluginSettingString, &PluginSetting, nullptr, PPF_None, GLog, PluginSettingsName);
			if (Result != nullptr)
			{
				PluginSettings.Emplace(PluginSetting);
			}
		}
	}
}

void FContentBrowserSingleton::GetContentBrowserSubMenu(UToolMenu* Menu, TSharedRef<FWorkspaceItem> ContentBrowserGroup)
{
	// Register the tab spawners for all content browsers
	FToolMenuSection& Section = Menu->AddSection("ContentBrowser");

	Section.AddMenuEntry("FocusContentBrowser",
		LOCTEXT("FocusContentBrowser_Label", "Focus Content Browser"),
		LOCTEXT("FocusContentBrowser_Desc", "Focuses the most recently active content browser tab."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateRaw(this, &FContentBrowserSingleton::FocusPrimaryContentBrowser, true), FCanExecuteAction())
	);

	auto AddSpawnerEntryToMenuSection = [](FToolMenuSection& InSection, TSharedPtr<FTabSpawnerEntry> InSpawnerNode, FName InTabID)
	{
		InSection.AddMenuEntry(
			InTabID,
			InSpawnerNode->GetDisplayName().IsEmpty() ? FText::FromName(InTabID) : InSpawnerNode->GetDisplayName(),
			InSpawnerNode->GetTooltipText(),
			FSlateIcon(),
			FGlobalTabmanager::Get()->GetUIActionForTabSpawnerMenuEntry(InSpawnerNode),
			EUserInterfaceActionType::Check
		);
	};

	for (int32 BrowserIdx = 0; BrowserIdx < UE_ARRAY_COUNT(ContentBrowserTabIDs); BrowserIdx++)
	{
		const FName TabID = ContentBrowserTabIDs[BrowserIdx];
		AddSpawnerEntryToMenuSection(Section, ContentBrowserTabs[BrowserIdx], TabID);
	}

}

FContentBrowserItemPath FContentBrowserSingleton::GetInitialPathToSaveAsset(const FContentBrowserItemPath& InPath)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");

	bool bPathIsWritable = true;
	if (InPath.GetVirtualPathName().IsNone() || !InPath.HasInternalPath())
	{
		bPathIsWritable = false;
	}
	else
	{
		bPathIsWritable = AssetToolsModule.Get().GetWritableFolderPermissionList()->PassesStartsWithFilter(InPath.GetInternalPathName(), /*bAllowParentPaths*/true);
	}

	FString AssetPath;
	if (bPathIsWritable)
	{
		AssetPath = InPath.GetInternalPathString();
	}
	else
	{
		// Try last path
		const FString DefaultFilesystemDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::NEW_ASSET);
		if (!DefaultFilesystemDirectory.IsEmpty() && FPackageName::TryConvertFilenameToLongPackageName(DefaultFilesystemDirectory, AssetPath))
		{
			if (AssetToolsModule.Get().GetWritableFolderPermissionList()->PassesStartsWithFilter(AssetPath, /*bAllowParentPaths*/true))
			{
				bPathIsWritable = true;
			}
		}

		if (!bPathIsWritable)
		{
			AssetPath.Reset();

			// Request default virtual paths, use first path that passes
			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
			TArray<FName> VirtualPaths;
			if (ContentBrowserModule.GetDefaultSelectedPathsDelegate().ExecuteIfBound(VirtualPaths) && VirtualPaths.Num() > 0)
			{
				for (const FName VirtualPath : VirtualPaths)
				{
					if (IContentBrowserDataModule::Get().GetSubsystem()->TryConvertVirtualPath(FNameBuilder(VirtualPath), AssetPath) == EContentBrowserPathType::Internal)
					{
						if (AssetToolsModule.Get().GetWritableFolderPermissionList()->PassesStartsWithFilter(AssetPath, /*bAllowParentPaths*/true))
						{
							break;
						}
					}
				
					AssetPath.Reset();
				}
			}
		}

		if (AssetPath.IsEmpty())
		{
			// No saved path, just use the game content root
			AssetPath = TEXT("/Game");
		}
	}

	// Remove trailing slash
	if (AssetPath.EndsWith(TEXT("/")) || AssetPath.EndsWith(TEXT("\\")))
	{
		AssetPath.LeftChopInline(1, EAllowShrinking::No);
	}

	return FContentBrowserItemPath(AssetPath, EContentBrowserPathType::Internal);
}

TArray<FString> FContentBrowserSingleton::GetAliasesForPath(const FSoftObjectPath& InPath) const
{
	TArray<FString> OutPaths;

	const TArray<FContentBrowserItemPath> Items = IContentBrowserDataModule::Get().GetSubsystem()->GetAliasesForPath(InPath);
	for (const FContentBrowserItemPath& Item : Items)
	{
		OutPaths.Add(Item.GetInternalPathString());
	}

	return OutPaths;
}

void FContentBrowserSingleton::AddCustomTextFilter(const FCustomTextFilterData& FilterData, bool bApplyFilter)
{
	if (!HasPrimaryContentBrowser())
	{
		ChooseNewPrimaryBrowser();
	}

	if (PrimaryContentBrowser.IsValid())
	{
		PrimaryContentBrowser.Pin()->AddCustomTextFilter(FilterData, bApplyFilter);
	}
}

void FContentBrowserSingleton::DeleteCustomTextFilterByLabel(const FText& FilterLabel)
{
	if (!HasPrimaryContentBrowser())
	{
		ChooseNewPrimaryBrowser();
	}

	if (PrimaryContentBrowser.IsValid())
	{
		PrimaryContentBrowser.Pin()->DeleteCustomTextFilterByLabel(FilterLabel);
	}
}

void FContentBrowserSingleton::ModifyCustomTextFilterByLabel(const FCustomTextFilterData& NewFilterData, const FText& FilterLabel)
{
	if (!HasPrimaryContentBrowser())
	{
		ChooseNewPrimaryBrowser();
	}

	if (PrimaryContentBrowser.IsValid())
	{
		PrimaryContentBrowser.Pin()->ModifyCustomTextFilterByLabel(NewFilterData, FilterLabel);
	}
}

/**
 * Checks if the Primary Content Browser's AssetView finished filtering
 * 
 * @return true if the AssetView is done filtering
 */
bool FContentBrowserSingleton::IsAssetViewDoneFiltering()
{
	bool isDoneFiltering = false;

	if (!HasPrimaryContentBrowser())
	{
		ChooseNewPrimaryBrowser();
	}

	if (PrimaryContentBrowser.IsValid())
	{
		isDoneFiltering = PrimaryContentBrowser.Pin()->IsAssetViewDoneFiltering();
	}

	return isDoneFiltering;
}

/**
 * Sets the search text for the Primary Content Browser
 * 
 * @params NewText - the text to put in the search box
 */
void FContentBrowserSingleton::SetSearchText(const FText& NewText)
{
	if (!HasPrimaryContentBrowser())
	{
		ChooseNewPrimaryBrowser();
	}

	if (PrimaryContentBrowser.IsValid())
	{
		if (TSharedPtr<SContentBrowser> PrimaryContentBrowserPinned = PrimaryContentBrowser.Pin())
		{
			PrimaryContentBrowserPinned->SetSearchText(NewText);
			PrimaryContentBrowserPinned->OnSearchBoxCommitted(NewText, ETextCommit::Type::Default);
		}
	}
}

void FContentBrowserSingleton::ToggleFolderFavorite(const TArray<FString>& FolderPaths)
{
	if (!HasPrimaryContentBrowser())
	{
		ChooseNewPrimaryBrowser();
	}

	if (PrimaryContentBrowser.IsValid())
	{
		PrimaryContentBrowser.Pin()->ToggleFolderFavorite(FolderPaths);
	}
}

void FContentBrowserSingleton::ForEachFavoriteFolder(TFunctionRef<void(const FContentBrowserItemPath&)> InDelegate)
{
	if (const UContentBrowserFavoriteProjectSettings* FavoriteSettings = UContentBrowserFavoriteProjectSettings::Get())
	{
		for (const FContentBrowserFavoriteItem& Item : FavoriteSettings->FavoriteItems)
		{
			InDelegate(FContentBrowserItemPath(Item.Path, EContentBrowserPathType::Internal));
		}
	}
}

void FContentBrowserSingleton::SetFilterLayout(const EFilterBarLayout InFilterBarLayout)
{
	if (!HasPrimaryContentBrowser())
	{
		ChooseNewPrimaryBrowser();
	}

	if (PrimaryContentBrowser.IsValid())
	{
		PrimaryContentBrowser.Pin()->SetFilterLayout(InFilterBarLayout);
	}
}

EFilterBarLayout FContentBrowserSingleton::GetFilterLayout()
{
	EFilterBarLayout FilterBarLayout = EFilterBarLayout::Vertical;

	if (!HasPrimaryContentBrowser())
	{
		ChooseNewPrimaryBrowser();
	}

	if (PrimaryContentBrowser.IsValid())
	{
		FilterBarLayout = PrimaryContentBrowser.Pin()->GetFilterLayout();
	}

	return FilterBarLayout;
}

TSharedPtr<SWidget> FContentBrowserSingleton::GetActiveFilterContainer()
{
	if (!HasPrimaryContentBrowser())
	{
		ChooseNewPrimaryBrowser();
	}

	if (PrimaryContentBrowser.IsValid())
	{
		return PrimaryContentBrowser.Pin()->GetActiveFilterContainer();
	}

	return nullptr;
}

void FContentBrowserSingleton::RegisterContentSourceFactory(const FName& InContentSourceName, const FContentSourceFactory& InContentSourceFactory)
{
	ContentSourceFactories.Emplace(InContentSourceName, InContentSourceFactory);
	ContentSourceFactoriesChangedEvent.Broadcast();
}

void FContentBrowserSingleton::UnregisterContentSourceFactory(const FName& InContentSourceName)
{
	ContentSourceFactories.Remove(InContentSourceName);
	ContentSourceFactoriesChangedEvent.Broadcast();
}

void FContentBrowserSingleton::ForEachContentSourceFactory(const TFunctionRef<void(const FName&, const FContentSourceFactory&)>& InFunction)
{
	for(const TPair<FName, FContentSourceFactory>& Pair : ContentSourceFactories)
	{
		InFunction(Pair.Key, Pair.Value);
	}
}

IContentBrowserSingleton::FOnContentSourceFactoriesChanged& FContentBrowserSingleton::OnContentSourceFactoriesChanged()
{
	return ContentSourceFactoriesChangedEvent;
}

#undef LOCTEXT_NAMESPACE
