// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserItemPath.h"
#include "HAL/Platform.h"
#include "IContentBrowserSingleton.h"
#include "Internationalization/Text.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

#include "ContentBrowserSingleton.generated.h"

struct FContentBrowserFavoriteItem;
class FCollectionAssetRegistryBridge;
class FMenuBuilder;
class FPathPermissionList;
class FSpawnTabArgs;
class FTabManager;
class FViewport;
class FWorkspaceItem;
class SContentBrowser;
class SDockTab;
class SWidget;
class UClass;
class UFactory;
class UObject;
class UToolMenu;
struct FAssetData;
struct FContentBrowserItem;
struct FTabSpawnerEntry;

#define MAX_CONTENT_BROWSERS 4

USTRUCT()
struct FContentBrowserPluginSettings
{
	GENERATED_BODY()

	UPROPERTY()
	FName PluginName;

	/** Used to control the order of plugin root folders in the path view. A higher priority sorts higher in the list. Game and Engine folders are priority 1.0 */
	UPROPERTY()
	float RootFolderSortPriority;

	FContentBrowserPluginSettings()
		: RootFolderSortPriority(0.f)
	{}
};

struct FShowPrivateContentState
{
	TSharedPtr<FPathPermissionList> InvariantPaths;
	TSharedPtr<FPathPermissionList> CachedVirtualPaths;
};

/**
 * Content browser module singleton implementation class
 */
class FContentBrowserSingleton : public IContentBrowserSingleton
{
public:
	/** Constructor, Destructor */
	FContentBrowserSingleton();
	virtual ~FContentBrowserSingleton();

	// IContentBrowserSingleton interface
	virtual TSharedRef<SWidget> CreateContentBrowser( const FName InstanceName, TSharedPtr<SDockTab> ContainingTab, const FContentBrowserConfig* ContentBrowserConfig ) override;
	virtual TSharedRef<SWidget> CreateAssetPicker(const FAssetPickerConfig& AssetPickerConfig) override;
	virtual TSharedPtr<SWidget> GetAssetPickerSearchBox(const TSharedRef<SWidget>& AssetPickerWidget) override;
	virtual TSharedRef<SWidget> CreatePathPicker(const FPathPickerConfig& PathPickerConfig) override;
	virtual TSharedRef<SWidget> CreateCollectionPicker(const FCollectionPickerConfig& CollectionPickerConfig) override;
	virtual TSharedRef<SWidget> CreateContentBrowserDrawer(const FContentBrowserConfig& ContentBrowserConfig, TFunction<TSharedPtr<SDockTab>()> InOnGetTabForDrawer) override;
	virtual void CreateOpenAssetDialog(const FOpenAssetDialogConfig& OpenAssetConfig, const FOnAssetsChosenForOpen& OnAssetsChosenForOpen, const FOnAssetDialogCancelled& OnAssetDialogCancelled) override;
	virtual TArray<FAssetData> CreateModalOpenAssetDialog(const FOpenAssetDialogConfig& InConfig) override;
	virtual void CreateSaveAssetDialog(const FSaveAssetDialogConfig& SaveAssetConfig, const FOnObjectPathChosenForSave& OnAssetNameChosenForSave, const FOnAssetDialogCancelled& OnAssetDialogCancelled) override;
	virtual FString CreateModalSaveAssetDialog(const FSaveAssetDialogConfig& SaveAssetConfig) override;
	virtual bool HasPrimaryContentBrowser() const override;
	virtual bool SetPrimaryContentBrowser(FName InstanceName) override;
	virtual void FocusPrimaryContentBrowser(bool bFocusSearch) override;
	virtual void FocusContentBrowserSearchField(TSharedPtr<SWidget> ContentBrowserWidget) override;
	virtual void CreateNewAsset(const FString& DefaultAssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory) override;
	virtual void SyncBrowserToAssets(const TArray<struct FAssetData>& AssetDataList, bool bAllowLockedBrowsers = false, bool bFocusContentBrowser = true, const FName& InstanceName = FName(), bool bNewSpawnBrowser = false) override;
	virtual void SyncBrowserToAssets(const TArray<UObject*>& AssetList, bool bAllowLockedBrowsers = false, bool bFocusContentBrowser = true, const FName& InstanceName = FName(), bool bNewSpawnBrowser = false) override;
	virtual void SyncBrowserToFolders(const TArray<FString>& FolderList, bool bAllowLockedBrowsers = false, bool bFocusContentBrowser = true, const FName& InstanceName = FName(), bool bNewSpawnBrowser = false) override;
	virtual void SyncBrowserToCollections(const TArray<FCollectionRef>& CollectionsToSync, bool bAllowLockedBrowsers = false, bool bFocusContentBrowser = true, const FName& InstanceName = FName(), bool bNewSpawnBrowser = false) override;
	virtual void SyncBrowserToItems(const TArray<FContentBrowserItem>& ItemsToSync, bool bAllowLockedBrowsers = false, bool bFocusContentBrowser = true, const FName& InstanceName = FName(), bool bNewSpawnBrowser = false) override;
	virtual void SyncBrowserTo(const FContentBrowserSelection& ItemSelection, bool bAllowLockedBrowsers = false, bool bFocusContentBrowser = true, const FName& InstanceName = FName(), bool bNewSpawnBrowser = false) override;
	virtual void GetSelectedAssets(TArray<FAssetData>& SelectedAssets) override;
	virtual void GetAllSelectedAssets(TArray<FAssetData>& SelectedAssets) override;
	virtual void GetSelectedFolders(TArray<FString>& SelectedFolders) override;
	virtual void GetSelectedPathViewFolders(TArray<FString>& SelectedFolders) override;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual FString GetCurrentPath(const EContentBrowserPathType PathType) override;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	virtual FContentBrowserItemPath GetCurrentPath() override;
	virtual void CaptureThumbnailFromViewport(FViewport* InViewport, TArray<FAssetData>& SelectedAssets) override;
	virtual void SetSelectedPaths(const TArray<FString>& FolderPaths, bool bNeedsRefresh = false, bool bPathsAreVirtual = false) override;
	virtual void ForceShowPluginContent(bool bEnginePlugin) override;
	virtual FContentBrowserItemPath GetInitialPathToSaveAsset(const FContentBrowserItemPath& InPath) override;
	virtual void SaveContentBrowserSettings(TSharedPtr<SWidget> ContentBrowserWidget) override;
	virtual void ExecuteRename(TSharedPtr<SWidget> PickerWidget) override;
	virtual void ExecuteAddFolder(TSharedPtr<SWidget> PathPickerWidget) override;
	virtual void RefreshPathView(TSharedPtr<SWidget> PathPickerWidget) override;
	virtual bool IsShowingPrivateContent(const FStringView VirtualFolderPath) override;
	virtual bool IsFolderShowPrivateContentToggleable(const FStringView VirtualFolderPath) override;
	virtual const TSharedPtr<FPathPermissionList>& GetShowPrivateContentPermissionList() override;
	virtual const TSharedPtr<FPathPermissionList>& GetShowEpicInternalContentPermissionList() override;
	virtual void SetPrivateContentPermissionListDirty() override;
	virtual void RegisterIsFolderShowPrivateContentToggleableDelegate(FIsFolderShowPrivateContentToggleableDelegate InIsFolderShowPrivateContentToggleableDelegate) override;
	virtual void UnregisterIsFolderShowPrivateContentToggleableDelegate() override;
	virtual FDelegateHandle RegisterOnFavoritesChangedHandler(FOnFavoritesChanged::FDelegate InOnFavoritesChanged) override;
	UE_DEPRECATED(5.7, "Use the overload that takes in an FOnFavoritesChanged delegate")
	virtual FDelegateHandle RegisterOnFavoritesChangedHandler(FSimpleDelegate OnFavoritesChanged) override;
	virtual void UnregisterOnFavoritesChangedDelegate(FDelegateHandle Handle) override;
	virtual void RefreshFavoritesPathView() override;
	virtual TArray<FString> GetAliasesForPath(const FSoftObjectPath& InPath) const override;
	virtual void AddCustomTextFilter(const FCustomTextFilterData& FilterData, bool bApplyFilter) override;
	virtual void DeleteCustomTextFilterByLabel(const FText& FilterLabel) override;
	virtual void ModifyCustomTextFilterByLabel(const FCustomTextFilterData& NewFilterData, const FText& FilterLabel) override;
	virtual bool IsAssetViewDoneFiltering() override;
	virtual void SetSearchText(const FText& NewText) override;
	virtual void ToggleFolderFavorite(const TArray<FString>& FolderPaths) override;
	virtual void ForEachFavoriteFolder(TFunctionRef<void(const FContentBrowserItemPath&)> InDelegate) override;
	virtual void SetFilterLayout(EFilterBarLayout InFilterBarLayout) override;
	virtual TSharedPtr<SWidget> GetActiveFilterContainer() override;
	virtual EFilterBarLayout GetFilterLayout() override;
	virtual void DockContentBrowserDrawer() override;
	virtual void RegisterContentSourceFactory(const FName& InContentSourceName, const FContentSourceFactory& InContentSourceFactory) override;
	virtual void UnregisterContentSourceFactory(const FName& InContentSourceName) override;
	virtual void ForEachContentSourceFactory(const TFunctionRef<void(const FName&, const FContentSourceFactory&)>& InFunction) override;
	virtual FOnContentSourceFactoriesChanged& OnContentSourceFactoriesChanged() override;

	/** Broadcast that the favorites have changed. */
	void BroadcastFavoritesChanged(const FContentBrowserItemPath& ChangedItem, bool bAdded) const;

	/** Gets the content browser singleton as a FContentBrowserSingleton */
	static FContentBrowserSingleton& Get();
	
	/** Sets the current primary content browser. */
	void SetPrimaryContentBrowser(const TSharedRef<SContentBrowser>& NewPrimaryBrowser);

	/** Notifies the singleton that a browser was closed */
	void ContentBrowserClosed(const TSharedRef<SContentBrowser>& ClosedBrowser);

	/** Gets the settings for the plugin with the specified name */
	const FContentBrowserPluginSettings& GetPluginSettings(FName PluginName) const;
	
	/** Opens window to edit a favorite item and close old one if any */
	void OpenEditFavoriteItemWindow(const FContentBrowserFavoriteItem& InItem);

private:
	/** Callback for when favorite item has been edited */
	void OnFavoriteItemEdited(const FContentBrowserFavoriteItem& InItem, bool bInAnyChanges);
	
	/** Util to get or create the content browser that should be used by the various Sync functions */
	TSharedPtr<SContentBrowser> FindContentBrowserToSync(bool bAllowLockedBrowsers, const FName& InstanceName = FName(), bool bNewSpawnBrowser = false);

	/** Shared code to open an asset dialog window with a config */
	void SharedCreateAssetDialogWindow(const TSharedRef<class SAssetDialog>& AssetDialog, const FSharedAssetDialogConfig& InConfig, bool bModal) const;

	/** 
	 * Delegate handlers
	 **/
	void OnEditorLoadSelectedAssetsIfNeeded();

	/** Sets the primary content browser to the next valid browser in the list of all browsers */
	void ChooseNewPrimaryBrowser();

	/** Gives focus to the specified content browser */
	void FocusContentBrowser(const TSharedPtr<SContentBrowser>& BrowserToFocus);

	/** Summons a new content browser */
	FName SummonNewBrowser(bool bAllowLockedBrowsers = false, TSharedPtr<FTabManager> SpecificTabManager = nullptr);

	/** Handler for a request to spawn a new content browser tab */
	TSharedRef<SDockTab> SpawnContentBrowserTab( const FSpawnTabArgs& SpawnTabArgs, int32 BrowserIdx );

	/** Handler for a request to spawn a new content browser tab */
	FText GetContentBrowserTabLabel(int32 BrowserIdx);

	/** Returns true if this content browser is locked (can be used even when closed) */
	bool IsLocked(const FName& InstanceName) const;

	/** Returns a localized name for the tab/menu entry with index */
	static FText GetContentBrowserLabelWithIndex( int32 BrowserIdx );

	/** Populates properties that come from ini files */
	void PopulateConfigValues();

	/** Creates the Content Browser submenu in the Level Editor Toolbar Add menu */
	void GetContentBrowserSubMenu(UToolMenu* Menu, TSharedRef<FWorkspaceItem> ContentBrowserGroup);

	void ExtendContentBrowserTabContextMenu(FMenuBuilder& InMenuBuilder);

	/** Rebuilds the private content state cache based off of the currently registered Invariant Path permission list */
	void RebuildPrivateContentStateCache();

public:
	/** The tab identifier/instance name for content browser tabs */
	FName ContentBrowserTabIDs[MAX_CONTENT_BROWSERS];

private:

	FIsFolderShowPrivateContentToggleableDelegate IsFolderShowPrivateContentToggleableDelegate;

	TArray<TWeakPtr<SContentBrowser>> AllContentBrowsers;

	TWeakPtr<SContentBrowser> ContentBrowserDrawer;

	TFunction<TSharedPtr<SDockTab>()> OnGetTabForDrawer;

	TMap<FName, TWeakPtr<FTabManager>> BrowserToLastKnownTabManagerMap;

	TWeakPtr<SContentBrowser> PrimaryContentBrowser;

	TSharedRef<FCollectionAssetRegistryBridge> CollectionAssetRegistryBridge;

	TArray<FContentBrowserPluginSettings> PluginSettings;

	TArray<TSharedPtr<FTabSpawnerEntry>> ContentBrowserTabs;

	/** An incrementing int32 which is used when making unique settings strings */
	int32 SettingsStringID;

	FShowPrivateContentState ShowPrivateContentState;
	FShowPrivateContentState ShowEpicInternalContentState;

	FOnFavoritesChanged OnFavoritesChanged;
	
	/** Window dialog to edit a favorite item */
	TWeakPtr<SWindow> EditFavoriteItemWindowWeak;

	TMap<FName, FContentSourceFactory> ContentSourceFactories;
	FOnContentSourceFactoriesChanged ContentSourceFactoriesChangedEvent;
};
