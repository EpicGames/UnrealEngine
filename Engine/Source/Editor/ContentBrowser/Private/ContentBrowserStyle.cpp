// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserStyle.h"

#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"

namespace UE::ContentBrowser::Private
{
	FName FContentBrowserStyle::StyleName("ContentBrowser");

	FContentBrowserStyle& FContentBrowserStyle::Get()
	{
		static FContentBrowserStyle Instance;
		return Instance;
	}

	const FName& FContentBrowserStyle::GetStyleSetName() const
	{
		return StyleName;
	}

	FContentBrowserStyle::FContentBrowserStyle()
		: FSlateStyleSet(StyleName)
	{
		SetParentStyleName(FAppStyle::GetAppStyleSetName());

		// Init from ParentStyle
		{
			const ISlateStyle* ParentStyle = GetParentStyle();

			DefaultForeground = ParentStyle->GetSlateColor("DefaultForeground");
			InvertedForeground = ParentStyle->GetSlateColor("InvertedForeground");
			SelectionColor = ParentStyle->GetSlateColor("SelectionColor");
			SelectionColor_Pressed = ParentStyle->GetSlateColor("SelectionColor_Pressed");

			NormalText = ParentStyle->GetWidgetStyle<FTextBlockStyle>("NormalText");
			Button = ParentStyle->GetWidgetStyle<FButtonStyle>("Button");
		}

		SetContentRoot( FPaths::EngineContentDir() / TEXT("Editor/Slate") );
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		// Originally in FStarshipEditorStyle::FStyle::SetupContentBrowserStyle()
		// @todo: Remove, deprecate or redirect from FStarshipEditorStyle::FStyle::SetupContentBrowserStyle()

		// Tab and menu icon
		Set("ContentBrowser.TabIcon", new IMAGE_BRUSH_SVG("Starship/Common/ContentBrowser", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.PrivateContentEdit", new IMAGE_BRUSH("Icons/hiererchy_16x", CoreStyleConstants::Icon16x16));

		// Sources View
		Set("ContentBrowser.SourceTitleFont", DEFAULT_FONT("Regular", 12));

		Set("ContentBrowser.SourceTreeItemFont", FStarshipCoreStyle::GetCoreStyle().GetFontStyle("NormalFont"));
		Set("ContentBrowser.SourceTreeRootItemFont", FStarshipCoreStyle::GetCoreStyle().GetFontStyle("NormalFont"));

		Set("ContentBrowser.BreadcrumbPathPickerFolder",	new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-closed", CoreStyleConstants::Icon16x16));

		Set("ContentBrowser.AssetTreeFolderClosed", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-closed", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetTreeFolderOpen", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-open", CoreStyleConstants::Icon16x16));

		Set("ContentBrowser.AssetTreeFolderClosedVirtual", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-virtual-closed", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetTreeFolderOpenVirtual",	new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-virtual-open", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetTreeFolderOpenDeveloper", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/folder-developer-open", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetTreeFolderClosedDeveloper",	new IMAGE_BRUSH_SVG("Starship/ContentBrowser/folder-developer", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetTreeFolderOpenCode", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/folder-code-open", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetTreeFolderClosedCode", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/folder-code", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetTreeFolderOpenPluginRoot", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/folder-pluginroot-open", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetTreeFolderClosedPluginRoot", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/folder-pluginroot", CoreStyleConstants::Icon16x16));

		Set("ContentBrowser.DefaultFolderColor", FStyleColors::AccentFolder);

		FSlateBrush* AssetTreeHeaderBrush = nullptr; 
		FSlateBrush* VerticalFilterViewHeaderBrush = nullptr;
		FSlateBrush* VerticalFilterViewBodyBrush = nullptr;

		FSlateBrush* AssetTreeBodyBrush = nullptr; 

		if (Editor::ContentBrowser::IsNewStyleEnabled())
		{
			AssetTreeHeaderBrush = new FSlateColorBrush(FStyleColors::Panel);
			VerticalFilterViewHeaderBrush = new FSlateColorBrush(FStyleColors::Panel);
			VerticalFilterViewBodyBrush = new FSlateColorBrush(FStyleColors::Panel);
			AssetTreeBodyBrush = new FSlateColorBrush(FStyleColors::Recessed);
		}
		else
		{
			AssetTreeHeaderBrush = new FSlateColorBrush(FStyleColors::Header);
			VerticalFilterViewHeaderBrush = new FSlateColorBrush(FStyleColors::Header);
			VerticalFilterViewBodyBrush = new FSlateColorBrush(FStyleColors::Header);
			AssetTreeBodyBrush = new FSlateColorBrush(FStyleColors::Recessed);
		}

		Set("ContentBrowser.AssetTreeHeaderBrush", AssetTreeHeaderBrush);
		Set("ContentBrowser.AssetTreeBodyBrush", AssetTreeBodyBrush);

		FExpandableAreaStyle AssetTreeExpandableAreaStyle = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FExpandableAreaStyle>("ExpandableArea");

		Set("ContentBrowser.AssetTreeExpandableArea", AssetTreeExpandableAreaStyle);

		// Asset list view
		Set("ContentBrowser.AssetListViewNameFont", DEFAULT_FONT("Regular", 12));
		Set("ContentBrowser.AssetListViewNameFontDirty", DEFAULT_FONT("Bold", 12));
		Set("ContentBrowser.AssetListViewClassFont", DEFAULT_FONT("Light", 10));

		// Asset picker
		Set("ContentBrowser.NoneButton",
			FButtonStyle(Button)
			.SetNormal(FSlateNoResource())
			.SetHovered(BOX_BRUSH("Common/Selection", 8.0f / 32.0f, SelectionColor))
			.SetPressed(BOX_BRUSH("Common/Selection", 8.0f / 32.0f, SelectionColor_Pressed))
		);
		Set("ContentBrowser.NoneButtonText",
			FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 12))
			.SetColorAndOpacity(FLinearColor::White)
		);

		// Tile view
		Set("ContentBrowser.AssetTileViewNameFont", DEFAULT_FONT("Regular", 9));
		Set("ContentBrowser.AssetTileViewNameFontSmall", DEFAULT_FONT("VeryLight", 8));
		Set("ContentBrowser.AssetTileViewNameFontVerySmall", DEFAULT_FONT("VeryLight", 7));
		Set("ContentBrowser.AssetTileViewNameFontDirty", FStyleFonts::Get().SmallBold);

		Set("ContentBrowser.AssetListView.ColumnListTableRow",
			FTableRowStyle()
			.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::Recessed))
			.SetEvenRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::SelectHover))
			.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Background))
			.SetOddRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::SelectHover))
			.SetSelectorFocusedBrush(BORDER_BRUSH("Common/Selector", FMargin(4.0f / 16.0f), FStyleColors::Select))
			.SetActiveBrush(IMAGE_BRUSH("Common/Selection", CoreStyleConstants::Icon8x8, FStyleColors::Select))
			.SetActiveHoveredBrush(IMAGE_BRUSH("Common/Selection", CoreStyleConstants::Icon8x8, FStyleColors::Select))
			.SetInactiveBrush(IMAGE_BRUSH("Common/Selection", CoreStyleConstants::Icon8x8, FStyleColors::SelectInactive))
			.SetInactiveHoveredBrush(IMAGE_BRUSH("Common/Selection", CoreStyleConstants::Icon8x8, FStyleColors::SelectInactive))
			.SetTextColor(DefaultForeground)
			.SetSelectedTextColor(InvertedForeground)
		);

		Set("ContentBrowser.AssetListView.TileTableRow",
			FTableRowStyle()
			.SetEvenRowBackgroundBrush(FSlateNoResource())
			.SetEvenRowBackgroundHoveredBrush(FSlateNoResource())
			.SetOddRowBackgroundBrush(FSlateNoResource())
			.SetOddRowBackgroundHoveredBrush(FSlateNoResource())
			.SetSelectorFocusedBrush(FSlateNoResource())
			.SetActiveBrush(FSlateNoResource())
			.SetActiveHoveredBrush(FSlateNoResource())
			.SetInactiveBrush(FSlateNoResource())
			.SetInactiveHoveredBrush(FSlateNoResource())
			.SetTextColor(DefaultForeground)
			.SetSelectedTextColor(DefaultForeground)
		);

		Set("ContentBrowser.TileViewTooltip.ToolTipBorder", new FSlateColorBrush(FLinearColor::Black));
		Set("ContentBrowser.TileViewTooltip.NonContentBorder", new BOX_BRUSH("Docking/TabContentArea", FMargin(4.0f / 16.0f)));
		Set("ContentBrowser.TileViewTooltip.ContentBorder", new FSlateColorBrush(FStyleColors::Panel));
		Set("ContentBrowser.TileViewTooltip.PillBorder",	new FSlateRoundedBoxBrush(FStyleColors::Transparent, 10.0f, FStyleColors::White, 1.0f));
		Set("ContentBrowser.TileViewTooltip.UnsupportedAssetPillBorder",	new FSlateRoundedBoxBrush(FStyleColors::Transparent, 10.0f, FStyleColors::Warning, 1.0f));
		Set("ContentBrowser.TileViewTooltip.NameFont", DEFAULT_FONT("Regular", 12));
		Set("ContentBrowser.TileViewTooltip.AssetUserDescriptionFont", DEFAULT_FONT("Regular", 12));

		// Columns view
		Set("ContentBrowser.SortUp", new IMAGE_BRUSH("Common/SortUpArrow", CoreStyleConstants::Icon8x4));
		Set("ContentBrowser.SortDown", new IMAGE_BRUSH("Common/SortDownArrow", CoreStyleConstants::Icon8x4));

		// Filter List - These are aliases for SBasicFilterBar styles in StarshipCoreStyle for backwards compatibility
		Set("ContentBrowser.FilterImage", new IMAGE_BRUSH_SVG("Starship/CoreWidgets/FilterBar/FilterColorSegment", FVector2D(8, 22)));
		Set("ContentBrowser.FilterBackground", new FSlateRoundedBoxBrush(FStyleColors::Secondary, 3.0f));

		Set("ContentBrowser.FilterButton", FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FCheckBoxStyle>("FilterBar.FilterButton"));
		Set("ContentBrowser.FilterToolBar",	FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("FilterBar.FilterToolBar"));

		// Sources view
		Set("ContentBrowser.Sources.Paths", new IMAGE_BRUSH("ContentBrowser/Sources_Paths_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.Sources.Collections", new IMAGE_BRUSH("ContentBrowser/Sources_Collections_Standard_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.Sources.Collections.Compact", new IMAGE_BRUSH("ContentBrowser/Sources_Collections_Compact_16x", CoreStyleConstants::Icon16x16));

		// Asset tags (common)
		Set("ContentBrowser.AssetTagBackground", new FSlateRoundedBoxBrush(FStyleColors::White, 2.0f));

		// Asset tags (standard)
		Set("ContentBrowser.AssetTagButton", FCheckBoxStyle()
			.SetUncheckedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2D(14.0f, 28.0f)))
			.SetUncheckedHoveredImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2D(14.0f, 28.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetUncheckedPressedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2D(14.0f, 28.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetUndeterminedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2D(14.0f, 28.0f)))
			.SetUndeterminedHoveredImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2D(14.0f, 28.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetUndeterminedPressedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2D(14.0f, 28.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetCheckedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2D(14.0f, 28.0f)))
			.SetCheckedHoveredImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2D(14.0f, 28.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetCheckedPressedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2D(14.0f, 28.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetPadding(0.0f)
		);

		Set("ContentBrowser.AssetTagNamePadding", FMargin(4.0f));
		Set("ContentBrowser.AssetTagCountPadding", FMargin(4.0f));

		// Asset tags (compact)
		Set("ContentBrowser.AssetTagButton.Compact", FCheckBoxStyle()
			.SetUncheckedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2D(10.0f, 20.0f)))
			.SetUncheckedHoveredImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2D(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetUncheckedPressedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2D(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetUndeterminedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2D(10.0f, 20.0f)))
			.SetUndeterminedHoveredImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2D(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetUndeterminedPressedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2D(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetCheckedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2D(10.0f, 20.0f)))
			.SetCheckedHoveredImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2D(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetCheckedPressedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2D(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetPadding(0.0f)
		);

		Set("ContentBrowser.AssetTagNamePadding.Compact", FMargin(2.0f));
		Set("ContentBrowser.AssetTagCountPadding.Compact", FMargin(2.0f));


		Set("ContentBrowser.PrimitiveCustom", new IMAGE_BRUSH("ContentBrowser/ThumbnailCustom", CoreStyleConstants::Icon32x32));
		Set("ContentBrowser.PrimitiveSphere", new IMAGE_BRUSH("ContentBrowser/ThumbnailSphere", CoreStyleConstants::Icon32x32));
		Set("ContentBrowser.PrimitiveCube", new IMAGE_BRUSH("ContentBrowser/ThumbnailCube", CoreStyleConstants::Icon32x32));
		Set("ContentBrowser.PrimitivePlane", new IMAGE_BRUSH("ContentBrowser/ThumbnailPlane", CoreStyleConstants::Icon32x32));
		Set("ContentBrowser.PrimitiveCylinder", new IMAGE_BRUSH("ContentBrowser/ThumbnailCylinder", CoreStyleConstants::Icon32x32));
		Set("ContentBrowser.ResetPrimitiveToDefault", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Undo", CoreStyleConstants::Icon20x20));

		Set("ContentBrowser.TopBar.Font", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 11))
			.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetHighlightColor(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetShadowOffset(FVector2D(1, 1))
			.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f)));

		Set("ContentBrowser.ClassFont", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 7)));


		Set("ContentBrowser.AddContent", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/ContentPack", CoreStyleConstants::Icon20x20));
		Set("ContentBrowser.ImportPackage", new IMAGE_BRUSH("Icons/icon_Import_40x", CoreStyleConstants::Icon25x25));

		// Asset Context Menu
		Set("ContentBrowser.AssetActions", new CORE_IMAGE_BRUSH("Icons/icon_tab_Tools_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetActions.Edit", new IMAGE_BRUSH("Icons/Edit/icon_Edit_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetActions.Delete", new IMAGE_BRUSH("Icons/icon_delete_16px", CoreStyleConstants::Icon16x16, FLinearColor(0.4f, 0.5f, 0.7f, 1.0f)));
		Set("ContentBrowser.AssetActions.Rename", new IMAGE_BRUSH("Icons/Icon_Asset_Rename_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetActions.Duplicate", new IMAGE_BRUSH("Icons/Edit/icon_Edit_Duplicate_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetActions.OpenSourceLocation", new IMAGE_BRUSH("Icons/icon_Asset_Open_Source_Location_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetActions.OpenInExternalEditor", new IMAGE_BRUSH("Icons/icon_Asset_Open_In_External_Editor_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetActions.PublicAssetToggle", new IMAGE_BRUSH("Icons/hiererchy_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetActions.ReimportAsset", new IMAGE_BRUSH("Icons/icon_TextureEd_Reimport_40x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetActions.GoToCodeForAsset", new IMAGE_BRUSH("GameProjectDialog/feature_code_32x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetActions.FindAssetInWorld", new IMAGE_BRUSH("/Icons/icon_Genericfinder_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetActions.CreateThumbnail", new IMAGE_BRUSH("Icons/icon_Asset_Create_Thumbnail_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetActions.DeleteThumbnail", new IMAGE_BRUSH("Icons/icon_Asset_Delete_Thumbnail_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetActions.GenericFind", new IMAGE_BRUSH("Icons/icon_Genericfinder_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetLocalization", new IMAGE_BRUSH("Icons/icon_localization_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetActions.VolumeTexture", new IMAGE_BRUSH_SVG("Starship/AssetActions/volume-texture", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetActions.RemoveVertexColors", new IMAGE_BRUSH_SVG("Starship/AssetActions/remove-vertex-colors", CoreStyleConstants::Icon16x16));

		// ContentBrowser Commands Icons
		Set("ContentBrowser.AssetViewCopyObjectPath", new IMAGE_BRUSH_SVG("../../Slate/Starship/Common/Copy", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetViewCopyPackageName", new IMAGE_BRUSH_SVG("../../Slate/Starship/Common/Copy", CoreStyleConstants::Icon16x16));

		// Misc
		/** Should be moved, shared */ Set("ContentBrowser.ThumbnailShadow", new BOX_BRUSH("ContentBrowser/ThumbnailShadow", FMargin(4.0f / 64.0f)));



		Set("ContentBrowser.ColumnViewAssetIcon", new IMAGE_BRUSH("Icons/doc_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.FileImage", new CORE_IMAGE_BRUSH_SVG("Starship/Common/file", CoreStyleConstants::Icon16x16));

		Set("ContentBrowser.ColumnViewFolderIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-closed", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.ColumnViewDeveloperFolderIcon", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/folder-developer", CoreStyleConstants::Icon16x16));

		Set("ContentBrowser.ListViewFolderIcon", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/folder", FVector2D(64, 64)));
		Set("ContentBrowser.ListViewVirtualFolderIcon", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/FolderLargeVirtual", FVector2D(64, 64)));
		Set("ContentBrowser.ListViewVirtualFolderShadow", new IMAGE_BRUSH("Starship/ContentBrowser/FolderLargeVirtualShadow", FVector2D(256, 256)));
		Set("ContentBrowser.ListViewDeveloperFolderIcon", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/FolderLargeDeveloper", FVector2D(64, 64)));
		Set("ContentBrowser.ListViewCodeFolderIcon", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/FolderLargeCode", FVector2D(64, 64)));
		Set("ContentBrowser.ListViewPluginFolderIcon", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/FolderLargePlugin", FVector2D(64, 64)));

		Set("ContentBrowser.AssetTileItem.FolderAreaHoveredBackground", new FSlateRoundedBoxBrush(FStyleColors::Secondary, 4.0f));
		Set("ContentBrowser.AssetTileItem.FolderAreaSelectedBackground", new FSlateRoundedBoxBrush(FStyleColors::Secondary, 4.0f));
		Set("ContentBrowser.AssetTileItem.FolderAreaSelectedHoverBackground", new FSlateRoundedBoxBrush(FStyleColors::Secondary, 4.0f));
		Set("ContentBrowser.AssetTileItem.ThumbnailAreaBackground", new FSlateRoundedBoxBrush(FStyleColors::Recessed, 4.0f));
		Set("ContentBrowser.AssetTileItem.NameAreaBackground", new FSlateRoundedBoxBrush(FStyleColors::Secondary, FVector4(0.0f, 0.0f, 4.0f, 4.0f)));
		Set("ContentBrowser.AssetTileItem.NameAreaHoverBackground", new FSlateRoundedBoxBrush(FStyleColors::Hover, FVector4(0.0f, 0.0f, 4.0f, 4.0f)));
		Set("ContentBrowser.AssetTileItem.NameAreaSelectedBackground", new FSlateRoundedBoxBrush(FStyleColors::Primary, FVector4(0.0f, 0.0f, 4.0f, 4.0f)));
		Set("ContentBrowser.AssetTileItem.NameAreaSelectedHoverBackground", new FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, FVector4(0.0f, 0.0f, 4.0f, 4.0f)));

		{
			FLinearColor TransparentPrimary = FStyleColors::Primary.GetSpecifiedColor();
			TransparentPrimary.A = 0.0f;
			Set("ContentBrowser.AssetTileItem.SelectedBorder", new FSlateRoundedBoxBrush(TransparentPrimary, 4.0f, FStyleColors::Primary, 1.0f));

			FLinearColor TransparentPrimaryHover = FStyleColors::PrimaryHover.GetSpecifiedColor();
			TransparentPrimaryHover.A = 0.0f;
			Set("ContentBrowser.AssetTileItem.SelectedHoverBorder", new FSlateRoundedBoxBrush(TransparentPrimaryHover, 4.0f, FStyleColors::PrimaryHover, 1.0f));

			FLinearColor TransparentHover = FStyleColors::Hover.GetSpecifiedColor();
			TransparentHover.A = 0.0f;
			Set("ContentBrowser.AssetTileItem.HoverBorder", new FSlateRoundedBoxBrush(TransparentHover, 4.0f, FStyleColors::Hover, 1.0f));
		}

		Set("ContentBrowser.AssetTileItem.DropShadow", new BOX_BRUSH("Starship/ContentBrowser/drop-shadow", FMargin(4.0f / 64.0f)));
		Set("ContentBrowser.FolderItem.DropShadow", new IMAGE_BRUSH("Starship/ContentBrowser/folder-drop-shadow", FVector2D(256, 256)));

		Set("ContentBrowser.ShowSourcesView", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/file-tree", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.HideSourcesView", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/file-tree-open", CoreStyleConstants::Icon16x16));

		Set("ContentBrowser.DirectoryUp", new IMAGE_BRUSH("Icons/icon_folder_up_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.PathPickerButton", new IMAGE_BRUSH("Icons/ellipsis_12x", CoreStyleConstants::Icon12x12, FLinearColor::Black));

		Set("ContentBrowser.ContentDirty", new IMAGE_BRUSH("ContentBrowser/ContentDirty", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.AssetDragDropTooltipBackground", new BOX_BRUSH("Old/Menu_Background", FMargin(8.0f / 64.0f)));
		Set("ContentBrowser.CollectionTreeDragDropBorder", new BOX_BRUSH("Old/Window/ViewportDebugBorder", 0.8f));
		Set("ContentBrowser.PopupMessageIcon.Check", new CORE_IMAGE_BRUSH_SVG("Starship/Common/check-circle-solid", CoreStyleConstants::Icon16x16, FStyleColors::AccentGreen));
		Set("ContentBrowser.PopupMessageIcon.Info", new CORE_IMAGE_BRUSH_SVG("Starship/Common/info-circle-solid", CoreStyleConstants::Icon16x16, FStyleColors::Foreground));
		Set("ContentBrowser.NewFolderIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-plus", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.ShowInExplorer", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/show-in-explorer", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.ReferenceViewer", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/reference-viewer", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.SizeMap", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/size-map", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.Collections", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/collections", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.Migrate", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/migrate", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.Local", new IMAGE_BRUSH("ContentBrowser/Content_Local_12x", CoreStyleConstants::Icon12x12));
		Set("ContentBrowser.Local.Small", new IMAGE_BRUSH("ContentBrowser/Content_Local_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.Local.Large", new IMAGE_BRUSH("ContentBrowser/Content_Local_64x", CoreStyleConstants::Icon64x64));
		Set("ContentBrowser.Shared", new IMAGE_BRUSH("ContentBrowser/Content_Shared_12x", CoreStyleConstants::Icon12x12));
		Set("ContentBrowser.Shared.Small", new IMAGE_BRUSH("ContentBrowser/Content_Shared_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.Shared.Large", new IMAGE_BRUSH("ContentBrowser/Content_Shared_64x", CoreStyleConstants::Icon64x64));
		Set("ContentBrowser.Private", new IMAGE_BRUSH("ContentBrowser/Content_Private_12x", CoreStyleConstants::Icon12x12));
		Set("ContentBrowser.Private.Small", new IMAGE_BRUSH("ContentBrowser/Content_Private_16x", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.Private.Large", new IMAGE_BRUSH("ContentBrowser/Content_Private_64x", CoreStyleConstants::Icon64x64));
		Set("ContentBrowser.CollectionStatus", new IMAGE_BRUSH("/Icons/CollectionStatus_8x", CoreStyleConstants::Icon8x8));


		Set("ContentBrowser.SaveAllCurrentFolder", new IMAGE_BRUSH_SVG("Starship/Common/SaveCurrent", CoreStyleConstants::Icon16x16));
		Set("ContentBrowser.ResaveAllCurrentFolder", new IMAGE_BRUSH_SVG("Starship/Common/SaveCurrent", CoreStyleConstants::Icon16x16));

		const FSplitterStyle ContentBrowserSplitterStyle =
			FSplitterStyle()
			.SetHandleNormalBrush(FSlateColorBrush(FStyleColors::Recessed))
			.SetHandleHighlightBrush(FSlateColorBrush(FStyleColors::Transparent));

		Set("ContentBrowser.Splitter", ContentBrowserSplitterStyle);

		FToolBarStyle ContentBrowserToolBarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar");

		ContentBrowserToolBarStyle.SetSeparatorBrush(FSlateNoResource());
		ContentBrowserToolBarStyle.SetSeparatorPadding(FMargin(4.0f, 0.0f));

		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			// Hide separators
			ContentBrowserToolBarStyle.SetSeparatorThickness(0.0f);
			ContentBrowserToolBarStyle.SetSeparatorPadding(FMargin(0.0f, 0.0f));

			ContentBrowserToolBarStyle.SetBlockPadding(FMargin(2.0f, 0.0f)); // Effectively makes custom widgets look like buttons

			ContentBrowserToolBarStyle.SetAllowWrapButton(false); // Never show the wrap button, just clip
			ContentBrowserToolBarStyle.SetIconSize(FVector2D(16.0f, 16.0f));
			ContentBrowserToolBarStyle.SetBackgroundPadding(FMargin(4.0f, 2.0f, 4.0f, 2.0f));
			ContentBrowserToolBarStyle.SetAllowWrappingDefault(false);
		}
		else
		{
			ContentBrowserToolBarStyle.SetBackgroundPadding(FMargin(4.0f, 2.0f, 0.0f, 2.0f));
		}

		Set("ContentBrowser.ToolBar", ContentBrowserToolBarStyle);

		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			FButtonStyle ButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button");

			constexpr int32 HorizontalButtonPadding = 2;

			FMargin ButtonPadding = ContentBrowserToolBarStyle.ButtonPadding;
			ButtonPadding.Left = HorizontalButtonPadding;
			ButtonPadding.Right = HorizontalButtonPadding;

			ContentBrowserToolBarStyle.SetButtonPadding(ButtonPadding);

			ContentBrowserToolBarStyle.SetButtonStyle(ButtonStyle);
		}

		// Separate style for buttons prior to the search box to get around SToolBarComboButtonBlock referencing the wrong style
		Set("ContentBrowser.ToolBar.Buttons", ContentBrowserToolBarStyle);

		Set("ContentBrowser.AddNewMenu.Separator", new FSlateColorBrush(FStyleColors::White25));
		Set("ContentBrowser.AddNewMenu.Separator.Padding", FAppStyle::Get().GetMargin("Menu.Separator.Padding"));

		Set("ContentBrowser.AddNewMenu.Label", FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Menu.Label"));
		Set("ContentBrowser.AddNewMenu.Heading", FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Menu.Heading"));
		Set("ContentBrowser.AddNewMenu.Heading.Padding", FAppStyle::Get().GetMargin("Menu.Heading.Padding"));
		Set("ContentBrowser.AddNewMenu.Keybinding", FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Menu.Keybinding"));

		Set("ContentBrowser.AddNewMenu.SubMenuIndicator", new CORE_IMAGE_BRUSH_SVG("Starship/Common/chevron-right", CoreStyleConstants::Icon16x16, FStyleColors::Foreground));

		Set("ContentBrowser.AddNewMenu.Button", FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Menu.Button"));
		Set("ContentBrowser.AddNewMenu.CheckBox", FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("Menu.CheckBox"));

		FMargin MenuBlockPadding = FAppStyle::Get().GetMargin("Menu.Block.Padding");
		FMargin IndentedMenuBlockPadding = FAppStyle::Get().GetMargin("Menu.Block.IndentedPadding");

		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			MenuBlockPadding.Left = 0.0f;
			IndentedMenuBlockPadding.Left = 0.0f;
		}

		Set("ContentBrowser.AddNewMenu.Block.Padding", MenuBlockPadding);
		Set("ContentBrowser.AddNewMenu.Block.IndentedPadding", IndentedMenuBlockPadding);

		MenuBlockPadding.Left = -100.0f;
		IndentedMenuBlockPadding.Left = -150.0f;

		Set("ContentBrowser.AssetViewOptions.Block.Padding", MenuBlockPadding);
		Set("ContentBrowser.AssetViewOptions.Block.IndentedPadding", IndentedMenuBlockPadding);

		// Content Sources
		{
			static const FVector2D DefaultIconSize = CoreStyleConstants::Icon24x24;
			static constexpr float DefaultButtonSize = 56.0f;
			
			FToolBarStyle ContentBrowserSourceBarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("FVerticalToolBar");
			
			ContentBrowserSourceBarStyle.SetIconSize(DefaultIconSize);
			ContentBrowserSourceBarStyle.SetUniformBlockWidth(DefaultButtonSize);
			ContentBrowserSourceBarStyle.SetUniformBlockHeight(DefaultButtonSize);
			ContentBrowserSourceBarStyle.SetButtonPadding(FMargin(0.0f, 2.0f));
			ContentBrowserSourceBarStyle.ButtonStyle.SetNormalPadding(FMargin(0.0f));
			ContentBrowserSourceBarStyle.ButtonStyle.SetPressedPadding(FMargin(0.0f));
			ContentBrowserSourceBarStyle.SetBackgroundPadding(FMargin(4.0f));
			
			Set("ContentBrowser.SourceBar", ContentBrowserSourceBarStyle);
			Set("ContentBrowser.Sources.PanelIcon.Opened", new CORE_IMAGE_BRUSH_SVG("Starship/Common/SidePanelLeft", DefaultIconSize));
			Set("ContentBrowser.Sources.PanelIcon.Closed", new CORE_IMAGE_BRUSH_SVG("Starship/Common/SidePanelLeftClosed", DefaultIconSize));
			Set("ContentBrowser.Sources.ProjectIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-closed", DefaultIconSize));
			Set("ContentBrowser.Sources.CreateIcon", new IMAGE_BRUSH_SVG("Starship/Common/PlaceActors", DefaultIconSize));
		}

		// Filter View, mostly matches AssetTree
		{
			static constexpr float FilterViewHorizontalPadding = 10.0f;
			static constexpr float FilterViewVerticalHeaderPadding = FilterViewHorizontalPadding - 6.0f;

			Set("ContentBrowser.VerticalFilterViewHeaderBrush", VerticalFilterViewHeaderBrush);
			Set("ContentBrowser.VerticalFilterViewHeaderPadding", FMargin(FilterViewHorizontalPadding, FilterViewVerticalHeaderPadding));
			Set("ContentBrowser.VerticalFilterViewHeaderTextHeight", CoreStyleConstants::Icon24x24.Y); // Based on button height when dictated by an icon, matches AssetTree header

			Set("ContentBrowser.VerticalFilterViewBodyBrush", VerticalFilterViewBodyBrush);
		}

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	FContentBrowserStyle::~FContentBrowserStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
}
