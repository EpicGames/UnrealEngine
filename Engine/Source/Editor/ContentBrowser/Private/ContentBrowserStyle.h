// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateColor.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateTypes.h"

namespace UE::Editor::ContentBrowser
{
	static bool IsNewStyleEnabled()
	{
		static bool bIsNewStyleEnabled = false;
		UE_CALL_ONCE([&]()
		{
			if (const IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("ContentBrowser.EnableNewStyle")))
			{
				ensureAlwaysMsgf(!EnumHasAnyFlags(CVar->GetFlags(), ECVF_Default), TEXT("The CVar should have already been set from commandline, @see: UnrealEdGlobals.cpp, UE::Editor::ContentBrowser::EnableContentBrowserNewStyleCVarRegistration."));
				bIsNewStyleEnabled = CVar->GetBool();
			}
		});

		return bIsNewStyleEnabled;
	}
}

namespace UE::ContentBrowser::Private
{
	class FContentBrowserStyle
		: public FSlateStyleSet
	{
	public:
		static FContentBrowserStyle& Get();

		virtual const FName& GetStyleSetName() const override;

	private:
		FContentBrowserStyle();
		virtual ~FContentBrowserStyle() override;

	private:
		static FName StyleName;

		// Colors and Styles inherited from the parent style
		FSlateColor DefaultForeground;
		FSlateColor InvertedForeground;
		FSlateColor SelectionColor;
		FSlateColor SelectionColor_Pressed;

		FTextBlockStyle NormalText;
		FButtonStyle Button;
	};
}
