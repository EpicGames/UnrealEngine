// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserUtils.h"

#include "Algo/MinElement.h"
#include "Algo/NoneOf.h"
#include "Algo/Unique.h"
#include "Algo/Transform.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "CollectionManagerModule.h"
#include "CollectionViewUtils.h"
#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "Containers/VersePath.h"
#include "ContentBrowserConfig.h"
#include "ContentBrowserDataFilter.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserDataUtils.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserItemData.h"
#include "ContentBrowserSingleton.h"
#include "ContentBrowserStyle.h"
#include "CoreGlobals.h"
#include "Framework/Application/IMenu.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "FrontendFilterBase.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "IAssetTools.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "ICollectionSource.h"
#include "IContentBrowserDataModule.h"
#include "Input/Reply.h"
#include "Interfaces/IPluginManager.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/SlateRect.h"
#include "Layout/WidgetPath.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "Misc/NamePermissionList.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "SAssetView.h"
#include "Settings/ContentBrowserSettings.h"
#include "SFilterList.h"
#include "SPathView.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "SNavigationBar.h"
#include "SourceControlOperations.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/TopLevelAssetPath.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

class SWidget;
struct FGeometry;
struct FPointerEvent;

#define LOCTEXT_NAMESPACE "ContentBrowser"

namespace ContentBrowserUtils
{
	/** Converts a virtual path such as /All/Plugins -> /Plugins or /All/Game -> /Game */
	FString ConvertVirtualPathToInvariantPathString(const FString& VirtualPath)
	{
		FName ConvertedPath;
		IContentBrowserDataModule::Get().GetSubsystem()->TryConvertVirtualPath(FName(VirtualPath), ConvertedPath);
		return ConvertedPath.ToString();
	}

	bool VirtualPathPassesBlockLists(const FContentBrowserItem& Item, const SPathView& PathViewForFiltering)
	{
		check(Item.GetInternalPath().IsNone());

		bool bPasses = false;
		// Test if any child internal paths pass for this fully virtual path
		for (const FContentBrowserItemData& ItemData : Item.GetInternalItems())
		{
			UContentBrowserDataSource* ItemDataSource = ItemData.GetOwnerDataSource();
			if (!ItemDataSource)
			{
				continue;
			}

			auto OnSubPath = [&PathViewForFiltering, &bPasses](FName VirtualSubPath, FName InternalPath)
			{
				if (InternalPath.IsNone())
				{
					// Keep enumerating, this path only exist virtually e.g. /All/Plugins
					return true;
				}

				const FNameBuilder InternalPathBuilder(InternalPath);
				if (ContentBrowserDataUtils::PathPassesAttributeFilter(InternalPathBuilder, 0, PathViewForFiltering.GetContentBrowserItemAttributeFilter())
					&& PathViewForFiltering.InternalPathPassesBlockLists(InternalPathBuilder))
				{
					bPasses = true;
					// Stop enumerating
					return false;
				}
				return true;
			};
			ItemDataSource->GetRootPathVirtualTree().EnumerateSubPaths(Item.GetVirtualPath(), OnSubPath, /*bRecurse*/ true);

			if (bPasses)
			{
				break;
			}
		}
		return bPasses;
	}
}

class SContentBrowserPopup : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SContentBrowserPopup ){}

		SLATE_ATTRIBUTE( FText, Message )
		SLATE_ARGUMENT( ContentBrowserUtils::EDisplayMessageType, MessageType )

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void Construct( const FArguments& InArgs )
	{
		Message = InArgs._Message;
		MessageType = InArgs._MessageType;

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			.Padding(10.f)
			.OnMouseButtonDown(this, &SContentBrowserPopup::OnBorderClicked)
			.BorderBackgroundColor(this, &SContentBrowserPopup::GetBorderBackgroundColor)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 4.f, 0.f)
				[
					SNew(SImage)
					.Image(this, &SContentBrowserPopup::GetDisplayMessageIconBrush)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(InArgs._Message)
					.WrapTextAt(450)
				]
			]
		];
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

	const FSlateBrush* GetDisplayMessageIconBrush() const
	{
		FName BrushName = FName(TEXT(""));
		switch (MessageType)
		{
			case ContentBrowserUtils::EDisplayMessageType::Successful:
				BrushName = FName(TEXT("ContentBrowser.PopupMessageIcon.Check"));
				break;

			case ContentBrowserUtils::EDisplayMessageType::Info:
				BrushName = FName(TEXT("ContentBrowser.PopupMessageIcon.Info"));
				break;

			case ContentBrowserUtils::EDisplayMessageType::Warning:
				BrushName = FName(TEXT("Icons.Warning.Solid"));
				break;

			case ContentBrowserUtils::EDisplayMessageType::Error:
				BrushName = FName(TEXT("Icons.Error.Solid"));
				break;

			default:
				break;
		}
		return UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush(BrushName);
	}

	static void DisplayMessage( const FText& Message, const FSlateRect& ScreenAnchor, TSharedRef<SWidget> ParentContent, ContentBrowserUtils::EDisplayMessageType InMessageType )
	{
		TSharedRef<SContentBrowserPopup> PopupContent = SNew(SContentBrowserPopup)
			.Message(Message)
			.MessageType(InMessageType);

		const FVector2D ScreenLocation = FVector2D(ScreenAnchor.Left, ScreenAnchor.Top);
		const bool bFocusImmediately = true;
		const FVector2D SummonLocationSize = ScreenAnchor.GetSize();

		TSharedPtr<IMenu> Menu = FSlateApplication::Get().PushMenu(
			ParentContent,
			FWidgetPath(),
			PopupContent,
			ScreenLocation,
			FPopupTransitionEffect( FPopupTransitionEffect::TopMenu ),
			bFocusImmediately,
			SummonLocationSize);

		PopupContent->SetMenu(Menu);
	}

private:
	void SetMenu(const TSharedPtr<IMenu>& InMenu)
	{
		Menu = InMenu;
	}

	FReply OnBorderClicked(const FGeometry& Geometry, const FPointerEvent& MouseEvent)
	{
		if (Menu.IsValid())
		{
			Menu.Pin()->Dismiss();
		}

		return FReply::Handled();
	}

	FSlateColor GetBorderBackgroundColor() const
	{
		return IsHovered() ? FLinearColor(0.5, 0.5, 0.5, 1) : FLinearColor::White;
	}

private:
	ContentBrowserUtils::EDisplayMessageType MessageType = ContentBrowserUtils::EDisplayMessageType::Info;
	TAttribute<FText> Message;
	TWeakPtr<IMenu> Menu;
};

/** A miniture confirmation popup for quick yes/no questions */
class SContentBrowserConfirmPopup :  public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SContentBrowserConfirmPopup ) {}
			
		/** The text to display */
		SLATE_ARGUMENT(FText, Prompt)

		/** The Yes Button to display */
		SLATE_ARGUMENT(FText, YesText)

		/** The No Button to display */
		SLATE_ARGUMENT(FText, NoText)

		/** Invoked when yes is clicked */
		SLATE_EVENT(FOnClicked, OnYesClicked)

		/** Invoked when no is clicked */
		SLATE_EVENT(FOnClicked, OnNoClicked)

	SLATE_END_ARGS()

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void Construct( const FArguments& InArgs )
	{
		OnYesClicked = InArgs._OnYesClicked;
		OnNoClicked = InArgs._OnNoClicked;

		ChildSlot
		[
			SNew(SBorder)
			. BorderImage(FAppStyle::GetBrush("Menu.Background"))
			. Padding(10.f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.f, 0.f, 0.f, 5.f)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
						.Text(InArgs._Prompt)
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(SUniformGridPanel)
					.SlotPadding(3.f)
					+ SUniformGridPanel::Slot(0, 0)
					.HAlign(HAlign_Fill)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(InArgs._YesText)
						.OnClicked( this, &SContentBrowserConfirmPopup::YesClicked )
					]

					+ SUniformGridPanel::Slot(1, 0)
					.HAlign(HAlign_Fill)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.Text(InArgs._NoText)
						.OnClicked( this, &SContentBrowserConfirmPopup::NoClicked )
					]
				]
			]
		];
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

	/** Opens the popup using the specified component as its parent */
	void OpenPopup(const TSharedRef<SWidget>& ParentContent)
	{
		// Show dialog to confirm the delete
		Menu = FSlateApplication::Get().PushMenu(
			ParentContent,
			FWidgetPath(),
			SharedThis(this),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect( FPopupTransitionEffect::TopMenu )
			);
	}

private:
	/** The yes button was clicked */
	FReply YesClicked()
	{
		if ( OnYesClicked.IsBound() )
		{
			OnYesClicked.Execute();
		}

		if (Menu.IsValid())
		{
			Menu.Pin()->Dismiss();
		}

		return FReply::Handled();
	}

	/** The no button was clicked */
	FReply NoClicked()
	{
		if ( OnNoClicked.IsBound() )
		{
			OnNoClicked.Execute();
		}

		if (Menu.IsValid())
		{
			Menu.Pin()->Dismiss();
		}

		return FReply::Handled();
	}

	/** The IMenu prepresenting this popup */
	TWeakPtr<IMenu> Menu;

	/** Delegates for button clicks */
	FOnClicked OnYesClicked;
	FOnClicked OnNoClicked;
};


void ContentBrowserUtils::DisplayMessage(const FText& Message, const FSlateRect& ScreenAnchor, const TSharedRef<SWidget>& ParentContent, EDisplayMessageType InMessageType)
{
	SContentBrowserPopup::DisplayMessage(Message, ScreenAnchor, ParentContent, InMessageType);
}

void ContentBrowserUtils::DisplayConfirmationPopup(const FText& Message, const FText& YesString, const FText& NoString, const TSharedRef<SWidget>& ParentContent, const FOnClicked& OnYesClicked, const FOnClicked& OnNoClicked)
{
	TSharedRef<SContentBrowserConfirmPopup> Popup = 
		SNew(SContentBrowserConfirmPopup)
		.Prompt(Message)
		.YesText(YesString)
		.NoText(NoString)
		.OnYesClicked( OnYesClicked )
		.OnNoClicked( OnNoClicked );

	Popup->OpenPopup(ParentContent);
}

bool ContentBrowserUtils::DisplayConfirmationDialog(const FText& InTitle, const FText& InMessage)
{
	return FMessageDialog::Open(EAppMsgType::YesNo, InMessage, InTitle) == EAppReturnType::Type::Yes;
}

FString ContentBrowserUtils::GetItemReferencesText(const TArray<FContentBrowserItem>& Items)
{
	TArray<FContentBrowserItem> SortedItems = Items;
	SortedItems.Sort([](const FContentBrowserItem& One, const FContentBrowserItem& Two)
	{
		return One.GetVirtualPath().Compare(Two.GetVirtualPath()) < 0;
	});

	FString Result;
	for (const FContentBrowserItem& Item : SortedItems)
	{
		if (ensure(!Item.IsFolder()))
		{
			Item.AppendItemReference(Result);
		}
	}

	return Result;
}

FString ContentBrowserUtils::GetItemObjectPathText(const TArray<FContentBrowserItem>& Items)
{
	TArray<FContentBrowserItem> SortedItems = Items;
	SortedItems.Sort([](const FContentBrowserItem& One, const FContentBrowserItem& Two)
	{
		return One.GetVirtualPath().Compare(Two.GetVirtualPath()) < 0;
	});

	FString Result;
	for (const FContentBrowserItem& Item : SortedItems)
	{
		if (ensure(!Item.IsFolder()))
		{
			Item.AppendItemObjectPath(Result);
		}
	}

	return Result;
}

FString ContentBrowserUtils::GetItemPackageNameText(const TArray<FContentBrowserItem>& Items)
{
	TArray<FContentBrowserItem> SortedItems = Items;
	SortedItems.Sort([](const FContentBrowserItem& One, const FContentBrowserItem& Two)
	{
		return One.GetVirtualPath().Compare(Two.GetVirtualPath()) < 0;
	});

	FString Result;
	for (const FContentBrowserItem& Item : SortedItems)
	{
		if (ensure(!Item.IsFolder()))
		{
			Item.AppendItemPackageName(Result);
		}
	}

	return Result;
}

FString ContentBrowserUtils::GetFolderReferencesText(const TArray<FContentBrowserItem>& Folders)
{
	TArray<FContentBrowserItem> SortedItems = Folders;
	SortedItems.Sort([](const FContentBrowserItem& One, const FContentBrowserItem& Two)
	{
		return One.GetVirtualPath().Compare(Two.GetVirtualPath()) < 0;
	});

	TStringBuilder<2048> Result;
	for (const FContentBrowserItem& Item : SortedItems)
	{
		if (ensure(Item.IsFolder()))
		{
			FName PackagePath;
			if (Item.Legacy_TryGetPackagePath(PackagePath))
			{
				if (Result.Len() > 0)
				{
					Result << LINE_TERMINATOR;
				}

				Result << PackagePath;
			}
		}
	}

	return Result.ToString();
}

void ContentBrowserUtils::CopyItemReferencesToClipboard(const TArray<FContentBrowserItem>& ItemsToCopy)
{
	FString Text = GetItemReferencesText(ItemsToCopy);
	if (!Text.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*Text);
	}
}

void ContentBrowserUtils::CopyItemObjectPathToClipboard(const TArray<FContentBrowserItem>& ItemsToCopy)
{
	FString Text = GetItemObjectPathText(ItemsToCopy);
	if (!Text.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*Text);
	}
}

void ContentBrowserUtils::CopyItemPackageNameToClipboard(const TArray<FContentBrowserItem>& ItemsToCopy)
{
	FString Text = GetItemPackageNameText(ItemsToCopy);
	if (!Text.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*Text);
	}
}

void ContentBrowserUtils::CopyFolderReferencesToClipboard(const TArray<FContentBrowserItem>& FoldersToCopy)
{
	FString Text = GetFolderReferencesText(FoldersToCopy);
	if (!Text.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*Text);
	}
}

void ContentBrowserUtils::CopyFilePathsToClipboard(const TArray<FContentBrowserItem>& ItemsToCopy)
{
	TArray<FContentBrowserItem> SortedItems = ItemsToCopy;
	SortedItems.Sort([](const FContentBrowserItem& One, const FContentBrowserItem& Two)
	{
		return One.GetVirtualPath().Compare(Two.GetVirtualPath()) < 0;
	});

	FString ClipboardText;
	for (const FContentBrowserItem& Item : SortedItems)
	{
		if (ClipboardText.Len() > 0)
		{
			ClipboardText += LINE_TERMINATOR;
		}

		FString ItemFilename;
		if (Item.GetItemPhysicalPath(ItemFilename) && FPaths::FileExists(ItemFilename))
		{
			ItemFilename = FPaths::ConvertRelativePathToFull(ItemFilename);
			FPaths::MakePlatformFilename(ItemFilename);
			ClipboardText += ItemFilename;
		}
		else
		{
			// Add a message for when a user tries to copy the path to a file that doesn't exist on disk of the form
			// <ItemName>: No file on disk
			ClipboardText += FString::Printf(TEXT("%s: No file on disk"), *Item.GetDisplayName().ToString());
		}
	}

	FPlatformApplicationMisc::ClipboardCopy(*ClipboardText);
}

bool ContentBrowserUtils::IsItemDeveloperContent(const FContentBrowserItem& InItem)
{
	const FContentBrowserItemDataAttributeValue IsDeveloperAttributeValue = InItem.GetItemAttribute(ContentBrowserItemAttributes::ItemIsDeveloperContent);
	return IsDeveloperAttributeValue.IsValid() && IsDeveloperAttributeValue.GetValue<bool>();
}

bool ContentBrowserUtils::IsItemLocalizedContent(const FContentBrowserItem& InItem)
{
	const FContentBrowserItemDataAttributeValue IsLocalizedAttributeValue = InItem.GetItemAttribute(ContentBrowserItemAttributes::ItemIsLocalizedContent);
	return IsLocalizedAttributeValue.IsValid() && IsLocalizedAttributeValue.GetValue<bool>();
}

bool ContentBrowserUtils::IsItemEngineContent(const FContentBrowserItem& InItem)
{
	const FContentBrowserItemDataAttributeValue IsEngineAttributeValue = InItem.GetItemAttribute(ContentBrowserItemAttributes::ItemIsEngineContent);
	return IsEngineAttributeValue.IsValid() && IsEngineAttributeValue.GetValue<bool>();
}

bool ContentBrowserUtils::IsItemProjectContent(const FContentBrowserItem& InItem)
{
	const FContentBrowserItemDataAttributeValue IsProjectAttributeValue = InItem.GetItemAttribute(ContentBrowserItemAttributes::ItemIsProjectContent);
	return IsProjectAttributeValue.IsValid() && IsProjectAttributeValue.GetValue<bool>();
}

bool ContentBrowserUtils::IsItemPluginContent(const FContentBrowserItem& InItem)
{
	const FContentBrowserItemDataAttributeValue IsPluginAttributeValue = InItem.GetItemAttribute(ContentBrowserItemAttributes::ItemIsPluginContent);
	return IsPluginAttributeValue.IsValid() && IsPluginAttributeValue.GetValue<bool>();
}

bool ContentBrowserUtils::IsItemPluginRootFolder(const FContentBrowserItem& InItem)
{
	if (!InItem.IsFolder())
	{
		return false;
	}
	FName InternalPath = InItem.GetInternalPath();
	if (InternalPath.IsNone())
	{
		return false;
	}
	FNameBuilder PathBuffer(InternalPath);
	FStringView Path = PathBuffer.ToView();
	if (int32 Index = 0; Path.RightChop(1).FindChar('/', Index) && Index != INDEX_NONE)
	{
		return false; // Contains a second slash, is not a root
	}
	return IsItemPluginContent(InItem);
}

bool ContentBrowserUtils::TryGetFolderBrushAndShadowName(const FContentBrowserItem& InFolder, FName& OutBrushName, FName& OutShadowBrushName)
{
	if (!InFolder.IsValid() || !InFolder.IsFolder())
	{
		return false;
	}

	OutShadowBrushName = TEXT("ContentBrowser.FolderItem.DropShadow");

	FContentBrowserItemDataAttributeValue FolderBrushAttribute = InFolder.GetItemAttribute(ContentBrowserItemAttributes::ItemListViewFolderBrush);
	if (FolderBrushAttribute.IsValid())
	{
		OutBrushName = FolderBrushAttribute.GetValueName();
	}
	else
	{
		const bool bDeveloperFolder = IsItemDeveloperContent(InFolder);
		const bool bCodeFolder = EnumHasAnyFlags(InFolder.GetItemCategory(), EContentBrowserItemFlags::Category_Class);
		const FContentBrowserItemDataAttributeValue VirtualAttributeValue = InFolder.GetItemAttribute(ContentBrowserItemAttributes::ItemIsCustomVirtualFolder);
		const bool bVirtualFolder = VirtualAttributeValue.IsValid() && VirtualAttributeValue.GetValue<bool>();
		const bool bPluginFolder = IsItemPluginRootFolder(InFolder);

		if (bDeveloperFolder)
		{
			OutBrushName = TEXT("ContentBrowser.ListViewDeveloperFolderIcon");
		}
		else if (bCodeFolder)
		{
			OutBrushName = TEXT("ContentBrowser.ListViewCodeFolderIcon");
		}
		else if (bVirtualFolder && ShouldShowCustomVirtualFolderIcon())
		{
			OutBrushName = TEXT("ContentBrowser.ListViewVirtualFolderIcon");
			OutShadowBrushName = TEXT("ContentBrowser.ListViewVirtualFolderShadow");
		}
		else if (bPluginFolder && ShouldShowPluginFolderIcon())
		{
			OutBrushName = TEXT("ContentBrowser.ListViewPluginFolderIcon");
		}
		else
		{
			OutBrushName = TEXT("ContentBrowser.ListViewFolderIcon");
		}
	}

	return true;
}

bool ContentBrowserUtils::TryGetFolderBrushAndShadowNameSmall(const FContentBrowserItem& InFolder, FName& OutBrushName, FName& OutShadowBrushName)
{
	if (!InFolder.IsValid() || !InFolder.IsFolder())
	{
		return false;
	}

	OutShadowBrushName = TEXT("ContentBrowser.FolderItem.DropShadow");

	FContentBrowserItemDataAttributeValue FolderBrushAttribute = InFolder.GetItemAttribute(ContentBrowserItemAttributes::ItemAssetTreeFolderClosedBrush);
	if (FolderBrushAttribute.IsValid())
	{
		OutBrushName = FolderBrushAttribute.GetValueName();
	}
	else
	{
		const bool bDeveloperFolder = IsItemDeveloperContent(InFolder);
		const bool bCodeFolder = EnumHasAnyFlags(InFolder.GetItemCategory(), EContentBrowserItemFlags::Category_Class);
		const FContentBrowserItemDataAttributeValue VirtualAttributeValue = InFolder.GetItemAttribute(ContentBrowserItemAttributes::ItemIsCustomVirtualFolder);
		const bool bVirtualFolder = VirtualAttributeValue.IsValid() && VirtualAttributeValue.GetValue<bool>();
		const bool bPluginFolder = IsItemPluginRootFolder(InFolder);

		if (bDeveloperFolder)
		{
			OutBrushName = TEXT("ContentBrowser.AssetTreeFolderClosedDeveloper");
		}
		else if (bCodeFolder)
		{
			OutBrushName = TEXT("ContentBrowser.AssetTreeFolderClosedCode");
		}
		else if (bVirtualFolder && ShouldShowCustomVirtualFolderIcon())
		{
			OutBrushName = TEXT("ContentBrowser.AssetTreeFolderClosedVirtual");
			OutShadowBrushName = TEXT("ContentBrowser.ListViewVirtualFolderShadow");
		}
		else if (bPluginFolder && ShouldShowPluginFolderIcon())
		{
			OutBrushName = TEXT("ContentBrowser.AssetTreeFolderClosedPluginRoot");
		}
		else
		{
			OutBrushName = TEXT("ContentBrowser.AssetTreeFolderClosed");
		}
	}

	return true;
}

bool ContentBrowserUtils::IsCollectionPath(const FString& InPath, TSharedPtr<ICollectionContainer>* OutCollectionContainer, FName* OutCollectionName, ECollectionShareType::Type* OutCollectionShareType)
{
	static const FString CollectionsRootPrefix = TEXT("/Collections/");
	if (InPath.StartsWith(CollectionsRootPrefix))
	{
		TArray<FString> PathParts;
		InPath.ParseIntoArray(PathParts, TEXT("/"));
		check(PathParts.Num() > 3);

		// The second part of the path is the collection container id
		if (OutCollectionContainer)
		{
			*OutCollectionContainer = FCollectionManagerModule::GetModule().Get().FindCollectionContainer(FName(PathParts[1], FNAME_Find));
		}

		// The third part of the path is the share type name
		if (OutCollectionShareType)
		{
			*OutCollectionShareType = ECollectionShareType::FromString(*PathParts[2]);
		}

		// The fourth part of the path is the collection name
		if (OutCollectionName)
		{
			*OutCollectionName = FName(*PathParts[3]);
		}

		return true;
	}
	return false;
}

void ContentBrowserUtils::CountPathTypes(const TArray<FString>& InPaths, int32& OutNumAssetPaths, int32& OutNumClassPaths)
{
	static const FString ClassesRootPrefix = TEXT("/Classes_");

	OutNumAssetPaths = 0;
	OutNumClassPaths = 0;

	for(const FString& Path : InPaths)
	{
		if(Path.StartsWith(ClassesRootPrefix))
		{
			++OutNumClassPaths;
		}
		else
		{
			++OutNumAssetPaths;
		}
	}
}

void ContentBrowserUtils::CountPathTypes(const TArray<FName>& InPaths, int32& OutNumAssetPaths, int32& OutNumClassPaths)
{
	static const FString ClassesRootPrefix = TEXT("/Classes_");

	OutNumAssetPaths = 0;
	OutNumClassPaths = 0;

	for(const FName& Path : InPaths)
	{
		if(Path.ToString().StartsWith(ClassesRootPrefix))
		{
			++OutNumClassPaths;
		}
		else
		{
			++OutNumAssetPaths;
		}
	}
}

void ContentBrowserUtils::CountItemTypes(const TArray<FAssetData>& InItems, int32& OutNumAssetItems, int32& OutNumClassItems)
{
	OutNumAssetItems = 0;
	OutNumClassItems = 0;

	const FTopLevelAssetPath ClassPath(TEXT("/Script/CoreUObject"), TEXT("Class"));
	for(const FAssetData& Item : InItems)
	{
		if(Item.AssetClassPath == ClassPath)
		{
			++OutNumClassItems;
		}
		else
		{
			++OutNumAssetItems;
		}
	}
}

FText ContentBrowserUtils::GetExploreFolderText()
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("FileManagerName"), FPlatformMisc::GetFileManagerName());
	return FText::Format(NSLOCTEXT("GenericPlatform", "ShowInFileManager", "Show in {FileManagerName}"), Args);
}

void ContentBrowserUtils::ExploreFolders(const TArray<FContentBrowserItem>& InItems, const TSharedRef<SWidget>& InParentContent)
{
	TArray<FString> ExploreItems;

	for (const FContentBrowserItem& SelectedItem : InItems)
	{
		FString ItemFilename;
		if (SelectedItem.GetItemPhysicalPath(ItemFilename))
		{
			const bool bExists = SelectedItem.IsFile() ? FPaths::FileExists(ItemFilename) : FPaths::DirectoryExists(ItemFilename);
			if (bExists)
			{
				ExploreItems.Add(IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ItemFilename));
			}
		}
	}

	const int32 BatchSize = 10;
	const FText FileManagerName = FPlatformMisc::GetFileManagerName();
	const bool bHasMultipleBatches = ExploreItems.Num() > BatchSize;
	for (int32 i = 0; i < ExploreItems.Num(); ++i)
	{
		bool bIsBatchBoundary = (i % BatchSize) == 0;
		if (bHasMultipleBatches && bIsBatchBoundary)
		{
			int32 RemainingCount = ExploreItems.Num() - i;
			int32 NextCount = FMath::Min(BatchSize, RemainingCount);
			FText Prompt = FText::Format(LOCTEXT("ExecuteExploreConfirm", "Show {0} {0}|plural(one=item,other=items) in {1}?\nThere {2}|plural(one=is,other=are) {2} remaining."), NextCount, FileManagerName, RemainingCount);
			if (FMessageDialog::Open(EAppMsgType::YesNo, Prompt) != EAppReturnType::Yes)
			{
				return;
			}
		}

		FPlatformProcess::ExploreFolder(*ExploreItems[i]);
	}
}

bool ContentBrowserUtils::CanExploreFolders(const TArray<FContentBrowserItem>& InItems)
{
	for (const FContentBrowserItem& SelectedItem : InItems)
	{
		FString ItemFilename;
		if (SelectedItem.GetItemPhysicalPath(ItemFilename))
		{
			const bool bExists = SelectedItem.IsFile() ? FPaths::FileExists(ItemFilename) : FPaths::DirectoryExists(ItemFilename);
			if (bExists)
			{
				return true;
			}
		}
	}

	return false;
}

TArray<FContentBrowserItem> ContentBrowserUtils::ConvertLegacySelectionToItems(TArrayView<const FAssetData> InAssets, TArrayView<const FString> InFolders)
{
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

	TArray<FContentBrowserItem> Items;
	Items.Reserve(InAssets.Num() + InFolders.Num());

	if (!InAssets.IsEmpty())
	{
		const TSharedRef<FPathPermissionList>& FolderPermissions = FAssetToolsModule::GetModule().Get().GetFolderPermissionList();

		TArray<FName> VirtualPaths;
		VirtualPaths.Reserve(InAssets.Num());

		auto AppendVirtualPath = [&VirtualPaths](FName VirtualPath)
		{
			VirtualPaths.Add(VirtualPath);
			return true;
		};

		for (const FAssetData& AssetData : InAssets)
		{
			if (FolderPermissions->PassesStartsWithFilter(AssetData.GetObjectPathString()))
			{
				const int32 PreviousNumVirtualPaths = VirtualPaths.Num();
				ContentBrowserData->Legacy_TryConvertAssetDataToVirtualPaths(AssetData, /*UseFolderPaths*/false, AppendVirtualPath);
				if (VirtualPaths.Num() > PreviousNumVirtualPaths)
				{
					// If we found a virtual path, continue, otherwise the item may only exist as an alias, so try those next.
					continue;
				}
			}

			TArray<FContentBrowserItemPath> Aliases = ContentBrowserData->GetAliasesForPath(*AssetData.GetObjectPathString());
			for (const FContentBrowserItemPath& Alias : Aliases)
			{
				if (FolderPermissions->PassesStartsWithFilter(Alias.GetInternalPathName()))
				{
					VirtualPaths.Add(Alias.GetVirtualPathName());
					break;
				}
			}
		}

		for (const FName& VirtualPath : VirtualPaths)
		{
			FContentBrowserItem ItemToSync = ContentBrowserData->GetItemAtPath(VirtualPath, EContentBrowserItemTypeFilter::IncludeFiles);
			if (ItemToSync.IsValid())
			{
				Items.Add(MoveTemp(ItemToSync));
			}
		}
	}

	if (!InFolders.IsEmpty())
	{
		TArray<FName> VirtualPaths;
		VirtualPaths.Reserve(InFolders.Num());

		auto AppendVirtualPath = [&VirtualPaths](FName VirtualPath)
		{
			VirtualPaths.Add(VirtualPath);
			return true;
		};

		for (const FString& Folder : InFolders)
		{
			ContentBrowserData->Legacy_TryConvertPackagePathToVirtualPaths(*Folder, AppendVirtualPath);
		}

		for (const FName& VirtualPath : VirtualPaths)
		{
			FContentBrowserItem ItemToSync = ContentBrowserData->GetItemAtPath(VirtualPath, EContentBrowserItemTypeFilter::IncludeFolders);
			if (ItemToSync.IsValid())
			{
				Items.Add(MoveTemp(ItemToSync));
			}
		}
	}

	return Items;
}

void ContentBrowserUtils::AppendAssetFilterToContentBrowserFilter(const FARFilter& InAssetFilter, const TSharedPtr<FPathPermissionList>& InAssetClassPermissionList, const TSharedPtr<FPathPermissionList>& InFolderPermissionList, FContentBrowserDataFilter& OutDataFilter)
{
	if (InAssetFilter.SoftObjectPaths.Num() > 0 || InAssetFilter.TagsAndValues.Num() > 0 || InAssetFilter.bIncludeOnlyOnDiskAssets)
	{
		FContentBrowserDataObjectFilter& ObjectFilter = OutDataFilter.ExtraFilters.FindOrAddFilter<FContentBrowserDataObjectFilter>();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// TODO: Modify this API to also use FSoftObjectPath with deprecation
		ObjectFilter.ObjectNamesToInclude = UE::SoftObjectPath::Private::ConvertSoftObjectPaths(InAssetFilter.SoftObjectPaths);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		ObjectFilter.TagsAndValuesToInclude = InAssetFilter.TagsAndValues;
		ObjectFilter.bOnDiskObjectsOnly = InAssetFilter.bIncludeOnlyOnDiskAssets;
	}

	if (InAssetFilter.PackageNames.Num() > 0 || InAssetFilter.PackagePaths.Num() > 0 || (InFolderPermissionList && InFolderPermissionList->HasFiltering()))
	{
		FContentBrowserDataPackageFilter& PackageFilter = OutDataFilter.ExtraFilters.FindOrAddFilter<FContentBrowserDataPackageFilter>();
		PackageFilter.PackageNamesToInclude = InAssetFilter.PackageNames;
		PackageFilter.PackagePathsToInclude = InAssetFilter.PackagePaths;
		PackageFilter.bRecursivePackagePathsToInclude = InAssetFilter.bRecursivePaths;
		PackageFilter.PathPermissionList = InFolderPermissionList;
	}

	if (InAssetFilter.ClassPaths.Num() > 0 || (InAssetClassPermissionList && InAssetClassPermissionList->HasFiltering()))
	{
		FContentBrowserDataClassFilter& ClassFilter = OutDataFilter.ExtraFilters.FindOrAddFilter<FContentBrowserDataClassFilter>();
		for (FTopLevelAssetPath ClassPathName : InAssetFilter.ClassPaths)
		{
			ClassFilter.ClassNamesToInclude.Add(ClassPathName.ToString());
		}
		ClassFilter.bRecursiveClassNamesToInclude = InAssetFilter.bRecursiveClasses;
		if (InAssetFilter.bRecursiveClasses)
		{
			for (FTopLevelAssetPath ClassPathName : InAssetFilter.RecursiveClassPathsExclusionSet)
			{
				ClassFilter.ClassNamesToExclude.Add(ClassPathName.ToString());
			}
			ClassFilter.bRecursiveClassNamesToExclude = false;
		}
		ClassFilter.ClassPermissionList = InAssetClassPermissionList;
	}
}

TSharedPtr<FPathPermissionList> ContentBrowserUtils::GetCombinedFolderPermissionList(const TSharedPtr<FPathPermissionList>& FolderPermissionList, const TSharedPtr<FPathPermissionList>& WritableFolderPermissionList)
{
	TSharedPtr<FPathPermissionList> CombinedFolderPermissionList;

	const bool bHidingFolders = FolderPermissionList && FolderPermissionList->HasFiltering();
	const bool bHidingReadOnlyFolders = WritableFolderPermissionList && WritableFolderPermissionList->HasFiltering();
	if (bHidingFolders || bHidingReadOnlyFolders)
	{
		CombinedFolderPermissionList = MakeShared<FPathPermissionList>();

		if (bHidingReadOnlyFolders && bHidingFolders)
		{
			FPathPermissionList IntersectedFilter = FolderPermissionList->CombinePathFilters(*WritableFolderPermissionList.Get());
			CombinedFolderPermissionList->Append(IntersectedFilter);
		}
		else if (bHidingReadOnlyFolders)
		{
			CombinedFolderPermissionList->Append(*WritableFolderPermissionList);
		}
		else if (bHidingFolders)
		{
			CombinedFolderPermissionList->Append(*FolderPermissionList);
		}
	}

	return CombinedFolderPermissionList;
}

bool ContentBrowserUtils::CanDeleteFromAssetView(TWeakPtr<SAssetView> AssetView, FText* OutErrorMsg)
{
	if (TSharedPtr<SAssetView> AssetViewPin = AssetView.Pin())
	{
		const TArray<FContentBrowserItem> SelectedItems = AssetViewPin->GetSelectedItems();

		bool bCanDelete = false;
		for (const FContentBrowserItem& SelectedItem : SelectedItems)
		{
			bCanDelete |= SelectedItem.CanDelete(OutErrorMsg);
		}
		return bCanDelete;
	}
	return false;
}

bool ContentBrowserUtils::CanRenameFromAssetView(TWeakPtr<SAssetView> AssetView, FText* OutErrorMsg)
{
	if (TSharedPtr<SAssetView> AssetViewPin = AssetView.Pin())
	{
		const TArray<FContentBrowserItem> SelectedItems = AssetViewPin->GetSelectedItems();
		return SelectedItems.Num() == 1 && SelectedItems[0].CanRename(nullptr, OutErrorMsg) && !AssetViewPin->IsThumbnailEditMode();
	}
	return false;
}

bool ContentBrowserUtils::CanDeleteFromPathView(TWeakPtr<SPathView> PathView, FText* OutErrorMsg)
{
	if (TSharedPtr<SPathView> PathViewPin = PathView.Pin())
	{
		const TArray<FContentBrowserItem> SelectedItems = PathViewPin->GetSelectedFolderItems();

		bool bCanDelete = false;
		for (const FContentBrowserItem& SelectedItem : SelectedItems)
		{
			bCanDelete |= SelectedItem.CanDelete(OutErrorMsg);
		}
		return bCanDelete;
	}
	return false;
}

bool ContentBrowserUtils::CanRenameFromPathView(TWeakPtr<SPathView> PathView, FText* OutErrorMsg)
{
	if (TSharedPtr<SPathView> PathViewPin = PathView.Pin())
	{
		const TArray<FContentBrowserItem> SelectedItems = PathViewPin->GetSelectedFolderItems();
		return SelectedItems.Num() == 1 && SelectedItems[0].CanRename(nullptr, OutErrorMsg);
	}
	return false;
}

FName ContentBrowserUtils::GetInvariantPath(const FContentBrowserItemPath& ItemPath)
{
	if (!ItemPath.HasInternalPath())
	{
		FName InvariantPath;
		const EContentBrowserPathType AssetPathType = IContentBrowserDataModule::Get().GetSubsystem()->TryConvertVirtualPath(ItemPath.GetVirtualPathName(), InvariantPath);
		if (AssetPathType == EContentBrowserPathType::Virtual)
		{
			return InvariantPath;
		}
		else
		{
			return NAME_None;
		}
	}

	return ItemPath.GetInternalPathName();
}

EContentBrowserIsFolderVisibleFlags ContentBrowserUtils::GetIsFolderVisibleFlags(const bool bDisplayEmpty)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return EContentBrowserIsFolderVisibleFlags::Default | (bDisplayEmpty ? EContentBrowserIsFolderVisibleFlags::None : EContentBrowserIsFolderVisibleFlags::HideEmptyFolders);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool ContentBrowserUtils::IsFavoriteFolder(const FString& FolderPath)
{
	return IsFavoriteFolder(FContentBrowserItemPath(FolderPath, EContentBrowserPathType::Virtual));
}

bool ContentBrowserUtils::IsFavoriteFolder(const FContentBrowserItemPath& FolderPath)
{
	const FName InvariantPath = ContentBrowserUtils::GetInvariantPath(FolderPath);
	if (!InvariantPath.IsNone())
	{
		if (const UContentBrowserFavoriteProjectSettings* FavoriteSettings = UContentBrowserFavoriteProjectSettings::Get())
		{
			const FContentBrowserFavoriteItem Favorite
			{
				.Path = InvariantPath.ToString()
			};
			return FavoriteSettings->FavoriteItems.Contains(Favorite);
		}
	}

	return false;
}

void ContentBrowserUtils::AddFavoriteFolder(const FString& FolderPath, bool bFlushConfig /*= true*/)
{
	AddFavoriteFolder(FContentBrowserItemPath(FolderPath, EContentBrowserPathType::Virtual));
}

void ContentBrowserUtils::AddFavoriteFolder(const FContentBrowserItemPath& FolderPath)
{
	// Insert at the end
	InsertFavoriteFolder(FolderPath, FContentBrowserItemPath(), EItemDropZone::OntoItem);
}

void ContentBrowserUtils::InsertFavoriteFolder(const FContentBrowserItemPath& InFolderPath, const FContentBrowserItemPath& InFolderContext, EItemDropZone InPositionContext)
{
	const FName InvariantPath = ContentBrowserUtils::GetInvariantPath(InFolderPath);
	if (InvariantPath.IsNone())
	{
		return;
	}
	
	const FName TargetFolderPath = ContentBrowserUtils::GetInvariantPath(InFolderContext);
	const FString TargetFolder = TargetFolderPath.ToString();

	// Do not move item on itself
	if (InvariantPath == TargetFolderPath)
	{
		return;
	}
	
	UContentBrowserFavoriteProjectSettings* FavoriteSettings = UContentBrowserFavoriteProjectSettings::Get();
	if (!FavoriteSettings)
	{
		return;
	}
	
	FContentBrowserFavoriteItem FavoriteItem
	{
		.Path = InvariantPath.ToString()
	};
	const int32 CurrentIndex = FavoriteSettings->FavoriteItems.Find(FavoriteItem);
	
	// Item already exists, do not proceed further
	if (CurrentIndex != INDEX_NONE)
	{
		return;
	}
	
	int32 TargetIndex = INDEX_NONE;
	if (!TargetFolderPath.IsNone())
	{
		const FContentBrowserFavoriteItem TargetItem
		{
			.Path = TargetFolder
		};

		TargetIndex = FavoriteSettings->FavoriteItems.Find(TargetItem);
		
		// Target item was not found, do not proceed further
		if (TargetIndex == INDEX_NONE)
		{
			return;
		}
	}
	
	// If no target specified, move it at the end
	if (TargetIndex == INDEX_NONE)
	{
		FavoriteSettings->FavoriteItems.Add(FavoriteItem);
	}
	else
	{
		FavoriteSettings->FavoriteItems.Insert(
			FavoriteItem,
			InPositionContext == EItemDropZone::AboveItem ? TargetIndex : TargetIndex + 1
		);
	}

	FavoriteSettings->SaveConfig();
	FContentBrowserSingleton::Get().BroadcastFavoritesChanged(InFolderPath, /** Added */true);
}

void ContentBrowserUtils::MoveFavoriteFolder(const FContentBrowserItemPath& InFolderPath, const FContentBrowserItemPath& InFolderContext, EItemDropZone InPositionContext)
{
	const FName InvariantPath = ContentBrowserUtils::GetInvariantPath(InFolderPath);
	if (InvariantPath.IsNone())
	{
		return;
	}
	
	const FName TargetFolderPath = ContentBrowserUtils::GetInvariantPath(InFolderContext);
	const FString TargetFolder = TargetFolderPath.ToString();

	// Do not move item on itself
	if (InvariantPath == TargetFolderPath)
	{
		return;
	}
	
	UContentBrowserFavoriteProjectSettings* FavoriteSettings = UContentBrowserFavoriteProjectSettings::Get();
	if (!FavoriteSettings)
	{
		return;
	}
	
	FContentBrowserFavoriteItem FavoriteItem
	{
		.Path = InvariantPath.ToString()
	};
	const int32 CurrentIndex = FavoriteSettings->FavoriteItems.Find(FavoriteItem);
	
	// Item does not exist, do not proceed further
	if (CurrentIndex == INDEX_NONE)
	{
		return;
	}

	int32 TargetIndex = INDEX_NONE;
	if (!TargetFolderPath.IsNone())
	{
		const FContentBrowserFavoriteItem TargetItem
		{
			.Path = TargetFolder
		};

		TargetIndex = FavoriteSettings->FavoriteItems.Find(TargetItem);
		
		// Target item was not found, do not proceed further
		if (TargetIndex == INDEX_NONE)
		{
			return;
		}
	}

	// In case of a move, get the actual item with all its options first then remove it to insert it again
	FavoriteItem = FavoriteSettings->FavoriteItems[CurrentIndex];
	FavoriteSettings->FavoriteItems.RemoveAt(CurrentIndex, EAllowShrinking::No);

	// If no target specified, move it at the end
	if (TargetIndex == INDEX_NONE)
	{
		FavoriteSettings->FavoriteItems.Add(FavoriteItem);
	}
	else
	{
		if (TargetIndex > CurrentIndex)
		{
			TargetIndex -= 1;
		}

		FavoriteSettings->FavoriteItems.Insert(
			FavoriteItem,
			InPositionContext == EItemDropZone::AboveItem ? TargetIndex : TargetIndex + 1
		);
	}

	FavoriteSettings->SaveConfig();
	FContentBrowserSingleton::Get().BroadcastFavoritesChanged(InFolderPath, /** Added */false);
}

FContentBrowserFavoriteItem* ContentBrowserUtils::FindFavoriteFolder(const FContentBrowserItemPath& InFolderPath)
{
	const FName InvariantPath = ContentBrowserUtils::GetInvariantPath(InFolderPath);
	if (InvariantPath.IsNone())
	{
		return nullptr;
	}

	UContentBrowserFavoriteProjectSettings* FavoriteSettings = UContentBrowserFavoriteProjectSettings::Get();
	if (!FavoriteSettings)
	{
		return nullptr;
	}
	
	return FavoriteSettings->FavoriteItems.FindByPredicate(
		[Path = InvariantPath.ToString()](const FContentBrowserFavoriteItem& InItem)
		{
			return InItem.Path.Equals(Path);
		}
	);
}

void ContentBrowserUtils::RemoveFavoriteFolder(const FContentBrowserItemPath& FolderPath)
{
	const FName InvariantPath = ContentBrowserUtils::GetInvariantPath(FolderPath);
	if (InvariantPath.IsNone())
	{
		return;
	}
	
	UContentBrowserFavoriteProjectSettings* FavoriteSettings = UContentBrowserFavoriteProjectSettings::Get();
	if (!FavoriteSettings)
	{
		return;
	}

	const FContentBrowserFavoriteItem FavoriteItem
	{
		.Path = InvariantPath.ToString()
	};

	if (FavoriteSettings->FavoriteItems.Remove(FavoriteItem) > 0)
	{
		FavoriteSettings->SaveConfig();
		FContentBrowserSingleton::Get().BroadcastFavoritesChanged(FolderPath, false);
	}
}

void ContentBrowserUtils::RemoveFavoriteFolder(const FString& FolderPath, bool bFlushConfig)
{
	RemoveFavoriteFolder(FContentBrowserItemPath(FolderPath, EContentBrowserPathType::Virtual));
}

TArray<FString> ContentBrowserUtils::GetFavoriteFolders()
{
	TArray<FString> FavoriteFolders;
	if (const UContentBrowserFavoriteProjectSettings* FavoriteSettings = UContentBrowserFavoriteProjectSettings::Get())
	{
		FavoriteFolders.Reserve(FavoriteSettings->FavoriteItems.Num());
		for (const FContentBrowserFavoriteItem& FavoriteItem : FavoriteSettings->FavoriteItems)
		{
			FavoriteFolders.Emplace(FavoriteItem.Path);
		}
	}
	return FavoriteFolders;
}

void ContentBrowserUtils::AddShowPrivateContentFolder(const FStringView VirtualFolderPath, const FName Owner)
{
	FContentBrowserSingleton& ContentBrowserSingleton = FContentBrowserSingleton::Get();

	if (!ContentBrowserSingleton.IsFolderShowPrivateContentToggleable(VirtualFolderPath))
	{
		return;
	}

	FName InvariantPath;
	IContentBrowserDataModule::Get().GetSubsystem()->TryConvertVirtualPath(VirtualFolderPath, InvariantPath);

	const TSharedPtr<FPathPermissionList>& ShowPrivateContentPermissionList = ContentBrowserSingleton.GetShowPrivateContentPermissionList();

	ShowPrivateContentPermissionList->AddAllowListItem(Owner, InvariantPath);

	ContentBrowserSingleton.SetPrivateContentPermissionListDirty();
}

void ContentBrowserUtils::RemoveShowPrivateContentFolder(const FStringView VirtualFolderPath, const FName Owner)
{
	FContentBrowserSingleton& ContentBrowserSingleton = FContentBrowserSingleton::Get();

	if (!ContentBrowserSingleton.IsFolderShowPrivateContentToggleable(VirtualFolderPath))
	{
		return;
	}

	FName InvariantPath;
	IContentBrowserDataModule::Get().GetSubsystem()->TryConvertVirtualPath(VirtualFolderPath, InvariantPath);

	const TSharedPtr<FPathPermissionList>& ShowPrivateContentPermissionList = ContentBrowserSingleton.GetShowPrivateContentPermissionList();

	ShowPrivateContentPermissionList->RemoveAllowListItem(Owner, InvariantPath);

	ContentBrowserSingleton.SetPrivateContentPermissionListDirty();
}

FAutoConsoleVariable CVarShowCustomVirtualFolderIcon(
	TEXT("ContentBrowser.ShowCustomVirtualFolderIcon"),
	1,
	TEXT("Whether to show a special icon for custom virtual folders added for organizational purposes in the content browser. E.g. EditorCustomVirtualPath field in plugins"));

bool ContentBrowserUtils::ShouldShowCustomVirtualFolderIcon()
{
	return CVarShowCustomVirtualFolderIcon->GetBool();
}

FAutoConsoleVariable CVarShowPluginFolderIcon(
	TEXT("ContentBrowser.ShowPluginFolderIcon"),
	1,
	TEXT("Whether to show a special icon for plugin folders in the content browser."));

bool ContentBrowserUtils::ShouldShowPluginFolderIcon()
{
	return CVarShowPluginFolderIcon->GetBool();
}

bool ContentBrowserUtils::ShouldShowRedirectors(TSharedPtr<SFilterList> Filters)
{
	if (Filters.IsValid())
	{
		TSharedPtr<FFrontendFilter> ShowRedirectorsFilter = Filters->GetFrontendFilter(TEXT("ShowRedirectorsBackend"));
		if (ShowRedirectorsFilter.IsValid())
		{
			return Filters->IsFrontendFilterActive(ShowRedirectorsFilter);
		}
	}
	return false;
}

FContentBrowserInstanceConfig* ContentBrowserUtils::GetContentBrowserConfig(FName InstanceName)
{
	if (InstanceName.IsNone())
	{
		return nullptr;
	}

	FContentBrowserInstanceConfig* Config = UContentBrowserConfig::Get()->Instances.Find(InstanceName);
	if (Config == nullptr)
	{
		return nullptr;
	}

	return Config;
}

FPathViewConfig* ContentBrowserUtils::GetPathViewConfig(FName InstanceName)
{
	if (InstanceName.IsNone())
	{
		return nullptr;
	}

	FContentBrowserInstanceConfig* Config = UContentBrowserConfig::Get()->Instances.Find(InstanceName);
	if (Config == nullptr)
	{
		return nullptr;
	}

	return &Config->PathView;
}

EContentBrowserItemAttributeFilter ContentBrowserUtils::GetContentBrowserItemAttributeFilter(FName InstanceName)
{
	const UContentBrowserSettings* ContentBrowserSettings = GetDefault<UContentBrowserSettings>();
	bool bDisplayEngineContent = ContentBrowserSettings->GetDisplayEngineFolder();
	bool bDisplayPluginContent = ContentBrowserSettings->GetDisplayPluginFolders();
	bool bDisplayDevelopersContent = ContentBrowserSettings->GetDisplayDevelopersFolder();
	bool bDisplayL10NContent = ContentBrowserSettings->GetDisplayL10NFolder();

	// check to see if we have an instance config that overrides the defaults in UContentBrowserSettings
	if (FContentBrowserInstanceConfig* EditorConfig = GetContentBrowserConfig(InstanceName))
	{
		bDisplayEngineContent = EditorConfig->bShowEngineContent;
		bDisplayPluginContent = EditorConfig->bShowPluginContent;
		bDisplayDevelopersContent = EditorConfig->bShowDeveloperContent;
		bDisplayL10NContent = EditorConfig->bShowLocalizedContent;
	}

	return EContentBrowserItemAttributeFilter::IncludeProject
		 | (bDisplayEngineContent ? EContentBrowserItemAttributeFilter::IncludeEngine : EContentBrowserItemAttributeFilter::IncludeNone)
		 | (bDisplayPluginContent ? EContentBrowserItemAttributeFilter::IncludePlugins : EContentBrowserItemAttributeFilter::IncludeNone)
		 | (bDisplayDevelopersContent ? EContentBrowserItemAttributeFilter::IncludeDeveloper : EContentBrowserItemAttributeFilter::IncludeNone)
		 | (bDisplayL10NContent ? EContentBrowserItemAttributeFilter::IncludeLocalized : EContentBrowserItemAttributeFilter::IncludeNone);
}

FContentBrowserItem ContentBrowserUtils::TryGetItemFromUserProvidedPath(FStringView RequestedPathView)
{
	// For all types of accepted input we can trim a trailing slash if it exists
	while (FPathViews::HasRedundantTerminatingSeparator(RequestedPathView))
	{
		RequestedPathView.LeftChopInline(1);
	}

	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	FName RequestedPath(RequestedPathView);

	// If the path is already a valid virtual path, go there
	FContentBrowserItem Item = ContentBrowserData->GetItemAtPath(RequestedPath, EContentBrowserItemTypeFilter::IncludeAll);
	if (Item.IsValid())
	{
		return Item;
	}

	auto ProjectVirtualPath = [](const FContentBrowserItem& Item)
	{
		return Item.GetVirtualPath();
	};

	{
		TArray<FContentBrowserItem> Items = ContentBrowserData->GetItemsAtUserProvidedPath(RequestedPathView, EContentBrowserItemTypeFilter::IncludeAll);
		
		if (!Items.IsEmpty())
		{
			return *Algo::MinElementBy(Items, ProjectVirtualPath, FNameLexicalLess());
		}
	}

	const IAssetTools& AssetTools = IAssetTools::Get();
	UE::Core::FVersePath VersePath;
	if (AssetTools.ShowingContentVersePath())
	{
		UE::Core::FVersePath::TryMake(VersePath, FString(RequestedPathView));
	}

	if (VersePath.IsValid())
	{
		// If the string is a complete Verse path to an asset, sync to that asset
		FAssetData AssetData = AssetTools.FindAssetByVersePath(VersePath);
		if (AssetData.IsValid())
		{
			FNameBuilder NameBuilder;
			AssetData.GetSoftObjectPath().ToString(NameBuilder);

			TArray<FContentBrowserItem> Items = ContentBrowserData->GetItemsAtUserProvidedPath(NameBuilder, EContentBrowserItemTypeFilter::IncludeFiles);

			if (!Items.IsEmpty())
			{
				return *Algo::MinElementBy(Items, ProjectVirtualPath, FNameLexicalLess());
			}
		}

		// If the string is a complete Verse path to a module, sync to that folder
		IPluginManager::Get().EnumerateVersePathMountPoints(VersePath, [ContentBrowserData, &Item](FStringView MountedAssetPath, FStringView LeafPath)
		{
			FNameBuilder NameBuilder;
			NameBuilder.Append(MountedAssetPath);
			if (ensure(NameBuilder.Len() > 0 && NameBuilder.LastChar() == TEXT('/')))
			{
				if (LeafPath.IsEmpty())
				{
					// ContentBrowserData expects no trailing /.
					NameBuilder.RemoveSuffix(1);
				}
				else
				{
					NameBuilder.Append(LeafPath);
				}

				FName VirtualPath;
				ContentBrowserData->ConvertInternalPathToVirtual(NameBuilder, VirtualPath);
				Item = ContentBrowserData->GetItemAtPath(VirtualPath, EContentBrowserItemTypeFilter::IncludeFolders);
			}
			return !Item.IsValid();
		});

		if (Item.IsValid())
		{
			return Item;
		}
	}

	// Try and remove elements from the end of the path until it's a valid virtual path 
	FPathViews::IterateAncestors(RequestedPathView, [RequestedPathView, ContentBrowserData, &Item](FStringView InAncestor){
		if (RequestedPathView == InAncestor)
		{
			return true;
		}
		FName AncestorName(InAncestor);
		Item = ContentBrowserData->GetItemAtPath(AncestorName, EContentBrowserItemTypeFilter::IncludeFolders);
		if (Item.IsValid())
		{
			return false;
		}
		return true;
	});
	if (Item.IsValid())
	{
		return Item;
	}

	return FContentBrowserItem();
}

TArray<FString> ContentBrowserUtils::GetChildPathsFromUserProvidedPath(FStringView RequestedPathView, EContentBrowserItemCategoryFilter ItemCategoryFilter, EContentBrowserItemAttributeFilter ItemAttributeFilter, const FName& ConfigInstanceName, const SPathView& PathViewForFiltering)
{
	// For all types of accepted input, we can trim a trailing slash if it exists.
	while (FPathViews::HasRedundantTerminatingSeparator(RequestedPathView))
	{
		RequestedPathView.LeftChopInline(1);
	}

	UContentBrowserDataSubsystem* const ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

	TArray<FContentBrowserItem> Items;
	enum class EUserProvidedPathType
	{
		None,
		VirtualPath,
		PackagePath,
		VersePath,
	} PathType = EUserProvidedPathType::None;
	{
		FName RequestedPath(RequestedPathView);

		// If the path is a valid virtual path, use that.
		FContentBrowserItem Item = ContentBrowserData->GetItemAtPath(RequestedPath, EContentBrowserItemTypeFilter::IncludeFolders);
		if (Item.IsValid())
		{
			Items.Add(MoveTemp(Item));
			PathType = EUserProvidedPathType::VirtualPath;
		}
		else
		{
			// Otherwise it may be some other type of valid path we understand, such as a package path.
			Items = ContentBrowserData->GetItemsAtUserProvidedPath(RequestedPathView, EContentBrowserItemTypeFilter::IncludeFolders);

			if (!Items.IsEmpty())
			{
				// At the time of authoring the only possible option that GetItemsAtUserProvidedPath could have parsed is a package path.
				PathType = EUserProvidedPathType::PackagePath;
			}
			else // Lastly, it could be a Verse path.
			{
				const IAssetTools& AssetTools = IAssetTools::Get();
				UE::Core::FVersePath VersePath;
				if (AssetTools.ShowingContentVersePath())
				{
					UE::Core::FVersePath::TryMake(VersePath, FString(RequestedPathView));
				}

				if (VersePath.IsValid())
				{
					IPluginManager::Get().EnumerateVersePathMountPoints(VersePath, [ContentBrowserData, &Items, &PathType](FStringView MountedAssetPath, FStringView LeafPath)
					{
						FNameBuilder NameBuilder;
						NameBuilder.Append(MountedAssetPath);
						if (ensure(NameBuilder.Len() > 0 && NameBuilder.LastChar() == TEXT('/')))
						{
							if (LeafPath.IsEmpty())
							{
								// ContentBrowserData expects no trailing /.
								NameBuilder.RemoveSuffix(1);
							}
							else
							{
								NameBuilder.Append(LeafPath);
							}

							FName VirtualPath;
							ContentBrowserData->ConvertInternalPathToVirtual(NameBuilder, VirtualPath);
							FContentBrowserItem Item = ContentBrowserData->GetItemAtPath(VirtualPath, EContentBrowserItemTypeFilter::IncludeFolders);
							if (Item.IsValid())
							{
								Items.Add(MoveTemp(Item));
								PathType = EUserProvidedPathType::VersePath;
							}
						}
						return true;
					});
				}
			}
		}
	}

	if (Items.IsEmpty())
	{
		return {};
	}

	const UContentBrowserSettings* const ContentBrowserSettings = GetDefault<UContentBrowserSettings>();
	bool bDisplayEmptyFolders = ContentBrowserSettings->DisplayEmptyFolders;
	// Check to see if we have an instance config that overrides the default in UContentBrowserSettings.
	if (const FContentBrowserInstanceConfig* EditorConfig = GetConstInstanceConfig(ConfigInstanceName))
	{
		bDisplayEmptyFolders = EditorConfig->bShowEmptyFolders;
	}

	const EContentBrowserIsFolderVisibleFlags IsFolderVisibleFlags = ContentBrowserUtils::GetIsFolderVisibleFlags(bDisplayEmptyFolders);

	FContentBrowserFolderContentsFilter FolderFilter;
	if (bDisplayEmptyFolders)
	{
		FolderFilter.HideFolderIfEmptyFilter = ContentBrowserData->CreateHideFolderIfEmptyFilter();
	}
	else
	{
		FolderFilter.ItemCategoryFilter = ItemCategoryFilter;
	}

	FContentBrowserDataFilter SubItemsFilter;
	SubItemsFilter.ItemTypeFilter = EContentBrowserItemTypeFilter::IncludeFolders;
	SubItemsFilter.bRecursivePaths = false;
	SubItemsFilter.ItemCategoryFilter = ItemCategoryFilter;
	SubItemsFilter.ItemAttributeFilter = ItemAttributeFilter;

	TArray<FString> Results;

	for (const FContentBrowserItem& Item : Items)
	{
		TArray<FContentBrowserItem> SubItems = ContentBrowserData->GetItemsUnderPath(Item.GetVirtualPath(), SubItemsFilter);

		for (const FContentBrowserItem& SubItem : SubItems)
		{
			// Filter subitems we can't see.
			if (!SubItem.GetInternalPath().IsNone())
			{
				if (!PathViewForFiltering.InternalPathPassesBlockLists(FNameBuilder(SubItem.GetInternalPath())))
				{
					continue;
				}
			}
			else if (!VirtualPathPassesBlockLists(SubItem, PathViewForFiltering))
			{
				continue;
			}

			if (!ContentBrowserData->IsFolderVisible(SubItem.GetVirtualPath(), IsFolderVisibleFlags, FolderFilter))
			{
				continue;
			}

			FNameBuilder PathBuilder;
			UE::Core::FVersePath VersePath;
			FStringView PathView;

			// Get the subitem path matching the requested path format.
			switch (PathType)
			{
			case EUserProvidedPathType::VirtualPath:
				SubItem.GetVirtualPath().AppendString(PathBuilder);
				PathView = PathBuilder;
				break;
			case EUserProvidedPathType::PackagePath:
			{
				FName PackagePath;
				if (SubItem.Legacy_TryGetPackagePath(PackagePath))
				{
					PackagePath.AppendString(PathBuilder);
					PathView = PathBuilder;
					break;
				}
				continue;
			}
			case EUserProvidedPathType::VersePath:
			{
				FName PackagePath;
				if (SubItem.Legacy_TryGetPackagePath(PackagePath))
				{
					PackagePath.AppendString(PathBuilder);
					VersePath = FPackageName::LongPackagePathToVersePath(PathBuilder);
					if (VersePath.IsValid())
					{
						PathView = VersePath.AsStringView();
						break;
					}
				}
				continue;
			}
			default:
				checkNoEntry();
				break;
			}

			// Verify that the new path is indeed syntactically a subpath.
			if (PathView.StartsWith(RequestedPathView) &&
				(RequestedPathView.EndsWith(TEXT('/')) || (PathView.Len() > RequestedPathView.Len() && PathView[RequestedPathView.Len()] == TEXT('/'))))
			{
				Results.Emplace(PathView);
			}
		}
	}

	// Check for duplicates - these should only be likely with Verse paths, but we don't know the internal implementation of custom data sources, so we'll always check just to be sure.
	Results.Sort();
	Results.SetNum(Algo::Unique(Results));

	return Results;
}

namespace ContentBrowserUtils
{
	static bool bEditNavigationBarPathAsVirtualPath = false;
	static FAutoConsoleVariableRef CVarShowContentSourcesBar(
		TEXT("ContentBrowser.UI.EditNavigationBarPathAsVirtualPath"),
		bEditNavigationBarPathAsVirtualPath,
		TEXT("When editing text in the navigation bar, edit the text as the virtual path if set, otherwise edit it as its native representation if available, such as a package path."));
}

FText ContentBrowserUtils::GetUserFacingPathFromVirtualPath(FName VirtualPath)
{
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	FContentBrowserItem Item = ContentBrowserData->GetItemAtPath(VirtualPath, EContentBrowserItemTypeFilter::IncludeFolders);
	if (Item.IsValid())
	{
		if (!bEditNavigationBarPathAsVirtualPath)
		{
			FName PackagePath;
			if (Item.Legacy_TryGetPackagePath(PackagePath))
			{
				if (IAssetTools::Get().ShowingContentVersePath())
				{
					// Check to see if the mount point maps to a unique Verse path and return that.
					FNameBuilder PackagePathBuilder(PackagePath);
					FStringView PackagePathView(PackagePathBuilder);

					// If the mount point is invalid, we can't create a Verse path from it.
					bool bHadClassesPrefix = false;
					const FStringView MountPointName = FPathViews::GetMountPointNameFromPath(PackagePathView, &bHadClassesPrefix);
					if (!MountPointName.IsEmpty() && !bHadClassesPrefix)
					{
						IPluginManager& PluginManager = IPluginManager::Get();

						// If the package path isn't mounted under a plugin, the package path doesn't have a Verse path.
						TSharedPtr<IPlugin> Plugin = PluginManager.FindPlugin(MountPointName);
						if (Plugin)
						{
							// If the plugin doesn't have a root Verse path, the package path doesn't have a Verse path.
							UE::Core::FVersePath PluginVersePath;
							if (UE::Core::FVersePath::TryMake(PluginVersePath, Plugin->GetVersePath()))
							{
								// Make sure there are no other plugins with the same Verse path.
								if (Algo::NoneOf(PluginManager.GetEnabledPluginsWithContent(), [&Plugin, &PluginVersePath](const TSharedPtr<IPlugin>& OtherPlugin)
								{
									if (Plugin == OtherPlugin)
									{
										return false;
									}

									UE::Core::FVersePath OtherPluginVersePath;
									if (!UE::Core::FVersePath::TryMake(OtherPluginVersePath, OtherPlugin->GetVersePath()))
									{
										return false;
									}

									return PluginVersePath == OtherPluginVersePath;
								}))
								{
									const int32 CharCount = MountPointName.Len() + 1;

									// If the package path is the mount point, return the mount point's Verse path.
									if (PackagePathView.Len() == CharCount)
									{
										return MoveTemp(PluginVersePath).AsText();
									}

									// Otherwise append the leaf path.
									UE::Core::FVersePath VersePath;
									if (UE::Core::FVersePath::TryMake(VersePath, FPaths::Combine(PluginVersePath.AsStringView(), PackagePathView.RightChop(CharCount))))
									{
										return MoveTemp(VersePath).AsText();
									}
								}
							}
						}
					}
				}

				// If not showing Verse paths, or the Verse path is not unique, prefer the package path.
				return FText::FromName(PackagePath);
			}
		}
		return FText::FromName(VirtualPath);
	}
	return FText::GetEmpty();
}

FString ContentBrowserUtils::FormatCollectionCrumbData(const ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection)
{
	return FString::Printf(
		TEXT("%s?%s?%s"),
		*CollectionContainer.GetCollectionSource()->GetName().ToString(),
		*Collection.Name.ToString(),
		*FString::FromInt(Collection.Type));
}

void ContentBrowserUtils::UpdateNavigationBar(const TSharedPtr<SNavigationBar>& NavigationBar, const TSharedPtr<SAssetView>& AssetView, const TSharedPtr<SPathView>& PathView)
{
	const FAssetViewContentSources& ContentSources = AssetView->GetContentSources();

	NavigationBar->ClearPaths();

	if (ContentSources.HasVirtualPaths())
	{
		UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

		TArray<FString> Crumbs;
		ContentSources.GetVirtualPaths()[0].ToString().ParseIntoArray(Crumbs, TEXT("/"), true);

		FContentBrowserDataFilter SubItemsFilter;
		SubItemsFilter.ItemTypeFilter = EContentBrowserItemTypeFilter::IncludeFolders;
		SubItemsFilter.bRecursivePaths = false;
		SubItemsFilter.ItemCategoryFilter = PathView->GetContentBrowserItemCategoryFilter();
		SubItemsFilter.ItemAttributeFilter = PathView->GetContentBrowserItemAttributeFilter();

		FString CrumbPath = TEXT("/");
		for (const FString& Crumb : Crumbs)
		{
			CrumbPath += Crumb;

			bool bHasSubItems = false;
			ContentBrowserData->EnumerateItemsUnderPath(*CrumbPath, SubItemsFilter, [&bHasSubItems](FContentBrowserItemData&& InSubItem)
			{
				bHasSubItems = true;
				return false;
			});

			const FContentBrowserItem CrumbFolderItem = ContentBrowserData->GetItemAtPath(*CrumbPath, EContentBrowserItemTypeFilter::IncludeFolders);
			NavigationBar->PushPath(CrumbFolderItem.IsValid() ? CrumbFolderItem.GetDisplayName() : FText::FromString(Crumb), CrumbPath, bHasSubItems);

			CrumbPath += TEXT("/");
		}
	}
	else if (ContentSources.HasCollections())
	{
		const FCollectionRef& Collection = ContentSources.GetCollections()[0];

		TArray<FCollectionNameType> CollectionPathItems;

		// Walk up the parents of this collection so that we can generate a complete path (this loop also adds the child collection to the array)
		for (TOptional<FCollectionNameType> CurrentCollection = FCollectionNameType(Collection.Name, Collection.Type);
			CurrentCollection.IsSet(); 
			CurrentCollection = Collection.Container->GetParentCollection(CurrentCollection->Name, CurrentCollection->Type))
		{
			CollectionPathItems.Insert(CurrentCollection.GetValue(), 0);
		}

		// Now add each part of the path to the breadcrumb trail
		for (const FCollectionNameType& CollectionPathItem : CollectionPathItems)
		{
			const FString CrumbData = FormatCollectionCrumbData(*Collection.Container, CollectionPathItem);
			
			TArray<FCollectionNameType> ChildCollections;
			Collection.Container->GetChildCollections(CollectionPathItem.Name, CollectionPathItem.Type, ChildCollections);
			const bool bHasChildren = ChildCollections.Num() > 0;

			FFormatNamedArguments Args;
			Args.Add(TEXT("CollectionName"), FText::FromName(CollectionPathItem.Name));
			const FText DisplayName = FText::Format(LOCTEXT("CollectionPathIndicator", "{CollectionName} (Collection)"), Args);

			NavigationBar->PushPath(DisplayName, CrumbData, bHasChildren);
		}
	}
	else
	{
		NavigationBar->PushPath(LOCTEXT("AllAssets", "All Assets"), TEXT(""), true); 
	}
}

TArray<FContentBrowserItem> ContentBrowserUtils::FilterOrAliasItems(TArrayView<const FContentBrowserItem> Items, const SPathView& PathViewForFiltering)
{
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	const TSharedRef<FPathPermissionList>& FolderPermissions = FAssetToolsModule::GetModule().Get().GetFolderPermissionList();

	// If any of the items to sync don't pass the permission filter, try to find an alias that might be visible
	TArray<FContentBrowserItem> NewItems;
	NewItems.Reserve(Items.Num());
	for (const FContentBrowserItem& Item : Items)
	{
		if (Item.GetInternalPath().IsNone())
		{
			if (VirtualPathPassesBlockLists(Item, PathViewForFiltering))
			{
				NewItems.Add(Item);
			}
		}
		else if (FolderPermissions->PassesStartsWithFilter(Item.GetInternalPath()))
		{
			NewItems.Add(Item);
		}
		else
		{
			TArray<FContentBrowserItemPath> Aliases = ContentBrowserData->GetAliasesForPath(Item.GetInternalPath());
			for (const FContentBrowserItemPath& Alias : Aliases)
			{
				if (FolderPermissions->PassesStartsWithFilter(Alias.GetInternalPathName()))
				{
					FContentBrowserItem AliasedItem = ContentBrowserData->GetItemAtPath(Alias.GetVirtualPathName(), EContentBrowserItemTypeFilter::IncludeFiles);
					if (AliasedItem.IsValid())
					{
						NewItems.Add(MoveTemp(AliasedItem));
						break;
					}
				}
			}
		}
	}
	
	return NewItems;
}

const FContentBrowserInstanceConfig* ContentBrowserUtils::GetConstInstanceConfig(const FName& ForInstance) 
{
	if (ForInstance.IsNone())
	{
		return nullptr;
	}

	UContentBrowserConfig* Config = UContentBrowserConfig::Get();
	if (Config == nullptr)
	{
		return nullptr;
	}

	const FContentBrowserInstanceConfig* InstanceConfig = Config->Instances.Find(ForInstance);
	return InstanceConfig;
}

TArray<FContentBrowserItem> ContentBrowserUtils::GetChildItemsFromVirtualPath(
	const FName& Path,
	EContentBrowserItemCategoryFilter ItemCategoryFilter, 
	EContentBrowserItemAttributeFilter ItemAttributeFilter,
	const FName& ConfigInstanceName,
	const SPathView& PathViewForFiltering 
)
{
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

	const UContentBrowserSettings* ContentBrowserSettings = GetDefault<UContentBrowserSettings>();
	bool bDisplayEmpty = ContentBrowserSettings->DisplayEmptyFolders;
	// check to see if we have an instance config that overrides the default in UContentBrowserSettings
	if (const FContentBrowserInstanceConfig* EditorConfig = GetConstInstanceConfig(ConfigInstanceName))
	{
		bDisplayEmpty = EditorConfig->bShowEmptyFolders;
	}
	
	EContentBrowserIsFolderVisibleFlags FolderFlags = ContentBrowserUtils::GetIsFolderVisibleFlags(bDisplayEmpty);

	FContentBrowserFolderContentsFilter FolderFilter;
	if (bDisplayEmpty)
	{
		FolderFilter.HideFolderIfEmptyFilter = ContentBrowserData->CreateHideFolderIfEmptyFilter();
	}
	else
	{
		FolderFilter.ItemCategoryFilter = ItemCategoryFilter;
	}

	FContentBrowserDataFilter SubItemsFilter;
	SubItemsFilter.ItemTypeFilter = EContentBrowserItemTypeFilter::IncludeFolders;
	SubItemsFilter.bRecursivePaths = false;
	SubItemsFilter.ItemCategoryFilter = ItemCategoryFilter;
	SubItemsFilter.ItemAttributeFilter = ItemAttributeFilter;

	TArray<FContentBrowserItem> SubItems = ContentBrowserData->GetItemsUnderPath(Path, SubItemsFilter);

	for (auto It = SubItems.CreateIterator(); It; ++It)
	{
		const FContentBrowserItem& Item = *It;
		if (!Item.GetInternalPath().IsNone())
		{
			if (!PathViewForFiltering.InternalPathPassesBlockLists(FNameBuilder(Item.GetInternalPath())))
			{
				It.RemoveCurrent();
				continue;
			}
		}
		else if (!VirtualPathPassesBlockLists(Item, PathViewForFiltering))
		{
			It.RemoveCurrent();
			continue;
		}

		if (!ContentBrowserData->IsFolderVisible(Item.GetVirtualPath(), FolderFlags, FolderFilter))
		{
			It.RemoveCurrent();
			continue;
		}
	}

	return SubItems;
}

TSharedPtr<SWidget> ContentBrowserUtils::GetFolderWidgetForNavigationBar(const FText& InFolderName, const FName& InFolderBrushName, const FLinearColor& InFolderColor)
{
	const FSlateBrush* FolderBrush = FAppStyle::GetBrush(InFolderBrushName);

	if (FolderBrush != FAppStyle::GetDefaultBrush())
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2, 0, 6, 0))
			[
				SNew(SBox)
				.WidthOverride(16.f)
				.HeightOverride(16.f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16, 16))
					.Image(FolderBrush)
					.ColorAndOpacity(InFolderColor)
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.Padding(FMargin(2, 0, 6, 0))
			.VAlign( VAlign_Center )
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), TEXT("Menu.Label"))
				.Text(InFolderName)
			];
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
