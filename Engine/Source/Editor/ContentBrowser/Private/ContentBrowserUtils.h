// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetViewUtils.h"
#include "CollectionManagerTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FPathPermissionList;
class FSlateRect;
class ICollectionContainer;
class SAssetView;
class SFilterList;
class SPathView;
class SWidget;
class SNavigationBar;
struct FARFilter;
struct FAssetData;
struct FContentBrowserDataFilter;
struct FContentBrowserFavoriteItem;
struct FContentBrowserInstanceConfig;
struct FContentBrowserItem;
struct FContentBrowserItemPath;
struct FPathViewConfig;
enum class EContentBrowserIsFolderVisibleFlags : uint8;
enum class EContentBrowserItemAttributeFilter : uint8;
enum class EContentBrowserItemCategoryFilter : uint8;
enum class EItemDropZone;

namespace ContentBrowserUtils
{
	// Import the functions that were moved into the more common AssetViewUtils namespace
	using namespace AssetViewUtils;

	// Display Message type, mainly used to decide the icon to use
	enum class EDisplayMessageType
	{
		Successful,
		Info,
		Warning,
		Error
	};

	/** Displays a modeless message at the specified anchor. It is fine to specify a zero-size anchor, just use the top and left fields */
	void DisplayMessage(const FText& Message, const FSlateRect& ScreenAnchor, const TSharedRef<SWidget>& ParentContent, EDisplayMessageType InMessageType);

	/** Displays a modeless message asking yes or no type question */
	void DisplayConfirmationPopup(const FText& Message, const FText& YesString, const FText& NoString, const TSharedRef<SWidget>& ParentContent, const FOnClicked& OnYesClicked, const FOnClicked& OnNoClicked = FOnClicked());

	/** Displays a modal dialog asking yes or no question (blocking) */
	bool DisplayConfirmationDialog(const FText& InTitle, const FText& InMessage);

	/** Returns references to the specified items */
	FString GetItemReferencesText(const TArray<FContentBrowserItem>& Items);

	/** Returns object path of the specified items */
	FString GetItemObjectPathText(const TArray<FContentBrowserItem>& Items);

	/** Returns package name of the specified items */
	FString GetItemPackageNameText(const TArray<FContentBrowserItem>& Items);

	/** Returns references to the specified folders */
	FString GetFolderReferencesText(const TArray<FContentBrowserItem>& Folders);

	/** Copies references to the specified items to the clipboard */
	void CopyItemReferencesToClipboard(const TArray<FContentBrowserItem>& ItemsToCopy);

	/** Copies object path of the specified items to the clipboard */
	void CopyItemObjectPathToClipboard(const TArray<FContentBrowserItem>& ItemsToCopy);

	/** Copies package name of the specified items to the clipboard */
	void CopyItemPackageNameToClipboard(const TArray<FContentBrowserItem>& ItemsToCopy);

	/** Copies references to the specified folders to the clipboard */
	void CopyFolderReferencesToClipboard(const TArray<FContentBrowserItem>& FoldersToCopy);

	/** Copies file paths on disk to the specified items to the clipboard */
	void CopyFilePathsToClipboard(const TArray<FContentBrowserItem>& ItemsToCopy);

	/** Check whether the given item is considered to be developer content */
	bool IsItemDeveloperContent(const FContentBrowserItem& InItem);

	/** Check whether the given item is considered to be localized content */
	bool IsItemLocalizedContent(const FContentBrowserItem& InItem);

	/** Check whether the given item is considered to be engine content (including engine plugins) */
	bool IsItemEngineContent(const FContentBrowserItem& InItem);

	/** Check whether the given item is considered to be project content (including project plugins) */
	bool IsItemProjectContent(const FContentBrowserItem& InItem);

	/** Check whether the given item is considered to be plugin content (engine or project) */
	bool IsItemPluginContent(const FContentBrowserItem& InItem);

	/** Check whether the given item is the root folder of a plugin */
	bool IsItemPluginRootFolder(const FContentBrowserItem& InItem);

	/** Given a Folder, work out the BrushName and ShadowBrushName to use for it */
	bool TryGetFolderBrushAndShadowName(const FContentBrowserItem& InFolder, FName& OutBrushName, FName& OutShadowBrushName);

	/** Given a Folder, work out the small version of the BrushName and ShadowBrushName to use for it */
	bool TryGetFolderBrushAndShadowNameSmall(const FContentBrowserItem& InFolder, FName& OutBrushName, FName& OutShadowBrushName);

	/** Check to see whether the given path is rooted against a collection directory, optionally extracting the collection container, name and share type from the path */
	bool IsCollectionPath(const FString& InPath, TSharedPtr<ICollectionContainer>* OutCollectionContainer = nullptr, FName* OutCollectionName = nullptr, ECollectionShareType::Type* OutCollectionShareType = nullptr);

	/** Given an array of paths, work out how many are rooted against class roots, and how many are rooted against asset roots */
	void CountPathTypes(const TArray<FString>& InPaths, int32& OutNumAssetPaths, int32& OutNumClassPaths);

	/** Given an array of paths, work out how many are rooted against class roots, and how many are rooted against asset roots */
	void CountPathTypes(const TArray<FName>& InPaths, int32& OutNumAssetPaths, int32& OutNumClassPaths);

	/** Given an array of "asset" data, work out how many are assets, and how many are classes */
	void CountItemTypes(const TArray<FAssetData>& InItems, int32& OutNumAssetItems, int32& OutNumClassItems);

	/** Gets the platform specific text for the "explore" command (FPlatformProcess::ExploreFolder) */
	FText GetExploreFolderText();

	/** Perform a batched "explore" operation on the specified file and/or folder paths */
	void ExploreFolders(const TArray<FContentBrowserItem>& InItems, const TSharedRef<SWidget>& InParentContent);

	/** Returns if can perform a batched "explore" operation on the specified file and/or folder paths */
	bool CanExploreFolders(const TArray<FContentBrowserItem>& InItems);

	/** Convert a legacy asset and path selection to their corresponding content browser data items */
	TArray<FContentBrowserItem> ConvertLegacySelectionToItems(TArrayView<const FAssetData> InAssets, TArrayView<const FString> InFolders);

	/** Append the asset registry filter and permission lists to the content browser data filter */
	void AppendAssetFilterToContentBrowserFilter(const FARFilter& InAssetFilter, const TSharedPtr<FPathPermissionList>& InAssetClassPermissionList, const TSharedPtr<FPathPermissionList>& InFolderPermissionList, FContentBrowserDataFilter& OutDataFilter);

	/* Combine folder filters into a new filter if either are active */
	TSharedPtr<FPathPermissionList> GetCombinedFolderPermissionList(const TSharedPtr<FPathPermissionList>& FolderPermissionList, const TSharedPtr<FPathPermissionList>& WritableFolderPermissionList);

	/** Shared logic to know if we can perform certain operation depending on which view it occurred, either PathView or AssetView */
	bool CanDeleteFromAssetView(TWeakPtr<SAssetView> AssetView, FText* OutErrorMsg = nullptr);
	bool CanRenameFromAssetView(TWeakPtr<SAssetView> AssetView, FText* OutErrorMsg = nullptr);
	bool CanDeleteFromPathView(TWeakPtr<SPathView> PathView, FText* OutErrorMsg = nullptr);
	bool CanRenameFromPathView(TWeakPtr<SPathView> PathView, FText* OutErrorMsg = nullptr);

	/** Returns internal path if it has one, otherwise strips /All prefix from virtual path*/
	FName GetInvariantPath(const FContentBrowserItemPath& ItemPath);

	/** Get the set of flags to use with IsFolderVisible */
	EContentBrowserIsFolderVisibleFlags GetIsFolderVisibleFlags(const bool bDisplayEmpty);

	/** Returns if this folder has been marked as a favorite folder */
	UE_DEPRECATED(5.3, "Use function that takes FContentBrowserItemPath instead.")
	bool IsFavoriteFolder(const FString& FolderPath);
	bool IsFavoriteFolder(const FContentBrowserItemPath& FolderPath);

	UE_DEPRECATED(5.3, "Use function that takes FContentBrowserItemPath instead.")
	void AddFavoriteFolder(const FString& FolderPath, bool bFlushConfig = true);
	void AddFavoriteFolder(const FContentBrowserItemPath& FolderPath);
	
	/** Adds a new favorite folder at a specific position */
	void InsertFavoriteFolder(const FContentBrowserItemPath& InFolderPath, const FContentBrowserItemPath& InFolderContext, EItemDropZone InPositionContext);
	
	/** Moves an existing favorite folder at a new position */
	void MoveFavoriteFolder(const FContentBrowserItemPath& InFolderPath, const FContentBrowserItemPath& InFolderContext, EItemDropZone InPositionContext);

	/** Finds existing favorite item based on path */
	FContentBrowserFavoriteItem* FindFavoriteFolder(const FContentBrowserItemPath& InFolderPath);
	
	UE_DEPRECATED(5.3, "Use function that takes FContentBrowserItemPath instead.")
	void RemoveFavoriteFolder(const FString& FolderPath, bool bFlushConfig = true);
	void RemoveFavoriteFolder(const FContentBrowserItemPath& FolderPath);

	TArray<FString> GetFavoriteFolders();

	/** Adds FolderPath as a private content edit folder if it's allowed to be toggled as such */
	void AddShowPrivateContentFolder(const FStringView VirtualFolderPath, const FName Owner);

	/** Removes FolderPath as a private content edit folder if it's allowed to be toggled as such */
	void RemoveShowPrivateContentFolder(const FStringView VirtualFolderPath, const FName Owner);

	/** Returns whether we should display icons for custom virtual folders in the content browser */
	bool ShouldShowCustomVirtualFolderIcon();

	/** Returns whether we should display icons for plugins in the content browser */
	bool ShouldShowPluginFolderIcon();
	
	/** Returns whether the content browser should be showing redirectors based on the current filter state or content browser settings */
	bool ShouldShowRedirectors(TSharedPtr<SFilterList> Filters);

	/**
	 * Returns config settings for the given content browser name.
	 * A name of none returns null.
	 */
	FContentBrowserInstanceConfig* GetContentBrowserConfig(FName InstanceName);
	
	/** 
	 * returns config settings for the path view for the given content browser name.
	 * A name of none returns null.
	 */
	FPathViewConfig* GetPathViewConfig(FName InstanceName);

	/**
	 * Returns the attribute filter to use when retrieving conent browser data for the given instance.
	 * An instance name of None gives global settings.
	 */
	EContentBrowserItemAttributeFilter GetContentBrowserItemAttributeFilter(FName InstanceName);

	/** 
	 * Try and find a valid content browser item from a path provided from user text input.
	 * Examples of paths:
	 * 	- A full virtual folder or item path such as /All/Game/Maps /All/Game/Maps/Arena.Arena
	 *  - An internal asset path such as /Game/Maps/Arena.Arena
	 *  - An internal package name such as /Game/Maps/Arena 
	 * 	- An internal package path such as /Game/Maps
	 * 	- An 'export text' path / asset reference such as /Script/Engine.World'/Game/Maps/Arena.Arena'
	 *  - A filesystem path to a uasset or umap file.
	 */
	FContentBrowserItem TryGetItemFromUserProvidedPath(FStringView RequestedPathView);

	/** Finds the child paths that are children of the supplied path, while obeying the supplied filters. */
	TArray<FString> GetChildPathsFromUserProvidedPath(FStringView RequestedPathView, EContentBrowserItemCategoryFilter ItemCategoryFilter, EContentBrowserItemAttributeFilter ItemAttributeFilter, const FName& ConfigInstanceName, const SPathView& PathViewForFiltering);

	/** Gets the actual path the user should edit when they start editing a virtual path in the navigation bar */
	FText GetUserFacingPathFromVirtualPath(FName VirtualPath);

	FString FormatCollectionCrumbData(const ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection);
	
	/** Updates a navigation bar based on the current state of an asset view and path view */
	void UpdateNavigationBar(const TSharedPtr<SNavigationBar>& NavigationBar, const TSharedPtr<SAssetView>& AssetView, const TSharedPtr<SPathView>& PathView);

	/** Filter a list of items based on permissions or find a valid alias. */
	TArray<FContentBrowserItem> FilterOrAliasItems(TArrayView<const FContentBrowserItem> Items, const SPathView& PathViewForFiltering);

	/** Retrieves a named config. */
	const FContentBrowserInstanceConfig* GetConstInstanceConfig(const FName& ForInstance);

	/** Finds the content items that are children of the supplied path, while obeying the supplied filters. */
	TArray<FContentBrowserItem> GetChildItemsFromVirtualPath(const FName& Path, EContentBrowserItemCategoryFilter ItemCategoryFilter, EContentBrowserItemAttributeFilter ItemAttributeFilter, const FName& ConfigInstanceName, const SPathView& PathViewForFiltering);

	/** Creates a widget that represents a folder with the given name, brush, and color. */
	TSharedPtr<SWidget> GetFolderWidgetForNavigationBar(const FText& InFolderName, const FName& InFolderBrushName, const FLinearColor& InFolderColor);
}
