// Copyright Epic Games, Inc. All Rights Reserved.

#include "SContentSource.h"

#include "ContentSources/Columns/ContentSourcesColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

namespace UE::Editor::ContentBrowser
{
	namespace Private
	{
		static const DataStorage::IUiProvider::FPurposeID
			Purpose(DataStorage::IUiProvider::FPurposeInfo("ContentBrowser", "AssetView", NAME_None).GeneratePurposeID());
	}
	
	void SContentSource::Construct(const FArguments& InArgs)
    {
		using namespace UE::Editor::DataStorage;
		
		DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		DataStorageUi = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);

		ChildSlot
		[
			CreateWidget()
		];
    }

	void SContentSource::SetContentSource(const TSharedPtr<IContentSource>& InContentSource)
	{
		// Let the old content source know it is being disabled
		if (ContentSource)
		{
			ContentSource->OnContentSourceDisabled();
		}
		
		ContentSource = InContentSource;

		// Let the new Content Source know it has been enabled
		if (ContentSource)
		{
			ContentSource->OnContentSourceEnabled();
		}

		ChildSlot
		[
			CreateWidget()
		];
	}

	TSharedRef<SWidget> SContentSource::CreateWidget()
	{
		// For now, if there was no valid content source provided we simply display nothing
		if (!ContentSource)
		{
			return SNullWidget::NullWidget;
		}
		
		using namespace UE::Editor::DataStorage;

		// Otherwise we display any widget constructors registered for the "ContentBrowser.AssetView" purpose currently
		IUiProvider::FWidgetConstructorPtr AssetViewWidgetConstructor;

		auto AssignWidgetToColumn = [&AssetViewWidgetConstructor] (IUiProvider::FWidgetConstructorPtr Constructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
		{
			AssetViewWidgetConstructor = MoveTemp(Constructor);;
			return false;
		};

		DataStorageUi->CreateWidgetConstructors(DataStorageUi->FindPurpose(Private::Purpose), FMetaDataView(), AssignWidgetToColumn);

		TSharedPtr<SWidget> AssetViewWidget;
		
		if (AssetViewWidgetConstructor)
		{
			RowHandle WidgetRow = DataStorage->AddRow(DataStorage->FindTable(FName("Editor_WidgetTable")));

			// Add FContentSourceColumn so the widget can use it to know which content source is active
			DataStorage->AddColumn(WidgetRow, FContentSourceColumn{.ContentSource = ContentSource});
			
			AssetViewWidget = DataStorageUi->ConstructWidget(WidgetRow, *AssetViewWidgetConstructor, FMetaDataView());
		}

		if (!AssetViewWidget)
		{
			return SNullWidget::NullWidget;
		}

		return AssetViewWidget.ToSharedRef();
	}
}

