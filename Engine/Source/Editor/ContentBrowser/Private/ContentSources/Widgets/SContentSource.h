// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentSources/IContentSource.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
	class IUiProvider;
} // namespace UE::Editor::DataStorage

namespace UE::Editor::ContentBrowser
{
	// Displays the widgets for a single content source that is currently active
	class SContentSource : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SContentSource) {}
		
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);
		void SetContentSource(const TSharedPtr<IContentSource>& ContentSource);

	private:
		TSharedRef<SWidget> CreateWidget();

	private:
		// The currently active content source
		TSharedPtr<IContentSource> ContentSource;
		DataStorage::ICoreProvider* DataStorage = nullptr; 
		DataStorage::IUiProvider* DataStorageUi = nullptr;
	};

}