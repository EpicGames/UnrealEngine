// Copyright Epic Games, Inc. All Rights Reserved.

#include "SContentSourcesView.h"

#include "ContentBrowserStyle.h"
#include "IContentBrowserSingleton.h"
#include "SLegacyContentSource.h"
#include "ToolMenus.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SContentSourcesView)

#define LOCTEXT_NAMESPACE "SContentSourcesView"

namespace UE::Editor::ContentBrowser
{
	static bool bShowContentSourcesBar = false;
	static FAutoConsoleVariableRef CVarShowContentSourcesBar(
		TEXT("ContentBrowser.UI.ShowContentSourcesBar"),
		bShowContentSourcesBar,
		TEXT("Show the UI to swap between content sources (experimental)"));
	
	FDelayedAutoRegisterHelper SContentSourcesView::SourceBarMenuRegistration(
	EDelayedRegisterRunPhase::EndOfEngineInit,
	[]
	{
		static const FName SourceBarMenuName("ContentBrowser.SourceBar");
		if (!ensureAlways(!UToolMenus::Get()->IsMenuRegistered(SourceBarMenuName)))
		{
			return;
		}
		
		FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
		UToolMenu* ToolBar = UToolMenus::Get()->RegisterMenu(SourceBarMenuName, NAME_None, EMultiBoxType::VerticalToolBar);
		ToolBar->SetStyleSet(&UE::ContentBrowser::Private::FContentBrowserStyle::Get());
		ToolBar->StyleName = SourceBarMenuName; // Style name is the same as the menu entry name
		
		ToolBar->AddSection("Sources")
		.AddDynamicEntry(NAME_None,
			FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if (const UContentSourcesViewMenuContext* Context = InSection.FindContext<UContentSourcesViewMenuContext>())
				{
					if (const TSharedPtr<SContentSourcesView> ContentSourceWidgetPin = Context->ContentSourcesWidget.Pin())
					{
						// Legacy Content Source entry
						InSection.AddEntry(
							FToolMenuEntry::InitToolBarButton(
								FName("LegacyContentSource"),
								FUIAction(
									FExecuteAction::CreateLambda([Context]()
									{
										if (TSharedPtr<SContentSourcesView> ContextContentSourceWidget = Context->ContentSourcesWidget.Pin())
										{
											ContextContentSourceWidget->ActivateLegacyContentSource();
										}
									}),
									FCanExecuteAction(),
									FIsActionChecked::CreateLambda([Context]()
									{
										if (TSharedPtr<SContentSourcesView> ContextContentSourceWidget = Context->ContentSourcesWidget.Pin())
										{
											return ContextContentSourceWidget->IsLegacyContentSourceActive();
										}
										return false;
									}),
									FIsActionButtonVisible::CreateLambda([Context]()
									{
										if (TSharedPtr<SContentSourcesView> ContextContentSourceWidget = Context->ContentSourcesWidget.Pin())
										{
											return ContextContentSourceWidget->HasLegacyContentSource();
										}
										return false;
									})),
									LOCTEXT("LegacyContentSourceName", "Project"),
									FText(),
									FSlateIcon(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetStyleSetName(),
										"ContentBrowser.Sources.ProjectIcon"),
								EUserInterfaceActionType::ToggleButton
							));

						// Regular Content Sources
						auto MakeSourceEntry = [Context, &InSection](const TSharedRef<IContentSource>& ContentSource)
						{
							return InSection.AddEntry(
							FToolMenuEntry::InitToolBarButton(
								ContentSource->GetName(),
								FUIAction(
									FExecuteAction::CreateLambda([ContentSource, Context]()
									{
										if (TSharedPtr<SContentSourcesView> ContextContentSourceWidget = Context->ContentSourcesWidget.Pin())
										{
											ContextContentSourceWidget->ActivateContentSource(ContentSource);
										}
									}),
									FCanExecuteAction(),
									FIsActionChecked::CreateLambda([ContentSource, Context]()
									{
										if (TSharedPtr<SContentSourcesView> ContextContentSourceWidget = Context->ContentSourcesWidget.Pin())
										{
											return ContextContentSourceWidget->IsContentSourceActive(ContentSource);
										}
										return false;
									})),
									ContentSource->GetDisplayName(),
									FText(),
									ContentSource->GetIcon(),
								EUserInterfaceActionType::ToggleButton
							));
						};

						ContentSourceWidgetPin->ForEachContentSource(MakeSourceEntry);
					}
				}
			}));
	});
	
	void SContentSourcesView::Construct(const FArguments& InArgs)
	{
		LegacyContentSource = InArgs._LegacyContentSource;
		OnLegacyContentSourceEnabledEvent = InArgs._OnLegacyContentSourceEnabled;
		OnLegacyContentSourceDisabledEvent = InArgs._OnLegacyContentSourceDisabled;
		
		constexpr float ContentSourceBarWidth = 64.0f;

		UpdateContentSourcesList();
		IContentBrowserSingleton::Get().OnContentSourceFactoriesChanged().AddSP(this, &SContentSourcesView::OnContentSourcesChanged);

		TSharedRef<SWidget> LegacyContentSourceWidget = LegacyContentSource ? LegacyContentSource.ToSharedRef() : SNullWidget::NullWidget;
		
		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.0f)
			.AutoWidth()
			[
				SNew(SBorder)
				.Visibility_Lambda([]()
				{
					return bShowContentSourcesBar ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
				[
					SAssignNew(SourcesBarContainer, SBox)
					.WidthOverride(ContentSourceBarWidth)
					[
						CreateSourceBar()
					]
				]
			]
			+ SHorizontalBox::Slot()
			[
				SAssignNew(LegacyWidgetSwitcher, SWidgetSwitcher)
				+SWidgetSwitcher::Slot()
				[
					LegacyContentSourceWidget
				]
				+SWidgetSwitcher::Slot()
				[
					SAssignNew(ContentSourceWidget, SContentSource)
				]
			]
		];
		
		ChooseActiveContentSource();
	}

	void SContentSourcesView::ForEachContentSource(const TFunctionRef<void(const TSharedRef<IContentSource>&)>& InFunctor)
	{
		for (const TSharedRef<IContentSource>& ContentSource : ContentSources)
		{
			InFunctor(ContentSource);
		}
	}

	void SContentSourcesView::ActivateContentSource(const TSharedRef<IContentSource>& InContentSource)
	{
		// If this content source is already active, do nothing
		if (ActiveContentSource == InContentSource)
		{
			return;
		}
		
		ActiveContentSource = InContentSource;

		// If the legacy content source was previously active, disable it and notify
		if (bIsLegacyContentSourceActive)
		{
			bIsLegacyContentSourceActive = false;
			OnLegacyContentSourceDisabledEvent.ExecuteIfBound();
		}
		
		LegacyWidgetSwitcher->SetActiveWidgetIndex(1);
		ContentSourceWidget->SetContentSource(ActiveContentSource);
	}

	bool SContentSourcesView::IsContentSourceActive(const TSharedRef<IContentSource>& InContentSource) const
	{
		return ActiveContentSource == InContentSource;
	}

	bool SContentSourcesView::HasLegacyContentSource() const
	{
		return LegacyContentSource.IsValid();
	}

	bool SContentSourcesView::IsLegacyContentSourceActive() const
	{
		return HasLegacyContentSource() && bIsLegacyContentSourceActive;
	}

	void SContentSourcesView::ActivateLegacyContentSource()
	{
		// If the legacy content source doesn't exist, or it is already active - do nothing
		if (HasLegacyContentSource() && !IsLegacyContentSourceActive())
		{
			bIsLegacyContentSourceActive = true;
			OnLegacyContentSourceEnabledEvent.ExecuteIfBound();
			
			LegacyWidgetSwitcher->SetActiveWidgetIndex(0);
			ActiveContentSource = nullptr;

			// Set the currently active content source widget as nullptr to empty the widget contents 
			ContentSourceWidget->SetContentSource(ActiveContentSource);
		}
	}

	void SContentSourcesView::ChooseActiveContentSource()
	{
		ActiveContentSource = nullptr;

		// If the legacy content source is available, it gets priority
		if (HasLegacyContentSource())
		{
			ActivateLegacyContentSource();
		}
		else if (ContentSources.Num())
		{
			ActivateContentSource(ContentSources[0]);
		}
		// If there is no legacy content source and no known content sources - just set our contents to SNullWidget for now
		else
		{
			ChildSlot
			[
				SNullWidget::NullWidget
			];
		}
	}

	TSharedRef<SWidget> SContentSourcesView::CreateSourceBar()
	{
		FToolMenuContext MenuContext;
		UContentSourcesViewMenuContext* CommonContextObject = NewObject<UContentSourcesViewMenuContext>();
		CommonContextObject->ContentSourcesWidget = SharedThis(this);
		MenuContext.AddObject(CommonContextObject);
		return UToolMenus::Get()->GenerateWidget("ContentBrowser.SourceBar", MenuContext);
	}

	void SContentSourcesView::OnContentSourcesChanged()
	{
		UpdateContentSourcesList();
		SourcesBarContainer->SetContent(CreateSourceBar());

		if (ActiveContentSource)
		{
			// If the active content source does not exist in the list of known content sources anymore, choose a new one
			if (ContentSources.FindByPredicate([this](const TSharedPtr<IContentSource>& InContentSource)
			{
				return InContentSource->GetName() == ActiveContentSource->GetName();
			}) == nullptr)
			{
				ChooseActiveContentSource();
			}
		}
	}

	void SContentSourcesView::UpdateContentSourcesList()
	{
		ContentSources.Empty();
		IContentBrowserSingleton::Get().ForEachContentSourceFactory([this]
			(const FName& Name, const IContentBrowserSingleton::FContentSourceFactory& Factory)
		{
			ContentSources.Add(Factory.Execute());
		});
	}
}

#undef LOCTEXT_NAMESPACE
