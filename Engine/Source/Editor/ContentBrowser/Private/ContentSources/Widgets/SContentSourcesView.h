// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SContentSource.h"
#include "ContentSources/IContentSource.h"
#include "Widgets/SCompoundWidget.h"

#include "SContentSourcesView.generated.h"

namespace UE::Editor::ContentBrowser
{
	class SLegacyContentSource;
	class SContentSourcesView;
}

class SBox;
class SWidgetSwitcher;

UCLASS()
class UContentSourcesViewMenuContext : public UObject
{
public:
	GENERATED_BODY()
		
	TWeakPtr<UE::Editor::ContentBrowser::SContentSourcesView> ContentSourcesWidget;
};


namespace UE::Editor::ContentBrowser
{
	DECLARE_DELEGATE(FOnLegacyContentSourceEnabled)
	DECLARE_DELEGATE(FOnLegacyContentSourceDisabled)

	// Widget that displays a vertical toolbar to swap between known content sources and a child widget that contains the contents of the
	// currently active content source
	class SContentSourcesView : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SContentSourcesView) {}

		// A legacy content source can be used to displays the widgets of the old asset based content browser
		SLATE_ARGUMENT(TSharedPtr<SLegacyContentSource>, LegacyContentSource)
		SLATE_EVENT(FOnLegacyContentSourceEnabled, OnLegacyContentSourceEnabled)
		SLATE_EVENT(FOnLegacyContentSourceDisabled, OnLegacyContentSourceDisabled)
			
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

		void ForEachContentSource(const TFunctionRef<void(const TSharedRef<IContentSource>&)>& InFunctor);
		void ActivateContentSource(const TSharedRef<IContentSource>& InContentSource);
		bool IsContentSourceActive(const TSharedRef<IContentSource>& InContentSource) const;

		// Whether this widget currently supports showing the legacy content source
		bool HasLegacyContentSource() const;

		// Whether the legacy content source is supported and currently active
		bool IsLegacyContentSourceActive() const;
		void ActivateLegacyContentSource();

		// Pick the first valid content source that is found
		void ChooseActiveContentSource();

	private:
		TSharedRef<SWidget> CreateSourceBar();
		void OnContentSourcesChanged();
		void UpdateContentSourcesList();
		
	private:

		// List of currently known content sources
		TArray<TSharedRef<IContentSource>> ContentSources;

		// Currently active content source
		TSharedPtr<IContentSource> ActiveContentSource;

		// Widget that displays the contents of the currently active content source
		TSharedPtr<SContentSource> ContentSourceWidget;

		// Container widget for the vertical toolbar to display all known content source
		TSharedPtr<SBox> SourcesBarContainer;

		// The legacy content source
		TSharedPtr<SLegacyContentSource> LegacyContentSource;
		
		bool bIsLegacyContentSourceActive = false;
		FOnLegacyContentSourceEnabled OnLegacyContentSourceEnabledEvent;
		FOnLegacyContentSourceDisabled OnLegacyContentSourceDisabledEvent;

		// Widget switcher to display either the legacy content source or the currently active content source
		TSharedPtr<SWidgetSwitcher> LegacyWidgetSwitcher;

		/** Registers the content source bar */
		static FDelayedAutoRegisterHelper SourceBarMenuRegistration;
	};
}