// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLegacyContentSource.h"

namespace UE::Editor::ContentBrowser
{
	void SLegacyContentSource::Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNullWidget::NullWidget
		];
	}

	void SLegacyContentSource::SetContent(const TSharedRef<SWidget>& InWidget)
	{
		ChildSlot
		[
			InWidget
		];
	}
}
