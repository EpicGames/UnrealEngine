// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IContentBrowserSingleton.h"
#include "Widgets/SCompoundWidget.h"

class SAssetView;

namespace UE::Editor::ContentBrowser
{
	// Widget to display the old asset based content browser
	// Currently does not contain any logic and simply displays a pre-existing widget that is supplied by SContentBrowser
	class SLegacyContentSource : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SLegacyContentSource) {}
		
		SLATE_END_ARGS()
		
		void Construct(const FArguments& InArgs);
		void SetContent(const TSharedRef<SWidget>& InWidget);
	private:
	};

}
