// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragDropHandler.h"

#include "AssetViewUtils.h"
#include "CollectionViewUtils.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "ContentBrowserDataDragDropOp.h"
#include "ContentBrowserDataMenuContexts.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserItemData.h"
#include "ContentBrowserUtils.h"
#include "Delegates/Delegate.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/DragAndDrop.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/SlateRect.h"
#include "Layout/WidgetPath.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/Invoke.h"
#include "Templates/Tuple.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

namespace DragDropHandlerUtilsPrivate
{
	void RetrieveFolderInformation(const FContentBrowserItem& InFolder, FName& OutFolderBrushName, FName& OutFolderShadowName, FLinearColor& OutFolderOverrideColor)
	{
		// We are just dragging folders so add information on the FolderBrush to use and the color
		if (InFolder.IsValid() && InFolder.IsFolder())
		{
			ContentBrowserUtils::TryGetFolderBrushAndShadowName(InFolder, OutFolderBrushName, OutFolderShadowName);

			const bool bCollectionFolder = EnumHasAnyFlags(InFolder.GetItemCategory(), EContentBrowserItemFlags::Category_Collection);
			if (bCollectionFolder)
			{
				TSharedPtr<ICollectionContainer> CollectionContainer;
				FName CollectionName;
				ECollectionShareType::Type CollectionFolderShareType = ECollectionShareType::CST_All;
				ContentBrowserUtils::IsCollectionPath(InFolder.GetVirtualPath().ToString(), &CollectionContainer, &CollectionName, &CollectionFolderShareType);
			
				if (TOptional<FLinearColor> Color = CollectionViewUtils::GetCustomColor(CollectionContainer.Get(), CollectionName, CollectionFolderShareType))
				{
					OutFolderOverrideColor = Color.GetValue();
				}
			}
			else
			{
				if (TOptional<FLinearColor> Color = ContentBrowserUtils::GetPathColor(InFolder.GetInvariantPath().ToString()))
				{
					OutFolderOverrideColor = Color.GetValue();
				}
			}
		}
	}
}

namespace DragDropHandler
{

TSharedPtr<FDragDropOperation> CreateDragOperation(TArrayView<const FContentBrowserItem> InItems)
{
	if (InItems.Num() == 0)
	{
		return nullptr;
	}

	// Batch these by their data sources
	TMap<UContentBrowserDataSource*, TArray<FContentBrowserItemData>> SourcesAndItems;
	for (const FContentBrowserItem& Item : InItems)
	{
		FContentBrowserItem::FItemDataArrayView ItemDataArray = Item.GetInternalItems();
		for (const FContentBrowserItemData& ItemData : ItemDataArray)
		{
			if (UContentBrowserDataSource* ItemDataSource = ItemData.GetOwnerDataSource())
			{
				TArray<FContentBrowserItemData>& ItemsForSource = SourcesAndItems.FindOrAdd(ItemDataSource);
				ItemsForSource.Add(ItemData);
			}
		}
	}

	// Custom handling via a data source?
	for (const auto& SourceAndItemsPair : SourcesAndItems)
	{
		if (TSharedPtr<FDragDropOperation> CustomDragOp = SourceAndItemsPair.Key->CreateCustomDragOperation(SourceAndItemsPair.Value))
		{
			return CustomDragOp;
		}
	}

	// Generic handling
	FName FolderBrushName = NAME_None;
	FName FolderShadowBrushName = NAME_None;
	FLinearColor FolderColor = ContentBrowserUtils::GetDefaultColor();
	// This information will be later used by the ContentBrowserDragAnDropOp in case you are dragging only folders
	if (InItems.Num() > 0)
	{
		DragDropHandlerUtilsPrivate::RetrieveFolderInformation(InItems[0], FolderBrushName, FolderShadowBrushName, FolderColor);
	}

	FThumbnailOverrideParams FolderThumbnailOverrideParams;
	FolderThumbnailOverrideParams.FolderBrushName = FolderBrushName;
	FolderThumbnailOverrideParams.FolderShadowBrushName = FolderShadowBrushName;
	FolderThumbnailOverrideParams.FolderColorOverride = FolderColor;

	TSharedRef<FContentBrowserDataDragDropOp> ContentBrowserDragAndDrop = FContentBrowserDataDragDropOp::New(InItems, FolderThumbnailOverrideParams);
	return ContentBrowserDragAndDrop;
}

template <typename FuncType>
bool HandleDragEventOverride(const FContentBrowserItem& InItem, const FDragDropEvent& InDragDropEvent, FuncType Func)
{
	FContentBrowserItem::FItemDataArrayView ItemDataArray = InItem.GetInternalItems();
	for (const FContentBrowserItemData& ItemData : ItemDataArray)
	{
		if (UContentBrowserDataSource* ItemDataSource = ItemData.GetOwnerDataSource())
		{
			if (Invoke(Func, ItemDataSource, ItemData, InDragDropEvent))
			{
				return true;
			}
		}
	}

	return false;
}

bool ValidateGenericDragEvent(const FContentBrowserItem& InItem, const FDragDropEvent& InDragDropEvent)
{
	if (!InItem.IsFolder())
	{
		return false;
	}

	if (TSharedPtr<FContentBrowserDataDragDropOp> ContentDragDropOp = InDragDropEvent.GetOperationAs<FContentBrowserDataDragDropOp>())
	{
		if (EnumHasAnyFlags(InItem.GetItemCategory(), EContentBrowserItemFlags::Category_Collection))
		{
			ContentDragDropOp->SetToolTip(LOCTEXT("OnDragFoldersOverFolder_CannotDropOnCollectionFolder", "Cannot drop onto a collection folder. Drop onto the collection in the collection view instead."), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
		}
		else if (ContentDragDropOp->GetDraggedItems().Num() == 1 && ContentDragDropOp->GetDraggedItems()[0].GetVirtualPath() == InItem.GetVirtualPath())
		{
			ContentDragDropOp->SetToolTip(LOCTEXT("OnDragFoldersOverFolder_CannotSelfDrop", "Cannot move or copy a folder onto itself"), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
		}
		else
		{
			int32 NumDraggedItems = ContentDragDropOp->GetDraggedItems().Num();
			int32 NumCanMoveOrCopy = 0;
			bool bCanCopyOnce = false;
			bool bCanMoveOnce = false;
			for (const FContentBrowserItem& DraggedItem : ContentDragDropOp->GetDraggedItems())
			{
				const bool bCanMove = DraggedItem.CanMove(InItem.GetVirtualPath());
				const bool bCanCopy = DraggedItem.CanCopy(InItem.GetVirtualPath());
				bCanCopyOnce |= bCanCopy;
				bCanMoveOnce |= bCanMove;
				if (bCanMove || bCanCopy)
				{
					++NumCanMoveOrCopy;
				}
			}

			if (NumCanMoveOrCopy == 0)
			{
				ContentDragDropOp->SetToolTip(LOCTEXT("OnDragFoldersOverFolder_CannotDrop", "Cannot move or copy to this folder"), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
			}
			else
			{
				FText MoveOrCopyString = FText::GetEmpty();
				if (bCanCopyOnce && bCanMoveOnce)
				{
					MoveOrCopyString = LOCTEXT("DragAndDropMoveOrCopy", "Move or copy");
				}
				else if (bCanCopyOnce)
				{
					MoveOrCopyString = LOCTEXT("DragAndDropCopy", "Copy");
				}
				else
				{
					MoveOrCopyString = LOCTEXT("DragAndDropMove", "Move");
				}

				const FText FirstItemText = ContentDragDropOp->GetDraggedItems()[0].GetDisplayName();
				const FText MoveOrCopyText = (NumCanMoveOrCopy > 1)
					? FText::Format(LOCTEXT("OnDragAssetsOverFolder_MultipleItems", "{0} '{1}' and {2} {2}|plural(one=other,other=others)"), MoveOrCopyString, FirstItemText, NumDraggedItems - 1)
					: FText::Format(LOCTEXT("OnDragAssetsOverFolder_SingularItems", "{0} '{1}'"), MoveOrCopyString, FirstItemText);

				if (NumCanMoveOrCopy < NumDraggedItems)
				{
					ContentDragDropOp->SetToolTip(FText::Format(LOCTEXT("OnDragAssetsOverFolder_SomeInvalidItems", "{0}\n\n{1} {1}|plural(one=item,other=items) will be ignored as they cannot be moved or copied"), MoveOrCopyText, NumDraggedItems - NumCanMoveOrCopy), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OKWarn")));
				}
				else
				{
					ContentDragDropOp->SetToolTip(MoveOrCopyText, FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")));
				}
			}
		}

		return true;
	}

	return false;
}

bool HandleDragEnterItem(const FContentBrowserItem& InItem, const FDragDropEvent& InDragDropEvent)
{
	// Custom handling via a data source?
	if (HandleDragEventOverride(InItem, InDragDropEvent, &UContentBrowserDataSource::HandleDragEnterItem))
	{
		return true;
	}

	// Generic handling
	return ValidateGenericDragEvent(InItem, InDragDropEvent);
}

bool HandleDragOverItem(const FContentBrowserItem& InItem, const FDragDropEvent& InDragDropEvent)
{
	// Custom handling via a data source?
	if (HandleDragEventOverride(InItem, InDragDropEvent, &UContentBrowserDataSource::HandleDragOverItem))
	{
		return true;
	}

	// Generic handling
	return ValidateGenericDragEvent(InItem, InDragDropEvent);
}

bool HandleDragLeaveItem(const FContentBrowserItem& InItem, const FDragDropEvent& InDragDropEvent)
{
	// Custom handling via a data source?
	if (HandleDragEventOverride(InItem, InDragDropEvent, &UContentBrowserDataSource::HandleDragLeaveItem))
	{
		return true;
	}

	if (!InItem.IsFolder())
	{
		return false;
	}

	// Generic handling
	if (TSharedPtr<FContentBrowserDataDragDropOp> ContentDragDropOp = InDragDropEvent.GetOperationAs<FContentBrowserDataDragDropOp>())
	{
		ContentDragDropOp->ResetToDefaultToolTip();
		return true;
	}

	return false;
}

template <typename CanMoveOrCopyFuncType, typename BulkMoveOrCopyFuncType>
void HandleDragDropMoveOrCopy(const FContentBrowserItem& InDropTargetItem, const TArray<FContentBrowserItem>& InDraggedItems, const TSharedPtr<SWidget>& InParentWidget, const FText InMoveOrCopyMsg, CanMoveOrCopyFuncType InCanMoveOrCopyFunc, BulkMoveOrCopyFuncType InBulkMoveOrCopyFunc)
{
	const FName InDropTargetPath = InDropTargetItem.GetVirtualPath();

	// Batch these by their data sources
	TMap<UContentBrowserDataSource*, TArray<FContentBrowserItemData>> SourcesAndItems;
	for (const FContentBrowserItem& DraggedItem : InDraggedItems)
	{
		FContentBrowserItem::FItemDataArrayView ItemDataArray = DraggedItem.GetInternalItems();
		for (const FContentBrowserItemData& ItemData : ItemDataArray)
		{
			if (UContentBrowserDataSource* ItemDataSource = ItemData.GetOwnerDataSource())
			{
				FText MoveOrCopyErrorMsg;
				if (Invoke(InCanMoveOrCopyFunc, *ItemDataSource, ItemData, InDropTargetPath, &MoveOrCopyErrorMsg))
				{
					TArray<FContentBrowserItemData>& ItemsForSource = SourcesAndItems.FindOrAdd(ItemDataSource);
					ItemsForSource.Add(ItemData);
				}
				else
				{
					AssetViewUtils::ShowErrorNotifcation(MoveOrCopyErrorMsg);
				}
			}
		}
	}

	// Execute the operation now
	int32 NumMovedOrCopiedItems = 0;
	for (const auto& SourceAndItemsPair : SourcesAndItems)
	{
		if (Invoke(InBulkMoveOrCopyFunc, *SourceAndItemsPair.Key, SourceAndItemsPair.Value, InDropTargetPath))
		{
			// This assumes that everything passed is moved or copied, which may not be true, but we've validated as best we can when building this array
			NumMovedOrCopiedItems += SourceAndItemsPair.Value.Num();
		}
	}

	// Show a message if the move or copy was successful
	if (InParentWidget)
	{
		const bool bWasSuccessful = NumMovedOrCopiedItems > 0;
		const ContentBrowserUtils::EDisplayMessageType MessageType = bWasSuccessful ? ContentBrowserUtils::EDisplayMessageType::Successful : ContentBrowserUtils::EDisplayMessageType::Info;
		const FText Message = FText::Format(InMoveOrCopyMsg, NumMovedOrCopiedItems, InDropTargetItem.GetDisplayName());
		const FVector2f& CursorPos = FSlateApplication::Get().GetCursorPos();
		FSlateRect MessageAnchor(CursorPos.X, CursorPos.Y, CursorPos.X, CursorPos.Y);
		ContentBrowserUtils::DisplayMessage(Message, MessageAnchor, InParentWidget.ToSharedRef(), MessageType);
	}
}

void HandleDragDropMove(const FContentBrowserItem& InDropTargetItem, const TArray<FContentBrowserItem>& InDraggedItems, const TSharedPtr<SWidget>& InParentWidget)
{
	return HandleDragDropMoveOrCopy(InDropTargetItem, InDraggedItems, InParentWidget, LOCTEXT("ItemsDroppedMove", "{0} {0}|plural(one=item,other=items) moved to '.../{1}'"), &UContentBrowserDataSource::CanMoveItem, &UContentBrowserDataSource::BulkMoveItems);
}

void HandleDragDropCopy(const FContentBrowserItem& InDropTargetItem, const TArray<FContentBrowserItem>& InDraggedItems, const TSharedPtr<SWidget>& InParentWidget)
{
	return HandleDragDropMoveOrCopy(InDropTargetItem, InDraggedItems, InParentWidget, LOCTEXT("ItemsDroppedCopy", "{0} {0}|plural(one=item,other=items) copied to '.../{1}'"), &UContentBrowserDataSource::CanCopyItem, &UContentBrowserDataSource::BulkCopyItems);
}

bool HandleDragDropOnItem(const FContentBrowserItem& InItem, const FDragDropEvent& InDragDropEvent, const TSharedRef<SWidget>& InParentWidget)
{
	// Custom handling via a data source?
	if (HandleDragEventOverride(InItem, InDragDropEvent, &UContentBrowserDataSource::HandleDragDropOnItem))
	{
		return true;
	}

	if (!InItem.IsFolder())
	{
		return false;
	}

	// Generic handling
	if (TSharedPtr<FContentBrowserDataDragDropOp> ContentDragDropOp = InDragDropEvent.GetOperationAs<FContentBrowserDataDragDropOp>())
	{
		static const FName MenuName = "ContentBrowser.DragDropContextMenu";

		UToolMenus* ToolMenus = UToolMenus::Get();
		if (!ToolMenus->IsMenuRegistered(MenuName))
		{
			UToolMenu* Menu = ToolMenus->RegisterMenu(MenuName);
			FToolMenuSection& Section = Menu->AddSection("MoveCopy", LOCTEXT("MoveCopyMenuHeading_Generic", "Move/Copy..."));

			Section.AddDynamicEntry("DragDropMoveCopy_Dynamic", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				const UContentBrowserDataMenuContext_DragDropMenu* ContextObject = InSection.FindContext<UContentBrowserDataMenuContext_DragDropMenu>();
				checkf(ContextObject, TEXT("Required context UContentBrowserDataMenuContext_DragDropMenu was missing!"));

				if (ContextObject->bCanMove)
				{
					InSection.AddMenuEntry(
						"DragDropMove",
						LOCTEXT("DragDropMove", "Move Here"),
						LOCTEXT("DragDropMoveTooltip", "Move the dragged items to this folder, preserving the structure of any copied folders."),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([DropTargetItem = ContextObject->DropTargetItem, DraggedItems = ContextObject->DraggedItems, ParentWidget = ContextObject->ParentWidget]() { HandleDragDropMove(DropTargetItem, DraggedItems, ParentWidget.Pin()); }))
						);
				}

				if (ContextObject->bCanCopy)
				{
					InSection.AddMenuEntry(
						"DragDropCopy",
						LOCTEXT("DragDropCopy", "Copy Here"),
						LOCTEXT("DragDropCopyTooltip", "Copy the dragged items to this folder, preserving the structure of any copied folders."),
						FSlateIcon(),
						FUIAction(FExecuteAction::CreateLambda([DropTargetItem = ContextObject->DropTargetItem, DraggedItems = ContextObject->DraggedItems, ParentWidget = ContextObject->ParentWidget]() { HandleDragDropCopy(DropTargetItem, DraggedItems, ParentWidget.Pin()); }))
						);
				}
			}));
		}

		if (UToolMenu* Menu = ToolMenus->ExtendMenu(MenuName))
		{
			// Update the section display name for the current drop target
			Menu->AddSection("MoveCopy", FText::Format(LOCTEXT("MoveCopyMenuHeading_Fmt", "Move/Copy to {0}"), InItem.GetDisplayName()));
		}

		UContentBrowserDataMenuContext_DragDropMenu* ContextObject = NewObject<UContentBrowserDataMenuContext_DragDropMenu>();
		ContextObject->DropTargetItem = InItem;
		ContextObject->DraggedItems = ContentDragDropOp->GetDraggedItems();
		ContextObject->bCanMove = false;
		ContextObject->bCanCopy = false;
		for (const FContentBrowserItem& DraggedItem : ContextObject->DraggedItems)
		{
			ContextObject->bCanMove |= DraggedItem.CanMove(ContextObject->DropTargetItem.GetVirtualPath());
			ContextObject->bCanCopy |= DraggedItem.CanCopy(ContextObject->DropTargetItem.GetVirtualPath());

			if (ContextObject->bCanMove && ContextObject->bCanCopy)
			{
				break;
			}
		}
		ContextObject->ParentWidget = InParentWidget;

		FToolMenuContext MenuContext(ContextObject);
		TSharedRef<SWidget> MenuWidget = ToolMenus->GenerateWidget(MenuName, MenuContext);

		FSlateApplication::Get().PushMenu(
			InParentWidget,
			FWidgetPath(),
			MenuWidget,
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
			);

		return true;
	}

	return false;
}

} // namespace DragDropHandler

#undef LOCTEXT_NAMESPACE
