// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Templates/SharedPointer.h"

class FDragDropEvent;
class FDragDropOperation;
class SWidget;
struct FContentBrowserItem;

/** Common Content Browser drag-drop handler logic */
namespace DragDropHandler
{
	/**
	 * Called to provide drag and drop handling when starting a drag event.
	 * @return A drag operation, or null if no drag can be performed.
	 */
	TSharedPtr<FDragDropOperation> CreateDragOperation(TArrayView<const FContentBrowserItem> InItems);

	/**
	 * Called to provide drag and drop handling when a drag event enters an item, such as performing validation and reporting error information.
	 * @return True if the drag event can be handled (even if it won't be because it's invalid), or false to allow something else to deal with it instead.
	 */
	bool HandleDragEnterItem(const FContentBrowserItem& InItem, const FDragDropEvent& InDragDropEvent);

	/**
	 * Called to provide drag and drop handling while a drag event is over an item, such as performing validation and reporting error information.
	 * @return True if the drag event can be handled (even if it won't be because it's invalid), or false to allow something else to deal with it instead.
	 */
	bool HandleDragOverItem(const FContentBrowserItem& InItem, const FDragDropEvent& InDragDropEvent);

	/**
	 * Called to provide drag and drop handling when a drag event leaves an item, such as clearing any error information set during earlier validation.
	 * @return True if the drag event can be handled (even if it won't be because it's invalid), or false to allow something else to deal with it instead.
	 */
	bool HandleDragLeaveItem(const FContentBrowserItem& InItem, const FDragDropEvent& InDragDropEvent);

	/**
	 * Called to provide drag and drop handling when a drag event is dropped on an item.
	 * @return True if the drag event can be handled (even if it wasn't because it was invalid), or false to allow something else to deal with it instead.
	 */
	bool HandleDragDropOnItem(const FContentBrowserItem& InItem, const FDragDropEvent& InDragDropEvent, const TSharedRef<SWidget>& InParentWidget);
}
