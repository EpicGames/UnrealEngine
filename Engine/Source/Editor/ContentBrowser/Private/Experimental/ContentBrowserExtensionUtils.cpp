// Copyright Epic Games, Inc. All Rights Reserved.

#include "Experimental/ContentBrowserExtensionUtils.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserUtils.h"
#include "CollectionViewUtils.h"
#include "ContentBrowserItemPath.h"
#include "IContentBrowserDataModule.h"

namespace UE::Editor::ContentBrowser::ExtensionUtils
{
	TOptional<FLinearColor> CheckAndGetCollectionColor(const FString& InVirtualPath)
	{
		TSharedPtr<ICollectionContainer> CollectionContainer;
		FName CollectionName;
		ECollectionShareType::Type CollectionFolderShareType = ECollectionShareType::CST_All;

		if (ContentBrowserUtils::IsCollectionPath(InVirtualPath, &CollectionContainer , &CollectionName, &CollectionFolderShareType))
		{
			return CollectionViewUtils::GetCustomColor(CollectionContainer.Get(), CollectionName, CollectionFolderShareType);
		}
		
		return TOptional<FLinearColor>();
	} 

	TOptional<FLinearColor> GetFolderColor(const FContentBrowserItem& FolderItem)
	{
		if (TOptional<FLinearColor> CollectionColor = CheckAndGetCollectionColor(FolderItem.GetVirtualPath().ToString()))
		{
			return CollectionColor;
		}
		
		return ContentBrowserUtils::GetPathColor(FolderItem.GetInvariantPath().ToString());
	}

	TOptional<FLinearColor> GetFolderColor(const FContentBrowserItemPath& FolderPath)
	{
		if (TOptional<FLinearColor> CollectionColor = CheckAndGetCollectionColor(FolderPath.GetVirtualPathString()))
		{
			return CollectionColor;
		}

		if (FolderPath.HasInternalPath())
		{
			return ContentBrowserUtils::GetPathColor(FolderPath.GetInternalPathString());
		}

		return TOptional<FLinearColor>();
	}
	
	TOptional<FLinearColor> GetFolderColor(const FName& FolderPath)
	{
		const FContentBrowserItemPath ItemPath(FolderPath, EContentBrowserPathType::Internal);
		return GetFolderColor(ItemPath);
	}

	void SetFolderColor(const FName& FolderPath, const FLinearColor& FolderColor)
	{
		ContentBrowserUtils::SetPathColor(FolderPath.ToString(), FolderColor);
	}

	bool IsFolderFavorite(const FContentBrowserItemPath& ItemPath)
	{
		return ContentBrowserUtils::IsFavoriteFolder(ItemPath);
	}

	void SetFolderFavorite(const FString& FolderPath, bool bIsFavorite)
	{
		if (bIsFavorite)
		{
			ContentBrowserUtils::AddFavoriteFolder(FContentBrowserItemPath(FolderPath, EContentBrowserPathType::Internal));
		}
		else
		{
			ContentBrowserUtils::RemoveFavoriteFolder(FContentBrowserItemPath(FolderPath, EContentBrowserPathType::Internal));
		}
		
	}
}
