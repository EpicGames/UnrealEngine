// Copyright Epic Games, Inc. All Rights Reserved.

#include "Experimental/ContentBrowserSearchOverride.h"
#include "Experimental/ContentBrowserSearchOverrideInternal.h"

namespace UE::Editor::ContentBrowser::Extension
{

// ---------------------------------------------------------------------------
// ICBSearchOverrideInstance
// ---------------------------------------------------------------------------

ICBSearchOverrideInstance::ICBSearchOverrideInstance() = default;
ICBSearchOverrideInstance::~ICBSearchOverrideInstance() = default;

bool ICBSearchOverrideInstance::IsOverrideActive() const
{
	return bIsOverrideActive;
}

void ICBSearchOverrideInstance::ToggleOverride()
{
	bIsOverrideActive = !bIsOverrideActive;
	GetInternalApi().OnOverrideActiveChanged.Broadcast();
}

void ICBSearchOverrideInstance::SetUserSearching(bool bIsSearching)
{
	GetInternalApi().OnRequestSetUserSearching.ExecuteIfBound(bIsSearching);
}

void ICBSearchOverrideInstance::RequestQuickFrontendListRefresh()
{
	GetInternalApi().OnRequestQuickFrontendListRefresh.ExecuteIfBound();
}

void ICBSearchOverrideInstance::PublishResults(TArrayView<TSharedPtr<FAssetViewItem>> Items)
{
	GetInternalApi().PublishedResults.Append(Items);
}

FInternalApiWrapper& ICBSearchOverrideInstance::GetInternalApi()
{
	if (!InternalApi)
	{
		InternalApi = MakeUnique<FInternalApiWrapper>(*this);
	}
	return *InternalApi;
}

void FInternalApiWrapper::OnFilteringReset()
{
	PublishedResults.Reset();
	bAllKnownItemsAvailableNotified = false;
	Instance.OnFilteringResetImplementation();
}

void FInternalApiWrapper::OnAllKnownItemsAvailable()
{
	if (!bAllKnownItemsAvailableNotified)
	{
		bAllKnownItemsAvailableNotified = true;
		Instance.OnAllKnownItemsAvailableImplementation();
	}
}

} // namespace UE::Editor::ContentBrowser::Extension
