// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Experimental/ContentBrowserSearchOverride.h"

namespace UE::Editor::ContentBrowser::Extension
{
	/**
	 * Provides write access to the private state of ICBSearchOverrideInstance.
	 * Only ContentBrowser internals should hold a reference to this wrapper.
	 */
	class FInternalApiWrapper
	{
	public:
		explicit FInternalApiWrapper(ICBSearchOverrideInstance& InInstance)
			: Instance(InInstance)
		{}

		void OnFilteringReset();
		void OnAllKnownItemsAvailable();

		/** Fires when the override is toggled on or off. */
		FSimpleMulticastDelegate OnOverrideActiveChanged;

		/** Executed when the override calls SetUserSearching(). */
		TDelegate<void(bool)> OnRequestSetUserSearching;

		/** Executed when the override calls RequestQuickFrontendListRefresh(). */
		FSimpleDelegate OnRequestQuickFrontendListRefresh;

		/** Results set by PublishResults(); consumed by the asset view on the next tick. */
		TArray<TSharedPtr<FAssetViewItem>> PublishedResults;

		bool bAllKnownItemsAvailableNotified = false;

	private:
		ICBSearchOverrideInstance& Instance;
	};

} // namespace UE::Editor::ContentBrowser::Extension
