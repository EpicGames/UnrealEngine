// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters.h"

#include "Algo/AnyOf.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserLog.h"
#include "FrontendFilterBase.h"
#include "IContentBrowserDataModule.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

/** 
 * FFilter_ShowRedirectors 
 */

FFilter_ShowRedirectors::FFilter_ShowRedirectors(TSharedPtr<FFrontendFilterCategory> InCategory)
	: FFrontendFilter(InCategory)
{
}

/** Returns the human readable name for this filter */
FText FFilter_ShowRedirectors::GetDisplayName() const
{
	return LOCTEXT("FrontendFilter_ShowRedirectors", "Show Redirectors");
}

/** Returns the tooltip for this filter, shown in the filters menu */
FText FFilter_ShowRedirectors::GetToolTipText() const
{
	return LOCTEXT("FrontendFilter_ShowRedirectorsToolTip", "Allow display of Redirectors.");
}

/** Returns the name of the icon to use in menu entries */
FName FFilter_ShowRedirectors::GetIconName() const
{
	return NAME_None;
}

/** Notification that the filter became active or inactive */
void FFilter_ShowRedirectors::ActiveStateChanged(bool bActive)
{
	// Do nothing, filter state is queried externally e.g. by SContentBrowser
}

/** Can be overriden for custom FilterBar subclasses to save settings, currently not implemented in any gneeric Filter Bar */
void FFilter_ShowRedirectors::SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const 
{

}

/** Can be overriden for custom FilterBar subclasses to load settings, currently not implemented in any gneeric Filter Bar */
void FFilter_ShowRedirectors::LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) 
{

}

/** 
 * FFilter_OtherDevelopers 
 */

FFilter_HideOtherDevelopers::FFilter_HideOtherDevelopers(TSharedPtr<FFrontendFilterCategory> InCategory, FName InFilterBarIdentifier)
	: FFrontendFilter(InCategory)
	, FilterBarIdentifier(InFilterBarIdentifier)
	, PathPermissionList(MakeShared<FPathPermissionList>())
{
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	ItemDataUpdatedHandle = ContentBrowserData->OnItemDataUpdated().AddRaw(this, &FFilter_HideOtherDevelopers::HandleItemDataUpdated);
	ItemDataRefreshedHandle = ContentBrowserData->OnItemDataRefreshed().AddRaw(this, &FFilter_HideOtherDevelopers::HandleItemDataRefreshed);

	BuildFilter();
}

FFilter_HideOtherDevelopers::~FFilter_HideOtherDevelopers()
{
	if (IContentBrowserDataModule* ContentBrowserModule = IContentBrowserDataModule::GetPtr())
	{
		if (UContentBrowserDataSubsystem* ContentBrowserData = ContentBrowserModule->GetSubsystem())
		{
			ContentBrowserData->OnItemDataUpdated().Remove(ItemDataUpdatedHandle);
			ContentBrowserData->OnItemDataRefreshed().Remove(ItemDataRefreshedHandle);
		}
	}
}

TSharedRef<const FPathPermissionList> FFilter_HideOtherDevelopers::GetPathPermissionList() 
{
	return PathPermissionList;
}

void FFilter_HideOtherDevelopers::BuildFilter()
{
	static const FName NAME_OtherDevelopers{"OtherDevelopers"};
	// Update list of other developer folders and put into permission list 
	FName BaseDevelopersPath{TEXTVIEW("/Game/Developers")};
	FName UserDeveloperFolder(TStringBuilder<256>{ InPlace, BaseDevelopersPath, TEXTVIEW("/"), FPaths::GameUserDeveloperFolderName() });

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	TArray<FName> ChildPaths;
	AssetRegistry.GetSubPaths(BaseDevelopersPath, ChildPaths, false);
	TSet<FName> PreviousPaths = MoveTemp(OtherDeveloperFolders);
	OtherDeveloperFolders.Reset();
	OtherDeveloperFolders.Append(TSet<FName>(ChildPaths));
	OtherDeveloperFolders.Remove(UserDeveloperFolder);
	if (OtherDeveloperFolders.Num() != PreviousPaths.Num() || OtherDeveloperFolders.Difference(PreviousPaths).Num() != 0
		|| PreviousPaths.Difference(OtherDeveloperFolders).Num() != 0)
	{
		UE_LOGF(LogContentBrowser, Verbose, "[%ls] FFilterHideOtherDevelopers rebuilt exclusion list: %ls", *FilterBarIdentifier.ToString(), 
			*FString::JoinBy(OtherDeveloperFolders, TEXT(","), UE_PROJECTION_MEMBER(FName, ToString)));
		
		// Recreate the permission list so that the content browser can do pointer comparison to tell that the list has changed rather than tbinding the delegate
		PathPermissionList = MakeShared<FPathPermissionList>();
		for (FName OtherPath : OtherDeveloperFolders)
		{
			PathPermissionList->AddDenyListItem(NAME_OtherDevelopers, WriteToString<256>(OtherPath).ToView());
		}
		BroadcastChangedEvent();
	}
	else
	{
		UE_LOGF(LogContentBrowser, Verbose, "[%ls] FFilterHideOtherDevelopers keeping previous exclusion list", *FilterBarIdentifier.ToString());
	}
}

void FFilter_HideOtherDevelopers::HandleItemDataRefreshed()
{
	BuildFilter();
}

void FFilter_HideOtherDevelopers::HandleItemDataUpdated(TArrayView<const FContentBrowserItemDataUpdate> InUpdatedItems)
{
	const bool bNeedsRebuild = Algo::AnyOf(InUpdatedItems, [](const FContentBrowserItemDataUpdate& Update){
		FName InternalPath = Update.GetItemData().GetInternalPath();
		if (!InternalPath.IsNone() && Update.GetItemData().IsFolder() && Update.GetUpdateType() == EContentBrowserItemUpdateType::Added)
		{
			if (WriteToString<256>(InternalPath).ToView().StartsWith(TEXTVIEW("/Game/Developers"), ESearchCase::IgnoreCase))
			{
				return true;
			}
		}
		return false;
	});	
	if (bNeedsRebuild)
	{
		BuildFilter();
	}
}

/** Returns the human readable name for this filter */
FText FFilter_HideOtherDevelopers::GetDisplayName() const
{
	return LOCTEXT("FrontendFilter_HideOtherDevelopers", "Hide Other Developers"); 
}

/** Returns the tooltip for this filter, shown in the filters menu */
FText FFilter_HideOtherDevelopers::GetToolTipText() const
{ 
	return LOCTEXT("FrontendFilter_HideOtherDevelopersTooltip", "Hide the display of assets in developer folders that aren't yours."); 
}

/** Returns the name of the icon to use in menu entries */
FName FFilter_HideOtherDevelopers::GetIconName() const
{
	return NAME_None;
}

/** Notification that the filter became active or inactive */
void FFilter_HideOtherDevelopers::ActiveStateChanged(bool bActive)
{
	if (bActive)
	{
		UE_LOGF(LogContentBrowser, Verbose, "[%ls] FFilterHideOtherDevelopers active, hiding content from other developers", *FilterBarIdentifier.ToString());
	}
	else
	{
		UE_LOGF(LogContentBrowser, Verbose, "[%ls] FFilterHideOtherDevelopers inactive, showing content from all developers", *FilterBarIdentifier.ToString());
	}
}

/** Can be overriden for custom FilterBar subclasses to save settings, currently not implemented in any gneeric Filter Bar */
void FFilter_HideOtherDevelopers::SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const
{
}

/** Can be overriden for custom FilterBar subclasses to load settings, currently not implemented in any gneeric Filter Bar */
void FFilter_HideOtherDevelopers::LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString)
{
}

#undef LOCTEXT_NAMESPACE