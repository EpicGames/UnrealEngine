// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Private filters not exposed to modules that depend on ContentBrowser

#include "IContentBrowserSingleton.h"
#include "Filters/FilterBase.h"
#include "Filters/GenericFilter.h"
#include "FrontendFilterBase.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Templates/SharedPointer.h"

struct FCollectionRef;
class ICollectionContainer;

/** 
 *  A custom filter that appears in the filter bar but actually controls a content browser setting controlling 
 *  visibility of redirectors for use in backend filtering.
 */
class FFilter_ShowRedirectors : public FFrontendFilter
{
public:
	// TODO: Consider not giving this the entire viewmodel to sync with filter state
	// could make viewmode own/create this, or send a smaller interface ptr into the constructor?
	FFilter_ShowRedirectors (TSharedPtr<FFrontendFilterCategory> InCategory);

	/** Returns the system name for this filter */
	virtual FString GetName() const override { return TEXT("ShowRedirectorsBackend"); } 

	/** Returns the human readable name for this filter */
	virtual FText GetDisplayName() const override;

	/** Returns the tooltip for this filter, shown in the filters menu */
	virtual FText GetToolTipText() const override;

	/** Returns the name of the icon to use in menu entries */
	virtual FName GetIconName() const override;

	/** If true, the filter will be active in the FilterBar when it is inactive in the UI (i.e the filter pill is grayed out)
	 */
	virtual bool IsInverseFilter() const 
	{
		// This has to be an inverse filter to prevent the asset view from recursively displaying all assets 
		return true; 
	}

	/** Notification that the filter became active or inactive */
	virtual void ActiveStateChanged(bool bActive) override;

	/** Can be overriden for custom FilterBar subclasses to save settings, currently not implemented in any gneeric Filter Bar */
	virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const override;

	/** Can be overriden for custom FilterBar subclasses to load settings, currently not implemented in any gneeric Filter Bar */
	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) override;

	/** Pass all objects - filter is just used to set backend query state. */
	virtual bool PassesFilter(FAssetFilterType InItem) const override { return true; }
};

/** 
 * Expression context which gathers up the names of any dynamic collections being referenced by the current query 
 * Private utility class for FAssetTextFilter and deprecated FFrontendFilter_Text
 */
class FFrontendFilter_GatherDynamicCollectionsExpressionContext : public ITextFilterExpressionContext
{
public:
	FFrontendFilter_GatherDynamicCollectionsExpressionContext(const TArray<TSharedPtr<ICollectionContainer>>& CollectionContainers, TArray<FCollectionRef>& OutReferencedDynamicCollections);
	~FFrontendFilter_GatherDynamicCollectionsExpressionContext();

	virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override;
	virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override;

private:
	bool TestAgainstAvailableCollections(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const;

	/** Contains a collection ref along with its recursion depth in the dynamic query - used so we can test them depth first */
	struct FDynamicCollectionRefAndDepth
	{
		FDynamicCollectionRefAndDepth(const FCollectionRef& InCollection, const int32 InRecursionDepth)
			: Collection(InCollection)
			, RecursionDepth(InRecursionDepth)
		{
		}

		FCollectionRef Collection;
		int32 RecursionDepth;
	};

	/** The currently available dynamic collections */
	TArray<FCollectionRef> AvailableDynamicCollections;

	/** This will be populated with any dynamic collections that are being referenced by the current query - these collections may not all match when tested against the actual asset data */
	TArray<FCollectionRef>& ReferencedDynamicCollections;

	/** Dynamic collections that have currently be found as part of the query (or recursive sub-query) */
	mutable TArray<FDynamicCollectionRefAndDepth> FoundDynamicCollections;

	/** Incremented when we test a sub-query, decremented once we're done */
	mutable int32 CurrentRecursionDepth;
};

// Non-frontend filter which modifies content browser backend query to exclude folders belonging to other developers 
class FFilter_HideOtherDevelopers : public FFrontendFilter
{
public: 
	FFilter_HideOtherDevelopers(TSharedPtr<FFrontendFilterCategory> InCategory, FName FilterBarIdentifier);
	FFilter_HideOtherDevelopers& operator=(const FFilter_HideOtherDevelopers&) = delete;
	virtual ~FFilter_HideOtherDevelopers();

	/** Get the list of folders to be denied when this filter is active (visually disabled, becuase it's an inverse filter) */
	TSharedRef<const FPathPermissionList> GetPathPermissionList();


	/** Pass all objects - filter is just used to set backend query state. */
	virtual bool PassesFilter(FAssetFilterType InItem) const override { return true; }

	/** Returns the system name for this filter */
	virtual FString GetName() const override { return TEXT("HideOtherDevelopersBackend"); }

	/** Returns the human readable name for this filter */
	virtual FText GetDisplayName() const override;

	/** Returns the tooltip for this filter, shown in the filters menu */
	virtual FText GetToolTipText() const override;

	/** Returns the name of the icon to use in menu entries */
	virtual FName GetIconName() const override;

	/** Notification that the filter became active or inactive */
	virtual void ActiveStateChanged(bool bActive) override;

	/** Called when the right-click context menu is being built for this filter */
	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override {}
	
	/** Can be overriden for custom FilterBar subclasses to save settings, currently not implemented in any gneeric Filter Bar */
	virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const override;

	/** Can be overriden for custom FilterBar subclasses to load settings, currently not implemented in any gneeric Filter Bar */
	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) override;

private:
	void BuildFilter();

	/** Handle new folders being added to rebuild the filter */
	void HandleItemDataUpdated(TArrayView<const FContentBrowserItemDataUpdate> InUpdatedItems);
	void HandleItemDataRefreshed();

	FName FilterBarIdentifier;
	TSet<FName> OtherDeveloperFolders;
	TSharedRef<FPathPermissionList> PathPermissionList;
	FDelegateHandle ItemDataUpdatedHandle;
	FDelegateHandle ItemDataRefreshedHandle;
};