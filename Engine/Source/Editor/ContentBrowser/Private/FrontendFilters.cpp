// Copyright Epic Games, Inc. All Rights Reserved.

#include "FrontendFilters.h"

#include "AssetCompilingManager.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetToolsModule.h"
#include "Blueprint/BlueprintSupport.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "ContentBrowserDataFilter.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserItemData.h"
#include "ContentBrowserModule.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Engine/World.h"
#include "Filters.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "IAssetTools.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "IContentBrowserDataModule.h"
#include "ISourceControlModule.h"
#include "ISourceControlState.h"
#include "MRUFavoritesList.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ObjectTools.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Settings/ContentBrowserSettings.h"
#include "SourceControlOperations.h"
#include "SourceControlHelpers.h"
#include "Templates/RemoveReference.h"
#include "Templates/UnrealTemplate.h"
#include "TextFilterKeyValueHandlers.h"
#include "TextFilterValueHandlers.h"
#include "DataStorage/Features.h"
#include "DataStorage/MapKey.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/Package.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectMarks.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableTextBox.h"

class SWidget;

/** Helper functions for frontend filters */
namespace FrontendFilterHelper
{
	/**
	 * Get a set of dependencies as package name's from a list of assets found with the given Asset Registry Filter.
	 * @param InAssetRegistryFilter		The filter to find assets for in the asset registry.
	 * @param AssetRegistry				The Asset Registry to find assets and dependencies.
	 * @param OutDependencySet			The output of dependencies found from a set of assets.
	 */
	void GetDependencies(const FARFilter& InAssetRegistryFilter, const IAssetRegistry& AssetRegistry, TSet<FName>& OutDependencySet)
	{
		TArray<FName> PackageNamesToProcess;
		{
			TArray<FAssetData> FoundAssets;
			AssetRegistry.GetAssets(InAssetRegistryFilter, FoundAssets);
			for (const FAssetData& AssetData : FoundAssets)
			{
				PackageNamesToProcess.Add(AssetData.PackageName);
				OutDependencySet.Add(AssetData.PackageName);
			}
		}

		TArray<FAssetIdentifier> AssetDependencies;
		while (PackageNamesToProcess.Num() > 0)
		{
			const FName PackageName = PackageNamesToProcess.Pop(EAllowShrinking::No);
			AssetDependencies.Reset();
			AssetRegistry.GetDependencies(FAssetIdentifier(PackageName), AssetDependencies);
			for (const FAssetIdentifier& Dependency : AssetDependencies)
			{
				bool bIsAlreadyInSet = false;
				OutDependencySet.Add(Dependency.PackageName, &bIsAlreadyInSet);
				if (bIsAlreadyInSet == false)
				{
					PackageNamesToProcess.Add(Dependency.PackageName);
				}
			}
		}
	}

	static const FName NAME_BookmarkWorld(TEXT("WorldName"));
	bool GetWorldBookmark(const IAssetRegistry* AssetRegistry, const FAssetData& InAssetData, FAssetData& OutValue)
	{
		// This would soon to be replace by World Asset concept.
		FString WorldAssetName;
		if (InAssetData.GetTagValue(NAME_BookmarkWorld, WorldAssetName))
		{
			FSoftObjectPath WorldAsset = FSoftObjectPath(WorldAssetName);
			UAssetRegistryHelpers::FixupRedirectedAssetPath(WorldAsset);
			OutValue = AssetRegistry->GetAssetByObjectPath(WorldAsset);
			return true;
		}

		return false;
	}
}

/////////////////////////////////////////
// FFrontendFilter_Text
/////////////////////////////////////////
namespace UE::ContentBrowser
{
	/** Keys used by FFrontendFilter_GatherDynamicCollectionsExpressionContext::TestComplexExpression */
	const FName CollectionKeyName("Collection");
	const FName TagKeyName("Tag");
}

FFrontendFilter_GatherDynamicCollectionsExpressionContext::FFrontendFilter_GatherDynamicCollectionsExpressionContext(const TArray<TSharedPtr<ICollectionContainer>>& CollectionContainers, TArray<FCollectionRef>& OutReferencedDynamicCollections)
	: AvailableDynamicCollections()
	, ReferencedDynamicCollections(OutReferencedDynamicCollections)
	, CurrentRecursionDepth(0)
{
	TArray<FCollectionNameType> AvailableCollections;
	for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
	{
		if (CollectionContainer->IsHidden())
		{
			continue;
		}

		AvailableCollections.Reset();
		CollectionContainer->GetCollections(AvailableCollections);

		for (const FCollectionNameType& AvailableCollection : AvailableCollections)
		{
			// Only care about dynamic collections
			ECollectionStorageMode::Type StorageMode = ECollectionStorageMode::Static;
			CollectionContainer->GetCollectionStorageMode(AvailableCollection.Name, AvailableCollection.Type, StorageMode);
			if (StorageMode != ECollectionStorageMode::Dynamic)
			{
				continue;
			}

			AvailableDynamicCollections.Emplace(CollectionContainer, AvailableCollection);
		}
	}
}

FFrontendFilter_GatherDynamicCollectionsExpressionContext::~FFrontendFilter_GatherDynamicCollectionsExpressionContext()
{
	// Sort and populate the final list of referenced dynamic collections
	FoundDynamicCollections.Sort([](const FDynamicCollectionRefAndDepth& A, const FDynamicCollectionRefAndDepth& B)
	{
		return A.Collection.Container.Get() < B.Collection.Container.Get() ||
			(A.Collection.Container.Get() == B.Collection.Container.Get() && A.RecursionDepth > B.RecursionDepth);
	});

	ReferencedDynamicCollections.Reset();
	ReferencedDynamicCollections.Reserve(FoundDynamicCollections.Num());
	for (const auto& FoundDynamicCollection : FoundDynamicCollections)
	{
		ReferencedDynamicCollections.Add(FoundDynamicCollection.Collection);
	}
}

bool FFrontendFilter_GatherDynamicCollectionsExpressionContext::TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const 
{
	TestAgainstAvailableCollections(InValue, InTextComparisonMode);
	return false;
}

bool FFrontendFilter_GatherDynamicCollectionsExpressionContext::TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const 
{
	using namespace UE::ContentBrowser;

	// Special case for collections, as these aren't contained within the asset registry meta-data
	if (InKey == CollectionKeyName || InKey == TagKeyName)
	{
		// Collections can only work with Equal or NotEqual type tests
		if (InComparisonOperation != ETextFilterComparisonOperation::Equal && InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
		{
			return false;
		}

		TestAgainstAvailableCollections(InValue, InTextComparisonMode);
	}

	return false;
}

bool FFrontendFilter_GatherDynamicCollectionsExpressionContext::TestAgainstAvailableCollections(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	for (const FCollectionRef& DynamicCollection : AvailableDynamicCollections)
	{
		const FString DynamicCollectionNameStr = DynamicCollection.Name.ToString();
		if (TextFilterUtils::TestBasicStringExpression(DynamicCollectionNameStr, InValue, InTextComparisonMode))
		{
			const bool bCollectionAlreadyProcessed = FoundDynamicCollections.ContainsByPredicate([&DynamicCollection](const FDynamicCollectionRefAndDepth& Other)
			{
				return DynamicCollection.Container == Other.Collection.Container &&
					DynamicCollection.Name == Other.Collection.Name &&
					DynamicCollection.Type == Other.Collection.Type;
			});

			if (!bCollectionAlreadyProcessed)
			{
				FoundDynamicCollections.Emplace(DynamicCollection, CurrentRecursionDepth);

				// Also need to gather any collections referenced by this dynamic collection
				++CurrentRecursionDepth;
				bool bUnused = false;
				DynamicCollection.Container->TestDynamicQuery(DynamicCollection.Name, DynamicCollection.Type, *this, bUnused);
				--CurrentRecursionDepth;
			}

			return true;
		}
	}

	return false;
}

/** Expression context to test the given asset data against the current text filter */
class FFrontendFilter_TextFilterExpressionContext : public ITextFilterExpressionContext
{
public:
	typedef TRemoveReference<FAssetFilterType>::Type* FAssetFilterTypePtr;

	FFrontendFilter_TextFilterExpressionContext(const TArray<TSharedPtr<ICollectionContainer>>& InCollectionContainers, const TArray<FCollectionRef>& InReferencedDynamicCollections)
		: CollectionContainers(InCollectionContainers)
		, ReferencedDynamicCollections(InReferencedDynamicCollections)
		, AssetPtr(nullptr)
		, bIncludeClassName(true)
		, bIncludeAssetPath(false)
		, bIncludeCollectionNames(true)
		, NameKeyName("Name")
		, PathKeyName("Path")
		, ClassKeyName("Class")
		, TypeKeyName("Type")
		, CollectionKeyName("Collection")
		, TagKeyName("Tag")
	{
	}

	void SetAsset(FAssetFilterTypePtr InAsset)
	{
		AssetPtr = InAsset;

		AssetDisplayName = AssetPtr->GetDisplayName().ToString();
		AssetDisplayName.ToUpperInline();

		if (bIncludeAssetPath)
		{
			// Get the full asset path, and also split it so we can compare each part in the filter
			AssetPtr->GetVirtualPath().AppendString(AssetFullPath);
			{
				int32 LastSlashIndex = INDEX_NONE;
				int32 LastDotIndex = INDEX_NONE;
				if (AssetFullPath.FindLastChar(TEXT('/'), LastSlashIndex) && AssetFullPath.FindLastChar(TEXT('.'), LastDotIndex) && LastDotIndex > LastSlashIndex)
				{
					AssetFullPath.LeftInline(LastDotIndex, EAllowShrinking::No);
				}
			}
			AssetFullPath.ParseIntoArray(AssetSplitPath, TEXT("/"));
			AssetFullPath.ToUpperInline();

			if (bIncludeClassName)
			{
				// Get the full export text path as people sometimes search by copying this (requires class and asset path search to be enabled in order to match)
				AssetPtr->AppendItemReference(AssetExportTextName);
				AssetExportTextName.ToUpperInline();
			}
		}

		if (!CollectionContainers.IsEmpty())
		{
			FSoftObjectPath ItemCollectionId;
			if (AssetPtr->TryGetCollectionId(ItemCollectionId))
			{
				for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
				{
					CollectionContainer->GetCollectionsContainingObject(ItemCollectionId, ECollectionShareType::CST_All, AssetCollectionNames, ECollectionRecursionFlags::SelfAndChildren);
				}

				// Test the dynamic collections from the active query against the current asset
				// We can do this as a flat list since FFrontendFilter_GatherDynamicCollectionsExpressionContext has already taken care of processing the recursion
				for (const FCollectionRef& DynamicCollection : ReferencedDynamicCollections)
				{
					bool bPassesCollectionFilter = false;
					DynamicCollection.Container->TestDynamicQuery(DynamicCollection.Name, DynamicCollection.Type, *this, bPassesCollectionFilter);
					if (bPassesCollectionFilter)
					{
						AssetCollectionNames.AddUnique(DynamicCollection.Name);
					}
				}
			}
		}
	}

	void ClearAsset()
	{
		AssetPtr = nullptr;
		AssetFullPath.Reset();
		AssetExportTextName.Reset();
		AssetSplitPath.Reset();
		AssetCollectionNames.Reset();
		AssetDisplayName.Reset();
	}

	void SetIncludeClassName(const bool InIncludeClassName)
	{
		bIncludeClassName = InIncludeClassName;
	}

	bool GetIncludeClassName() const
	{
		return bIncludeClassName;
	}

	void SetIncludeAssetPath(const bool InIncludeAssetPath)
	{
		bIncludeAssetPath = InIncludeAssetPath;
	}

	bool GetIncludeAssetPath() const
	{
		return bIncludeAssetPath;
	}

	void SetIncludeCollectionNames(const bool InIncludeCollectionNames)
	{
		bIncludeCollectionNames = InIncludeCollectionNames;
	}

	bool GetIncludeCollectionNames() const
	{
		return bIncludeCollectionNames;
	}

	virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		bool bIsHandlerMatch = false;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (UTextFilterValueHandlers::HandleTextFilterValue(*AssetPtr, InValue, InTextComparisonMode, bIsHandlerMatch))
		{
			return bIsHandlerMatch;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (InValue.CompareName(AssetPtr->GetItemName(), InTextComparisonMode))
		{
			return true;
		}

		if (InValue.CompareFString(AssetDisplayName, InTextComparisonMode))
		{
			return true;
		}

		if (bIncludeAssetPath)
		{
			if (InValue.CompareFString(AssetFullPath, InTextComparisonMode))
			{
				return true;
			}

			for (const FString& AssetPathPart : AssetSplitPath)
			{
				if (InValue.CompareFString(AssetPathPart, InTextComparisonMode))
				{
					return true;
				}
			}
		}

		if (bIncludeClassName)
		{
			const FContentBrowserItemDataAttributeValue ClassValue = AssetPtr->GetItemAttribute(NAME_Class);
			if (ClassValue.IsValid() && InValue.CompareName(ClassValue.GetValue<FName>(), InTextComparisonMode))
			{
				return true;
			}
		}

		if (bIncludeClassName && bIncludeAssetPath)
		{
			// Only test this if we're searching the class name and asset path too, as the exported text contains the type and path in the string
			if (InValue.CompareFString(AssetExportTextName, InTextComparisonMode))
			{
				return true;
			}
		}

		if (bIncludeCollectionNames)
		{
			for (const FName& AssetCollectionName : AssetCollectionNames)
			{
				if (InValue.CompareName(AssetCollectionName, InTextComparisonMode))
				{
					return true;
				}
			}
		}

		return false;
	}

	virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		bool bIsHandlerMatch = false;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (UTextFilterKeyValueHandlers::HandleTextFilterKeyValue(*AssetPtr, InKey, InValue, InComparisonOperation, InTextComparisonMode, bIsHandlerMatch))
		{
			return bIsHandlerMatch;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Special case for the asset name, as this isn't contained within the asset registry meta-data
		if (InKey == NameKeyName)
		{
			// Names can only work with Equal or NotEqual type tests
			if (InComparisonOperation != ETextFilterComparisonOperation::Equal && InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
			{
				return false;
			}

			const bool bIsMatch = TextFilterUtils::TestBasicStringExpression(AssetPtr->GetItemName(), InValue, InTextComparisonMode);
			return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bIsMatch : !bIsMatch;
		}

		// Special case for the asset path, as this isn't contained within the asset registry meta-data
		if (InKey == PathKeyName)
		{
			// Paths can only work with Equal or NotEqual type tests
			if (InComparisonOperation != ETextFilterComparisonOperation::Equal && InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
			{
				return false;
			}

			// If the comparison mode is partial, then we only need to test the ObjectPath as that contains the other two as sub-strings
			bool bIsMatch = false;
			if (InTextComparisonMode == ETextFilterTextComparisonMode::Partial)
			{
				bIsMatch = TextFilterUtils::TestBasicStringExpression(AssetPtr->GetVirtualPath(), InValue, InTextComparisonMode);
			}
			else
			{
				bIsMatch = TextFilterUtils::TestBasicStringExpression(AssetPtr->GetVirtualPath(), InValue, InTextComparisonMode)
					|| (!AssetFullPath.IsEmpty() && TextFilterUtils::TestBasicStringExpression(AssetFullPath, InValue, InTextComparisonMode));
			}
			return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bIsMatch : !bIsMatch;
		}

		// Special case for the asset type, as this isn't contained within the asset registry meta-data
		if (InKey == ClassKeyName || InKey == TypeKeyName)
		{
			// Class names can only work with Equal or NotEqual type tests
			if (InComparisonOperation != ETextFilterComparisonOperation::Equal && InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
			{
				return false;
			}

			const FContentBrowserItemDataAttributeValue ClassValue = AssetPtr->GetItemAttribute(NAME_Class);
			const bool bIsMatch = ClassValue.IsValid() && TextFilterUtils::TestBasicStringExpression(ClassValue.GetValue<FName>(), InValue, InTextComparisonMode);
			return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bIsMatch : !bIsMatch;
		}

		// Special case for collections, as these aren't contained within the asset registry meta-data
		if (InKey == CollectionKeyName || InKey == TagKeyName)
		{
			// Collections can only work with Equal or NotEqual type tests
			if (InComparisonOperation != ETextFilterComparisonOperation::Equal && InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
			{
				return false;
			}

			bool bFoundMatch = false;
			for (const FName& AssetCollectionName : AssetCollectionNames)
			{
				if (TextFilterUtils::TestBasicStringExpression(AssetCollectionName, InValue, InTextComparisonMode))
				{
					bFoundMatch = true;
					break;
				}
			}

			return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bFoundMatch : !bFoundMatch;
		}

		// Generic handling for anything in the asset meta-data
		{
			const FContentBrowserItemDataAttributeValue AttributeValue = AssetPtr->GetItemAttribute(InKey);
			if (AttributeValue.IsValid())
			{
				return TextFilterUtils::TestComplexExpression(AttributeValue.GetValue<FString>(), InValue, InComparisonOperation, InTextComparisonMode);
			}
		}

		return false;
	}

private:
	const TArray<TSharedPtr<ICollectionContainer>>& CollectionContainers;

	/** An array of dynamic collections that are being referenced by the current query. These should be tested against each asset when it's looking for collections that contain it */
	const TArray<FCollectionRef>& ReferencedDynamicCollections;

	/** Pointer to the asset we're currently filtering */
	FAssetFilterTypePtr AssetPtr;

	/** Full path of the current asset */
	FString AssetFullPath;

	/** The export text name of the current asset */
	FString AssetExportTextName;

	/** Display name of the current asset */
	FString AssetDisplayName;

	/** Split path of the current asset */
	TArray<FString> AssetSplitPath;

	/** Names of the collections that the current asset is in */
	TArray<FName> AssetCollectionNames;

	/** Are we supposed to include the class name in our basic string tests? */
	bool bIncludeClassName;

	/** Search inside the entire asset path? */
	bool bIncludeAssetPath;

	/** Search collection names? */
	bool bIncludeCollectionNames;

	/** Keys used by TestComplexExpression */
	const FName NameKeyName;
	const FName PathKeyName;
	const FName ClassKeyName;
	const FName TypeKeyName;
	const FName CollectionKeyName;
	const FName TagKeyName;
};

FFrontendFilter_Text::FFrontendFilter_Text()
	: FFrontendFilter(nullptr)
	, ReferencedDynamicCollections()
	, TextFilterExpressionContext(MakeShareable(new FFrontendFilter_TextFilterExpressionContext(CollectionContainers, ReferencedDynamicCollections)))
	, TextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::Complex)
{
	FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();

	CollectionManagerModule.Get().GetCollectionContainers(CollectionContainers);

	for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
	{
		FCollectionContainerHandles& Handles = CollectionContainerHandles.AddDefaulted_GetRef();

		Handles.OnIsHiddenChangedHandle = CollectionContainer->OnIsHiddenChanged().AddRaw(this, &FFrontendFilter_Text::HandleIsHiddenChanged);
		Handles.OnCollectionCreatedHandle = CollectionContainer->OnCollectionCreated().AddRaw(this, &FFrontendFilter_Text::HandleCollectionCreated);
		Handles.OnCollectionDestroyedHandle = CollectionContainer->OnCollectionDestroyed().AddRaw(this, &FFrontendFilter_Text::HandleCollectionDestroyed);
		Handles.OnCollectionRenamedHandle = CollectionContainer->OnCollectionRenamed().AddRaw(this, &FFrontendFilter_Text::HandleCollectionRenamed);
		Handles.OnCollectionUpdatedHandle = CollectionContainer->OnCollectionUpdated().AddRaw(this, &FFrontendFilter_Text::HandleCollectionUpdated);
	}

	OnCollectionContainerCreatedHandle = CollectionManagerModule.Get().OnCollectionContainerCreated().AddRaw(this, &FFrontendFilter_Text::HandleCollectionContainerCreated);
	OnCollectionContainerDestroyedHandle = CollectionManagerModule.Get().OnCollectionContainerDestroyed().AddRaw(this, &FFrontendFilter_Text::HandleCollectionContainerDestroyed);
}

FFrontendFilter_Text::~FFrontendFilter_Text()
{
	// Check IsModuleAvailable as we might be in the process of shutting down...
	if (FCollectionManagerModule::IsModuleAvailable())
	{
		FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();

		CollectionManagerModule.Get().OnCollectionContainerCreated().Remove(OnCollectionContainerCreatedHandle);
		CollectionManagerModule.Get().OnCollectionContainerDestroyed().Remove(OnCollectionContainerDestroyedHandle);

		for (int32 Index = 0; Index < CollectionContainers.Num(); ++Index)
		{
			const TSharedPtr<ICollectionContainer>& CollectionContainer = CollectionContainers[Index];
			FCollectionContainerHandles& Handles = CollectionContainerHandles[Index];

			CollectionContainer->OnIsHiddenChanged().Remove(Handles.OnIsHiddenChangedHandle);
			CollectionContainer->OnCollectionCreated().Remove(Handles.OnCollectionCreatedHandle);
			CollectionContainer->OnCollectionDestroyed().Remove(Handles.OnCollectionDestroyedHandle);
			CollectionContainer->OnCollectionRenamed().Remove(Handles.OnCollectionRenamedHandle);
			CollectionContainer->OnCollectionUpdated().Remove(Handles.OnCollectionUpdatedHandle);
		}
	}
}

bool FFrontendFilter_Text::PassesFilter(FAssetFilterType InItem) const
{
	TextFilterExpressionContext->SetAsset(&InItem);
	const bool bMatched = TextFilterExpressionEvaluator.TestTextFilter(*TextFilterExpressionContext);
	TextFilterExpressionContext->ClearAsset();
	return bMatched;
}

FText FFrontendFilter_Text::GetRawFilterText() const
{
	return TextFilterExpressionEvaluator.GetFilterText();
}

void FFrontendFilter_Text::SetRawFilterText(const FText& InFilterText)
{
	if (TextFilterExpressionEvaluator.SetFilterText(InFilterText))
	{
		RebuildReferencedDynamicCollections();

		// Will trigger a re-filter with the new text
		BroadcastChangedEvent();
	}
}

FText FFrontendFilter_Text::GetFilterErrorText() const
{
	return TextFilterExpressionEvaluator.GetFilterErrorText();
}

void FFrontendFilter_Text::SetIncludeClassName(const bool InIncludeClassName)
{
	if (TextFilterExpressionContext->GetIncludeClassName() != InIncludeClassName)
	{
		TextFilterExpressionContext->SetIncludeClassName(InIncludeClassName);

		// Will trigger a re-filter with the new setting
		BroadcastChangedEvent();
	}
}

void FFrontendFilter_Text::SetIncludeAssetPath(const bool InIncludeAssetPath)
{
	if (TextFilterExpressionContext->GetIncludeAssetPath() != InIncludeAssetPath)
	{
		TextFilterExpressionContext->SetIncludeAssetPath(InIncludeAssetPath);

		// Will trigger a re-filter with the new setting
		BroadcastChangedEvent();
	}
}

bool FFrontendFilter_Text::GetIncludeAssetPath() const
{
	return TextFilterExpressionContext->GetIncludeAssetPath();
}

void FFrontendFilter_Text::SetIncludeCollectionNames(const bool InIncludeCollectionNames)
{
	if (TextFilterExpressionContext->GetIncludeCollectionNames() != InIncludeCollectionNames)
	{
		TextFilterExpressionContext->SetIncludeCollectionNames(InIncludeCollectionNames);

		// Will trigger a re-filter with the new collections
		BroadcastChangedEvent();
	}
}

bool FFrontendFilter_Text::GetIncludeCollectionNames() const
{
	return TextFilterExpressionContext->GetIncludeCollectionNames();
}

void FFrontendFilter_Text::HandleCollectionContainerCreated(const TSharedRef<ICollectionContainer>& CollectionContainer)
{
	CollectionContainers.Add(CollectionContainer);
	FCollectionContainerHandles& Handles = CollectionContainerHandles.AddDefaulted_GetRef();

	Handles.OnIsHiddenChangedHandle = CollectionContainer->OnIsHiddenChanged().AddRaw(this, &FFrontendFilter_Text::HandleIsHiddenChanged);
	Handles.OnCollectionCreatedHandle = CollectionContainer->OnCollectionCreated().AddRaw(this, &FFrontendFilter_Text::HandleCollectionCreated);
	Handles.OnCollectionDestroyedHandle = CollectionContainer->OnCollectionDestroyed().AddRaw(this, &FFrontendFilter_Text::HandleCollectionDestroyed);
	Handles.OnCollectionRenamedHandle = CollectionContainer->OnCollectionRenamed().AddRaw(this, &FFrontendFilter_Text::HandleCollectionRenamed);
	Handles.OnCollectionUpdatedHandle = CollectionContainer->OnCollectionUpdated().AddRaw(this, &FFrontendFilter_Text::HandleCollectionUpdated);

	if (!CollectionContainer->IsHidden())
	{
		RebuildReferencedDynamicCollections();

		// Will trigger a re-filter with the new collections
		BroadcastChangedEvent();
	}
}

void FFrontendFilter_Text::HandleCollectionContainerDestroyed(const TSharedRef<ICollectionContainer>& CollectionContainer)
{
	int32 Index;
	if (CollectionContainers.Find(CollectionContainer, Index))
	{
		{
			FCollectionContainerHandles& Handles = CollectionContainerHandles[Index];

			CollectionContainer->OnIsHiddenChanged().Remove(Handles.OnIsHiddenChangedHandle);
			CollectionContainer->OnCollectionCreated().Remove(Handles.OnCollectionCreatedHandle);
			CollectionContainer->OnCollectionDestroyed().Remove(Handles.OnCollectionDestroyedHandle);
			CollectionContainer->OnCollectionRenamed().Remove(Handles.OnCollectionRenamedHandle);
			CollectionContainer->OnCollectionUpdated().Remove(Handles.OnCollectionUpdatedHandle);
		}

		CollectionContainers.RemoveAt(Index);
		CollectionContainerHandles.RemoveAt(Index);

		if (ReferencedDynamicCollections.ContainsByPredicate([&CollectionContainer](const FCollectionRef& DynamicCollection)
			{
				return DynamicCollection.Container == CollectionContainer;
			}))
		{
			RebuildReferencedDynamicCollections();

			// Will trigger a re-filter with the new collections
			BroadcastChangedEvent();
		}
	}
}

void FFrontendFilter_Text::HandleIsHiddenChanged(ICollectionContainer& CollectionContainer, bool bIsHidden)
{
	if (!bIsHidden || ReferencedDynamicCollections.ContainsByPredicate([&CollectionContainer](const FCollectionRef& DynamicCollection)
		{
			return DynamicCollection.Container.Get() == &CollectionContainer;
		}))
	{
		RebuildReferencedDynamicCollections();

		// Will trigger a re-filter with the new collections
		BroadcastChangedEvent();
	}
}

void FFrontendFilter_Text::HandleCollectionCreated(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection)
{
	RebuildReferencedDynamicCollections();

	// Will trigger a re-filter with the new collections
	BroadcastChangedEvent();
}

void FFrontendFilter_Text::HandleCollectionDestroyed(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection)
{
	if (ReferencedDynamicCollections.ContainsByPredicate([&CollectionContainer, &Collection](const FCollectionRef& DynamicCollection)
		{
			return DynamicCollection.Container.Get() == &CollectionContainer &&
				DynamicCollection.Name == Collection.Name &&
				DynamicCollection.Type == Collection.Type;
		}))
	{
		RebuildReferencedDynamicCollections();

		// Will trigger a re-filter with the new collections
		BroadcastChangedEvent();
	}
}

void FFrontendFilter_Text::HandleCollectionRenamed(ICollectionContainer& CollectionContainer, const FCollectionNameType& OriginalCollection, const FCollectionNameType& NewCollection)
{
	for (FCollectionRef& DynamicCollection : ReferencedDynamicCollections)
	{
		if (DynamicCollection.Container.Get() == &CollectionContainer &&
			DynamicCollection.Name == OriginalCollection.Name &&
			DynamicCollection.Type == OriginalCollection.Type)
		{
			DynamicCollection.Name = NewCollection.Name;
			DynamicCollection.Type = NewCollection.Type;
		}
	}
}

void FFrontendFilter_Text::HandleCollectionUpdated(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection)
{
	RebuildReferencedDynamicCollections();

	// Will trigger a re-filter with the new collections
	BroadcastChangedEvent();
}

void FFrontendFilter_Text::RebuildReferencedDynamicCollections()
{
	TextFilterExpressionEvaluator.TestTextFilter(FFrontendFilter_GatherDynamicCollectionsExpressionContext(CollectionContainers, ReferencedDynamicCollections));
}

/////////////////////////////////////////
// FFrontendFilter_CheckedOut
/////////////////////////////////////////

FFrontendFilter_CheckedOut::FFrontendFilter_CheckedOut(TSharedPtr<FFrontendFilterCategory> InCategory) 
	: FFrontendFilter(MoveTemp(InCategory)),
	bSourceControlEnabled(false)
{
	
}

void FFrontendFilter_CheckedOut::ActiveStateChanged(bool bActive)
{
	if(bActive)
	{
		RequestStatus();
	}
}

void FFrontendFilter_CheckedOut::SetCurrentFilter(TArrayView<const FName> InSourcePaths, const FContentBrowserDataFilter& InBaseFilter)
{
	bSourceControlEnabled = ISourceControlModule::Get().IsEnabled();
}

bool FFrontendFilter_CheckedOut::PassesFilter(FAssetFilterType InItem) const
{
	if (!bSourceControlEnabled)
	{
		return false;
	}

	FString ItemDiskPath;
	if (!InItem.GetItemPhysicalPath(ItemDiskPath))
	{
		return false;
	}
	ItemDiskPath = FPaths::ConvertRelativePathToFull(MoveTemp(ItemDiskPath));

	FSourceControlStatePtr SourceControlState = ISourceControlModule::Get().GetProvider().GetState(ItemDiskPath, EStateCacheUsage::Use);
	return SourceControlState.IsValid() && (SourceControlState->IsCheckedOut() || SourceControlState->IsAdded());
}

void FFrontendFilter_CheckedOut::RequestStatus()
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	if ( ISourceControlModule::Get().IsEnabled() )
	{
		// Request the opened files at filter construction time to make sure checked out files have the correct state for the filter
		TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
		UpdateStatusOperation->SetGetOpenedOnly(true);
		SourceControlProvider.Execute(UpdateStatusOperation, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &FFrontendFilter_CheckedOut::SourceControlOperationComplete) );
	}
}

void FFrontendFilter_CheckedOut::SourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	BroadcastChangedEvent();
}

/////////////////////////////////////////
// FFrontendFilter_NotSourceControlled
/////////////////////////////////////////

FFrontendFilter_NotSourceControlled::FFrontendFilter_NotSourceControlled(TSharedPtr<FFrontendFilterCategory> InCategory) 
	: FFrontendFilter(MoveTemp(InCategory)),
	bSourceControlEnabled(false),
	bIsRequestStatusRunning(false),
	bInitialRequestCompleted(false)
{

}

void FFrontendFilter_NotSourceControlled::ActiveStateChanged(bool bActive)
{
	if (bActive)
	{
		if (!bIsRequestStatusRunning)
		{
			RequestStatus();
		}
	}
}

void FFrontendFilter_NotSourceControlled::SetCurrentFilter(TArrayView<const FName> InSourcePaths, const FContentBrowserDataFilter& InBaseFilter)
{
	bSourceControlEnabled = ISourceControlModule::Get().IsEnabled();
}

bool FFrontendFilter_NotSourceControlled::PassesFilter(FAssetFilterType InItem) const
{
	if (!bSourceControlEnabled)
	{
		return true;
	}

	// Hide all items until the first status request finishes
	if (!bInitialRequestCompleted)
	{
		return false;
	}

	FString ItemDiskPath;
	if (!InItem.GetItemPhysicalPath(ItemDiskPath))
	{
		return false;
	}
	ItemDiskPath = FPaths::ConvertRelativePathToFull(MoveTemp(ItemDiskPath));

	FSourceControlStatePtr SourceControlState = ISourceControlModule::Get().GetProvider().GetState(ItemDiskPath, EStateCacheUsage::Use);
	if (!SourceControlState.IsValid())
	{
		return false;
	}

	if (SourceControlState->IsUnknown())
	{
		return true;
	}

	if (SourceControlState->IsSourceControlled())
	{
		return false;
	}

	return true;
}

void FFrontendFilter_NotSourceControlled::RequestStatus()
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	bSourceControlEnabled = ISourceControlModule::Get().IsEnabled();
	if ( bSourceControlEnabled )
	{
		bSourceControlEnabled = true;
		bIsRequestStatusRunning = true;

		// Request the state of files at filter construction time to make sure files have the correct state for the filter
		TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();

		TArray<FString> Filenames = SourceControlHelpers::GetSourceControlLocations(/*bContentOnly*/true);

		UpdateStatusOperation->SetCheckingAllFiles(false);
		SourceControlProvider.Execute(UpdateStatusOperation, Filenames, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &FFrontendFilter_NotSourceControlled::SourceControlOperationComplete));
	}
}

void FFrontendFilter_NotSourceControlled::SourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	bIsRequestStatusRunning = false;
	bInitialRequestCompleted = true;

	BroadcastChangedEvent();
}

/////////////////////////////////////////
// FFrontendFilter_Modified
/////////////////////////////////////////

FFrontendFilter_Modified::FFrontendFilter_Modified(TSharedPtr<FFrontendFilterCategory> InCategory)
	: FFrontendFilter(MoveTemp(InCategory))
	, bIsCurrentlyActive(false)
{
	UPackage::PackageDirtyStateChangedEvent.AddRaw(this, &FFrontendFilter_Modified::OnPackageDirtyStateUpdated);
}

FFrontendFilter_Modified::~FFrontendFilter_Modified()
{
	UPackage::PackageDirtyStateChangedEvent.RemoveAll(this);
}

void FFrontendFilter_Modified::ActiveStateChanged(bool bActive)
{
	bIsCurrentlyActive = bActive;
}

bool FFrontendFilter_Modified::PassesFilter(FAssetFilterType InItem) const
{
	return InItem.IsDirty();
}

void FFrontendFilter_Modified::OnPackageDirtyStateUpdated(UPackage* Package)
{
	if (bIsCurrentlyActive)
	{
		BroadcastChangedEvent();
	}
}

/////////////////////////////////////////
// FFrontendFilter_ReplicatedBlueprint
/////////////////////////////////////////

bool FFrontendFilter_ReplicatedBlueprint::PassesFilter(FAssetFilterType InItem) const
{
	const FContentBrowserItemDataAttributeValue AttributeValue = InItem.GetItemAttribute(FBlueprintTags::NumReplicatedProperties);
	if (AttributeValue.IsValid())
	{
		const int32 NumReplicatedProperties = AttributeValue.GetValue<int32>();
		return NumReplicatedProperties > 0;
	}
	return false;
}

/////////////////////////////////////////
// FFrontendFilter_ArbitraryComparisonOperation
/////////////////////////////////////////

#define LOCTEXT_NAMESPACE "ContentBrowser"

FFrontendFilter_ArbitraryComparisonOperation::FFrontendFilter_ArbitraryComparisonOperation(TSharedPtr<FFrontendFilterCategory> InCategory)
	: FFrontendFilter(MoveTemp(InCategory))
	, TagName(TEXT("TagName"))
	, TargetTagValue(TEXT("Value"))
	, ComparisonOp(ETextFilterComparisonOperation::NotEqual)
{
}

FString FFrontendFilter_ArbitraryComparisonOperation::GetName() const
{
	return TEXT("CompareTags");
}

FText FFrontendFilter_ArbitraryComparisonOperation::GetDisplayName() const
{
	return FText::Format(LOCTEXT("FFrontendFilter_CompareOperation", "Compare Tags ({0} {1} {2})"),
		FText::FromName(TagName),
		FText::AsCultureInvariant(ConvertOperationToString(ComparisonOp)),
		FText::AsCultureInvariant(TargetTagValue));
}

FText FFrontendFilter_ArbitraryComparisonOperation::GetToolTipText() const
{
	return LOCTEXT("FFrontendFilter_CompareOperation_ToolTip", "Compares AssetRegistrySearchable values on assets with a target value.");
}

bool FFrontendFilter_ArbitraryComparisonOperation::PassesFilter(FAssetFilterType InItem) const
{
	const FContentBrowserItemDataAttributeValue AttributeValue = InItem.GetItemAttribute(TagName);
	if (AttributeValue.IsValid())
	{
		return TextFilterUtils::TestComplexExpression(AttributeValue.GetValue<FString>(), TargetTagValue, ComparisonOp, ETextFilterTextComparisonMode::Exact);
	}
	else
	{
		// Failed to find the tag, can't pass the filter
		//@TODO: Maybe we should succeed here if the operation is !=
		return false;
	}
}

void FFrontendFilter_ArbitraryComparisonOperation::ModifyContextMenu(FMenuBuilder& MenuBuilder)
{
	FUIAction Action;

	MenuBuilder.BeginSection(TEXT("ComparsionSection"), LOCTEXT("ComparisonSectionHeading", "AssetRegistrySearchable Comparison"));

	TSharedRef<SWidget> KeyWidget =
		SNew(SEditableTextBox)
		.Text_Raw(this, &FFrontendFilter_ArbitraryComparisonOperation::GetKeyValueAsText)
		.OnTextCommitted_Raw(this, &FFrontendFilter_ArbitraryComparisonOperation::OnKeyValueTextCommitted)
		.MinDesiredWidth(100.0f);
	TSharedRef<SWidget> ValueWidget = SNew(SEditableTextBox)
		.Text_Raw(this, &FFrontendFilter_ArbitraryComparisonOperation::GetTargetValueAsText)
		.OnTextCommitted_Raw(this, &FFrontendFilter_ArbitraryComparisonOperation::OnTargetValueTextCommitted)
		.MinDesiredWidth(100.0f);

	MenuBuilder.AddWidget(KeyWidget, LOCTEXT("KeyMenuDesc", "Tag"));
	MenuBuilder.AddWidget(ValueWidget, LOCTEXT("ValueMenuDesc", "Target Value"));

#define UE_SET_COMP_OP(Operation) \
	MenuBuilder.AddMenuEntry(FText::AsCultureInvariant(ConvertOperationToString(Operation)), \
		LOCTEXT("SwitchOpsTooltip", "Switch comparsion type"), \
		FSlateIcon(), \
		FUIAction(FExecuteAction::CreateRaw(this, &FFrontendFilter_ArbitraryComparisonOperation::SetComparisonOperation, Operation), FCanExecuteAction(), FIsActionChecked::CreateRaw(this, &FFrontendFilter_ArbitraryComparisonOperation::IsComparisonOperationEqualTo, Operation)), \
		NAME_None, \
		EUserInterfaceActionType::RadioButton);

	UE_SET_COMP_OP(ETextFilterComparisonOperation::Equal);
	UE_SET_COMP_OP(ETextFilterComparisonOperation::NotEqual);
	UE_SET_COMP_OP(ETextFilterComparisonOperation::Less);
	UE_SET_COMP_OP(ETextFilterComparisonOperation::LessOrEqual);
	UE_SET_COMP_OP(ETextFilterComparisonOperation::Greater);
	UE_SET_COMP_OP(ETextFilterComparisonOperation::GreaterOrEqual);
#undef UE_SET_COMP_OP

	MenuBuilder.EndSection();
}

void FFrontendFilter_ArbitraryComparisonOperation::SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const
{
	GConfig->SetString(*IniSection, *(SettingsString + TEXT(".Key")), *TagName.ToString(), IniFilename);
	GConfig->SetString(*IniSection, *(SettingsString + TEXT(".Value")), *TargetTagValue, IniFilename);
	GConfig->SetString(*IniSection, *(SettingsString + TEXT(".Op")), *FString::FromInt((int32)ComparisonOp), IniFilename);
}

void FFrontendFilter_ArbitraryComparisonOperation::LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString)
{
	FString TagNameAsString;
	if (GConfig->GetString(*IniSection, *(SettingsString + TEXT(".Key")), TagNameAsString, IniFilename))
	{
		TagName = *TagNameAsString;
	}

	GConfig->GetString(*IniSection, *(SettingsString + TEXT(".Value")), TargetTagValue, IniFilename);

	int32 OpAsInteger;
	if (GConfig->GetInt(*IniSection, *(SettingsString + TEXT(".Op")), OpAsInteger, IniFilename))
	{
		ComparisonOp = (ETextFilterComparisonOperation)OpAsInteger;
	}
}

void FFrontendFilter_ArbitraryComparisonOperation::SetComparisonOperation(ETextFilterComparisonOperation NewOp)
{
	ComparisonOp = NewOp;
	BroadcastChangedEvent();
}

bool FFrontendFilter_ArbitraryComparisonOperation::IsComparisonOperationEqualTo(ETextFilterComparisonOperation TestOp) const
{
	return (ComparisonOp == TestOp);
}

FText FFrontendFilter_ArbitraryComparisonOperation::GetKeyValueAsText() const
{
	return FText::FromName(TagName);
}

FText FFrontendFilter_ArbitraryComparisonOperation::GetTargetValueAsText() const
{
	return FText::AsCultureInvariant(TargetTagValue);
}

void FFrontendFilter_ArbitraryComparisonOperation::OnKeyValueTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	if (!InText.IsEmpty())
	{
		TagName = *InText.ToString();
		BroadcastChangedEvent();
	}
}

void FFrontendFilter_ArbitraryComparisonOperation::OnTargetValueTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	TargetTagValue = InText.ToString();
	BroadcastChangedEvent();
}

FString FFrontendFilter_ArbitraryComparisonOperation::ConvertOperationToString(ETextFilterComparisonOperation Op)
{
	switch (Op)
	{
	case ETextFilterComparisonOperation::Equal:
		return TEXT("==");
	case ETextFilterComparisonOperation::NotEqual:
		return TEXT("!=");
	case ETextFilterComparisonOperation::Less:
		return TEXT("<");
	case ETextFilterComparisonOperation::LessOrEqual:
		return TEXT("<=");
	case ETextFilterComparisonOperation::Greater:
		return TEXT(">");
	case ETextFilterComparisonOperation::GreaterOrEqual:
		return TEXT(">=");
	default:
		check(false);
		return TEXT("op");
	};
}

#undef LOCTEXT_NAMESPACE

/////////////////////////////////////////
// FFrontendFilter_ShowOtherDevelopers
/////////////////////////////////////////

FFrontendFilter_ShowOtherDevelopers::FFrontendFilter_ShowOtherDevelopers(TSharedPtr<FFrontendFilterCategory> InCategory)
	: FFrontendFilter(MoveTemp(InCategory))
	, BaseDeveloperPath(TEXT("/Game/Developers/"))
	, BaseDeveloperPathAnsi()
	, UserDeveloperPath(BaseDeveloperPath + FPaths::GameUserDeveloperFolderName() + TEXT("/"))
	, bIsOnlyOneDeveloperPathSelected(false)
	, bShowOtherDeveloperAssets(false)
{
	TextFilterUtils::TryConvertWideToAnsi(BaseDeveloperPath, BaseDeveloperPathAnsi);
}

void FFrontendFilter_ShowOtherDevelopers::SetCurrentFilter(TArrayView<const FName> InSourcePaths, const FContentBrowserDataFilter& InBaseFilter)
{
	if ( InSourcePaths.Num() == 1 )
	{
		FString PackagePath;
		IContentBrowserDataModule::Get().GetSubsystem()->TryConvertVirtualPath(InSourcePaths[0].ToString(), PackagePath);
		PackagePath += TEXT("/");

		// If the path starts with the base developer path, and is not the path itself then only one developer path is selected
		bIsOnlyOneDeveloperPathSelected = PackagePath.StartsWith(BaseDeveloperPath) && PackagePath.Len() != BaseDeveloperPath.Len();
	}
	else
	{
		// More or less than one path is selected
		bIsOnlyOneDeveloperPathSelected = false;
	}
}

bool FFrontendFilter_ShowOtherDevelopers::PassesFilter(FAssetFilterType InItem) const
{
	// Pass all assets if other developer assets are allowed
	if ( bShowOtherDeveloperAssets )
	{
		return true;
	}
	// Never hide developer assets when a single developer folder is selected.
	if ( bIsOnlyOneDeveloperPathSelected )
	{
		return true;
	}

	// TODO: Have attribute flags for this so you can tell from the item whether it's a developer folder, and also whether it's yours
	// If selecting multiple folders, the Developers folder/parent folder, or "All Assets", hide assets which are found in the development folder unless they are in the current user's folder
	TStringBuilder<256> InternalPath{InPlace, InItem.GetInternalPath()};
	const bool bPackageInDeveloperFolder = InternalPath.ToView().StartsWith(BaseDeveloperPath, ESearchCase::IgnoreCase);
	const bool bPackageInUserDeveloperFolder = InternalPath.ToView().StartsWith(UserDeveloperPath, ESearchCase::IgnoreCase);
	if (bPackageInDeveloperFolder && !bPackageInUserDeveloperFolder)
	{
		// Test again using only the path part to avoid filtering files directly in the Developers folder
		// This happens after the above check to avoid string manipulation when not required
		FStringView ParentPath = FPathViews::GetPath(InternalPath.ToView());
		const bool bIsDirectlyInDevleoperFolder = ParentPath.Equals(FStringView(BaseDeveloperPath).LeftChop(1), ESearchCase::IgnoreCase);
		if(bIsDirectlyInDevleoperFolder )
		{
			return true;
		}
		return false;
	}

	return true;
}

void FFrontendFilter_ShowOtherDevelopers::SetShowOtherDeveloperAssets(bool bValue)
{
	if ( bShowOtherDeveloperAssets != bValue )
	{
		bShowOtherDeveloperAssets = bValue;
		BroadcastChangedEvent();
	}
}

bool FFrontendFilter_ShowOtherDevelopers::GetShowOtherDeveloperAssets() const
{
	return bShowOtherDeveloperAssets;
}

/////////////////////////////////////////
// FFrontendFilter_ShowRedirectors
/////////////////////////////////////////

FFrontendFilter_ShowRedirectors::FFrontendFilter_ShowRedirectors(TSharedPtr<FFrontendFilterCategory> InCategory)
	: FFrontendFilter(MoveTemp(InCategory))
{
	bAreRedirectorsInBaseFilter = false;
	RedirectorClassName = UObjectRedirector::StaticClass()->GetPathName();
}

void FFrontendFilter_ShowRedirectors::SetCurrentFilter(TArrayView<const FName> InSourcePaths, const FContentBrowserDataFilter& InBaseFilter)
{
	const FContentBrowserDataClassFilter* ClassFilter = InBaseFilter.ExtraFilters.FindFilter<FContentBrowserDataClassFilter>();
	bAreRedirectorsInBaseFilter = ClassFilter && ClassFilter->ClassNamesToInclude.Contains(RedirectorClassName);
}

bool FFrontendFilter_ShowRedirectors::PassesFilter(FAssetFilterType InItem) const
{
	// Never hide redirectors if they are explicitly searched for
	if ( !bAreRedirectorsInBaseFilter )
	{
		const FContentBrowserItemDataAttributeValue ClassValue = InItem.GetItemAttribute(NAME_Class);
		return !ClassValue.IsValid() || ClassValue.GetValue<FString>() != RedirectorClassName;
	}

	return true;
}

/////////////////////////////////////////
// FFrontendFilter_InUseByLoadedLevels
/////////////////////////////////////////

FFrontendFilter_InUseByLoadedLevels::FFrontendFilter_InUseByLoadedLevels(TSharedPtr<FFrontendFilterCategory> InCategory) 
	: FFrontendFilter(MoveTemp(InCategory))
{
	FEditorDelegates::MapChange.AddRaw(this, &FFrontendFilter_InUseByLoadedLevels::OnEditorMapChange);

	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	AssetTools.OnAssetPostRename().AddRaw(this, &FFrontendFilter_InUseByLoadedLevels::OnAssetPostRename);

	FAssetCompilingManager::Get().OnAssetPostCompileEvent().AddRaw(this, &FFrontendFilter_InUseByLoadedLevels::OnAssetPostCompile);
}

FFrontendFilter_InUseByLoadedLevels::~FFrontendFilter_InUseByLoadedLevels()
{
	FEditorDelegates::MapChange.RemoveAll(this);

	if(FAssetToolsModule::IsModuleLoaded())
	{
		IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
		AssetTools.OnAssetPostRename().RemoveAll(this);
	}

	FAssetCompilingManager::Get().OnAssetPostCompileEvent().RemoveAll(this);

	UnregisterDelayedRefresh();
}

void FFrontendFilter_InUseByLoadedLevels::ActiveStateChanged( bool bActive )
{
	bIsCurrentlyActive = bActive;

	if ( bActive )
	{
		ObjectTools::TagInUseObjects(ObjectTools::SO_LoadedLevels, ObjectTools::EInUseSearchFlags::SkipCompilingAssets);
		bIsDirty = false;
	}
}

void FFrontendFilter_InUseByLoadedLevels::RegisterDelayedRefresh(float DelayInSeconds)
{
	UnregisterDelayedRefresh();

	// The Editor might be unresponsive during heavy asset compilation so we 
	// not only need a delay, but also a minimum amount of frames
	// to pass until we call the actual refresh.
	DelayedRefreshHandle = FTSTicker::GetCoreTicker().AddTicker(
		TEXT("FFrontendFilter_InUseByLoadedLevels"),
		0.0f,
		[this, FireInTickCount = 16, DelayInSeconds](float DeltaTime) mutable
		{
			DelayInSeconds -= DeltaTime;
			if (--FireInTickCount == 0 && DelayInSeconds <= 0.0f && FAssetCompilingManager::Get().GetNumRemainingAssets() == 0)
			{
				Refresh();
				return false;
			}

			return true;
		}
	);
}

void FFrontendFilter_InUseByLoadedLevels::UnregisterDelayedRefresh()
{
	if (DelayedRefreshHandle.IsValid())
	{
		FTSTicker::RemoveTicker(DelayedRefreshHandle);
		DelayedRefreshHandle.Reset();
	}
}

void FFrontendFilter_InUseByLoadedLevels::Refresh()
{
	if (bIsCurrentlyActive)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FFrontendFilter_InUseByLoadedLevels::Refresh);

		// Update the tags identifying objects currently used by loaded levels
		ObjectTools::TagInUseObjects(ObjectTools::SO_LoadedLevels, ObjectTools::EInUseSearchFlags::SkipCompilingAssets);
		bIsDirty = false;
		BroadcastChangedEvent();
	}
}

void FFrontendFilter_InUseByLoadedLevels::OnAssetPostCompile(const TArray<FAssetCompileData>& CompiledAssets)
{
	if (bIsCurrentlyActive && !bIsDirty)
	{
		for (const FAssetCompileData& CompileData : CompiledAssets)
		{
			if (CompileData.Asset.IsValid())
			{
				bIsDirty = true;
				break;
			}
		}
	}

	// TagInUseObjects is really slow, only trigger a filter refresh when all assets are finished compiling.
	if (bIsDirty && FAssetCompilingManager::Get().GetNumRemainingAssets() == 0)
	{
		// Wait until we get some idle time to avoid refreshing too aggressively 
		RegisterDelayedRefresh(2.0f);
	}
	else
	{
		// We're not idle anymore, unregister until we get to 0 assets again
		UnregisterDelayedRefresh();
	}
}

void FFrontendFilter_InUseByLoadedLevels::OnAssetPostRename(const TArray<FAssetRenameData>& AssetsAndNames)
{
	if (bIsCurrentlyActive)
	{
		// Update the tags identifying objects currently used by loaded levels
		Refresh();
	}
}

bool FFrontendFilter_InUseByLoadedLevels::PassesFilter(FAssetFilterType InItem) const
{
	bool bObjectInUse = false;

	FAssetData ItemAssetData;
	if (InItem.Legacy_TryGetAssetData(ItemAssetData))
	{
		FString WorldAssetName;
		if (GWorld && ItemAssetData.GetTagValue(FrontendFilterHelper::NAME_BookmarkWorld, WorldAssetName))
		{
			FSoftObjectPath WorldAsset = FSoftObjectPath(WorldAssetName);
			UAssetRegistryHelpers::FixupRedirectedAssetPath(WorldAsset);
			FSoftObjectPath CurrentWorld = FSoftObjectPath(GWorld);
			return (CurrentWorld == WorldAsset);
		}

		if (UObject* Asset = ItemAssetData.FastGetAsset(false))
		{
			const bool bUnreferenced = !Asset->HasAnyMarks(OBJECTMARK_TagExp);
			const bool bIndirectlyReferencedObject = Asset->HasAnyMarks(OBJECTMARK_TagImp);
			const bool bRejectObject =
				Asset->GetOuter() == NULL || // Skip objects with null outers
				Asset->HasAnyFlags(RF_Transient) || // Skip transient objects (these shouldn't show up in the CB anyway)
				!IsValid(Asset) || // Objects that will be garbage collected 
				bUnreferenced || // Unreferenced objects 
				bIndirectlyReferencedObject; // Indirectly referenced objects

			if (!bRejectObject && Asset->HasAnyFlags(RF_Public))
			{
				// The object is in use 
				bObjectInUse = true;
			}
		}
	}

	return bObjectInUse;
}

void FFrontendFilter_InUseByLoadedLevels::OnEditorMapChange( uint32 MapChangeFlags )
{
	if ( MapChangeFlags == MapChangeEventFlags::NewMap && bIsCurrentlyActive )
	{
		Refresh();
	}
}

/////////////////////////////////////////
// FFrontendFilter_InUseByAnyLevel
/////////////////////////////////////////

FFrontendFilter_UsedInAnyLevel::FFrontendFilter_UsedInAnyLevel(TSharedPtr<FFrontendFilterCategory> InCategory)
	: FFrontendFilter(MoveTemp(InCategory))
{
	// Prepare asset registry.
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistry = &AssetRegistryModule.Get();
	check (AssetRegistry != nullptr);
}

void FFrontendFilter_UsedInAnyLevel::ActiveStateChanged(bool bActive)
{
	LevelsDependencies.Empty();

	if (bActive)
	{
		// Find all the levels & external actors
		FARFilter Filter;
		Filter.ClassPaths.Add(UWorld::StaticClass()->GetClassPathName());
		Filter.ClassPaths.Add(AActor::StaticClass()->GetClassPathName());
		Filter.bRecursiveClasses = true;
		FrontendFilterHelper::GetDependencies(Filter, *AssetRegistry, LevelsDependencies);
	}
}

bool FFrontendFilter_UsedInAnyLevel::PassesFilter(FAssetFilterType InItem) const	
{
	FAssetData ItemAssetData;
	if (InItem.Legacy_TryGetAssetData(ItemAssetData))
	{
		FAssetData WorldInBookmark;
		if (FrontendFilterHelper::GetWorldBookmark(AssetRegistry, ItemAssetData, WorldInBookmark))
		{
			return WorldInBookmark.IsValid();
		}

		return LevelsDependencies.Contains(ItemAssetData.PackageName);
	}
	return false;
}

/////////////////////////////////////////
// FFrontendFilter_NotUsedInAnyLevel
/////////////////////////////////////////

FFrontendFilter_NotUsedInAnyLevel::FFrontendFilter_NotUsedInAnyLevel(TSharedPtr<FFrontendFilterCategory> InCategory)
	: FFrontendFilter(MoveTemp(InCategory))
{
	// Prepare asset registry.
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistry = &AssetRegistryModule.Get();
	check (AssetRegistry != nullptr);
}

void FFrontendFilter_NotUsedInAnyLevel::ActiveStateChanged(bool bActive)
{
	LevelsDependencies.Empty();
	
	if (bActive)
	{
		// Find all the levels & external actors
		FARFilter Filter;
		Filter.ClassPaths.Add(UWorld::StaticClass()->GetClassPathName());
		Filter.ClassPaths.Add(AActor::StaticClass()->GetClassPathName());
		Filter.bRecursiveClasses = true;
		FrontendFilterHelper::GetDependencies(Filter, *AssetRegistry, LevelsDependencies);
	}
}

bool FFrontendFilter_NotUsedInAnyLevel::PassesFilter(FAssetFilterType InItem) const
{
	FAssetData ItemAssetData;
	if (InItem.Legacy_TryGetAssetData(ItemAssetData))
	{
		FAssetData WorldInBookmark;
		if (FrontendFilterHelper::GetWorldBookmark(AssetRegistry, ItemAssetData, WorldInBookmark))
		{
			return !WorldInBookmark.IsValid();
		}

		return !LevelsDependencies.Contains(ItemAssetData.PackageName);
	}
	return false;
}

/////////////////////////////////////////
// FFrontendFilter_NotUsedInAnyAsset
/////////////////////////////////////////

FFrontendFilter_NotUsedInAnyAsset::FFrontendFilter_NotUsedInAnyAsset(TSharedPtr<FFrontendFilterCategory> InCategory)
	: FFrontendFilter(MoveTemp(InCategory))
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistry = &AssetRegistryModule.Get();
}

bool FFrontendFilter_NotUsedInAnyAsset::PassesFilter(FAssetFilterType InItem) const
{
	FAssetData ItemAssetData;
	if (InItem.Legacy_TryGetAssetData(ItemAssetData))
	{
		TArray<FName> OutReferencers;
		AssetRegistry->GetReferencers(ItemAssetData.PackageName, OutReferencers);
		return OutReferencers.IsEmpty();
	}
	return false;
}

/////////////////////////////////////////
// FFrontendFilter_Recent
/////////////////////////////////////////

FFrontendFilter_Recent::FFrontendFilter_Recent(TSharedPtr<FFrontendFilterCategory> InCategory)
	: FFrontendFilter(MoveTemp(InCategory))
	, bIsCurrentlyActive(false)
{
	UContentBrowserSettings::OnSettingChanged().AddRaw(this, &FFrontendFilter_Recent::ResetFilter);
}

FFrontendFilter_Recent::~FFrontendFilter_Recent()
{
	UContentBrowserSettings::OnSettingChanged().RemoveAll(this);
}

void FFrontendFilter_Recent::ActiveStateChanged(bool bActive)
{
	bIsCurrentlyActive = bActive;
}

bool FFrontendFilter_Recent::PassesFilter(FAssetFilterType InItem) const
{
	FAssetData ItemAssetData;
	if (InItem.Legacy_TryGetAssetData(ItemAssetData))
	{
		return RecentPackagePaths.Contains(ItemAssetData.PackageName);
	}
	return false;
}

void FFrontendFilter_Recent::SetCurrentFilter(TArrayView<const FName> InSourcePaths, const FContentBrowserDataFilter& InBaseFilter)
{
	RefreshRecentPackagePaths();
}

void FFrontendFilter_Recent::RefreshRecentPackagePaths()
{
	static const FName ContentBrowserName(TEXT("ContentBrowser"));

	RecentPackagePaths.Reset();
	FContentBrowserModule& CBModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(ContentBrowserName);
	FMainMRUFavoritesList* RecentlyOpenedAssets = CBModule.GetRecentlyOpenedAssets();
	if (RecentlyOpenedAssets)
	{
		RecentPackagePaths.Reserve(RecentlyOpenedAssets->GetNumItems());
		for (int32 i = 0; i < RecentlyOpenedAssets->GetNumItems(); ++i)
		{
			RecentPackagePaths.Add(FName(*RecentlyOpenedAssets->GetMRUItem(i)));
		}
	}
}

void FFrontendFilter_Recent::ResetFilter(FName InName)
{
	if (InName == FContentBrowserModule::NumberOfRecentAssetsName)
	{
		BroadcastChangedEvent();
	}
}

/////////////////////////////////////////
// FFrontendFilter_Writable
/////////////////////////////////////////

FFrontendFilter_Writable::FFrontendFilter_Writable(TSharedPtr<FFrontendFilterCategory> InCategory)
	: FFrontendFilter(MoveTemp(InCategory))
{
}

FFrontendFilter_Writable::~FFrontendFilter_Writable()
{

}

bool FFrontendFilter_Writable::PassesFilter(FAssetFilterType InItem) const
{
	FString ItemDiskPath;
	if (!InItem.GetItemPhysicalPath(ItemDiskPath))
	{
		return false;
	}

	ItemDiskPath = FPaths::ConvertRelativePathToFull(MoveTemp(ItemDiskPath));

	return !IFileManager::Get().IsReadOnly(*ItemDiskPath);
}

/////////////////////////////////////////
// FFrontendFilter_VirtualizedData
/////////////////////////////////////////

FFrontendFilter_VirtualizedData::FFrontendFilter_VirtualizedData(TSharedPtr<FFrontendFilterCategory> InCategory)
	: FFrontendFilter(MoveTemp(InCategory))
{
}

bool FFrontendFilter_VirtualizedData::PassesFilter(FAssetFilterType InItem) const
{
	const FContentBrowserItemDataAttributeValue AttributeValue = InItem.GetItemAttribute(ContentBrowserItemAttributes::VirtualizedData);
	if (AttributeValue.IsValid())
	{
		return AttributeValue.GetValue<FString>() == TEXT("True");
	}
	else
	{
		return false;
	}
}

FFrontendFilter_Unsupported::FFrontendFilter_Unsupported(TSharedPtr<FFrontendFilterCategory> InCategory)
	:FFrontendFilter (InCategory)
{
}

bool FFrontendFilter_Unsupported::PassesFilter(FAssetFilterType InItem) const
{
	return !InItem.IsSupported();
}

/////////////////////////////////////////
// FFrontendFilter_Favorites
/////////////////////////////////////////

FFrontendFilter_Favorites::FFrontendFilter_Favorites(TSharedPtr<FFrontendFilterCategory> InCategory)
	:FFrontendFilter(InCategory)
{
	using namespace UE::Editor::DataStorage;
	DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
}

bool FFrontendFilter_Favorites::PassesFilter(FAssetFilterType InItem) const
{
	using namespace UE::Editor::DataStorage;

	if (!DataStorage)
	{
		return false;
	}

	FMapKey Path;

	if (InItem.GetItemType() == EContentBrowserItemFlags::Type_File)
	{
		if (FAssetData AssetData; InItem.Legacy_TryGetAssetData(AssetData))
		{
			Path = FMapKey(AssetData.GetSoftObjectPath());
		}
	}

	if (!Path.IsSet())
	{
		return false;
	}

	const RowHandle Row = DataStorage->LookupMappedRow("Asset", Path);

	if (DataStorage->IsRowAvailable(Row))
	{
		return DataStorage->HasColumns<FFavoriteTag>(Row);
	}

	return false;
}
