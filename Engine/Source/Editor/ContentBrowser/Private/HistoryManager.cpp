// Copyright Epic Games, Inc. All Rights Reserved.

#include "HistoryManager.h"

#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Textures/SlateIcon.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

FHistoryManager::FHistoryManager()
{
	CurrentHistoryIndex = 0;
	MaxHistoryEntries = 300;
}

void FHistoryManager::SetOnApplyHistoryData(const FOnApplyHistoryData& InOnApplyHistoryData)
{
	OnApplyHistoryData = InOnApplyHistoryData;
}

void FHistoryManager::SetOnUpdateHistoryData(const FOnUpdateHistoryData& InOnUpdateHistoryData)
{
	OnUpdateHistoryData = InOnUpdateHistoryData;
}

bool FHistoryManager::GoBack()
{
	if ( CanGoBack() )
	{
		// Update the current history data
		UpdateCurrentHistoryData();

		// if its possible to go back, decrement the index we are at
		--CurrentHistoryIndex;

		// Update the owner
		ApplyCurrentHistoryData();

		return true;
	}

	return false;
}

bool FHistoryManager::GoForward()
{
	if ( CanGoForward() )
	{
		// Update the current history data
		UpdateCurrentHistoryData();

		// if its possible to go forward, increment the index we are at
		++CurrentHistoryIndex;

		// Update the owner
		ApplyCurrentHistoryData();

		return true;
	}

	return false;
}

void FHistoryManager::AddHistoryData()
{
	if (HistoryData.Num() == 0)
	{
		// History added to the beginning
		HistoryData.Add(FHistoryData());
		CurrentHistoryIndex = 0;
	}
	else if (CurrentHistoryIndex == HistoryData.Num() - 1)
	{
		// History added to the end
		if (HistoryData.Num() == MaxHistoryEntries)
		{
			// If max history entries has been reached
			// remove the oldest history
			HistoryData.RemoveAt(0);
		}
		HistoryData.Add(FHistoryData());
		// Current history index is the last index in the list
		CurrentHistoryIndex = HistoryData.Num() - 1;
	}
	else
	{
		// History added to the middle
		// clear out all history after the current history index.
		HistoryData.RemoveAt(CurrentHistoryIndex + 1, HistoryData.Num() - (CurrentHistoryIndex + 1));
		HistoryData.Add(FHistoryData());
		// Current history index is the last index in the list
		CurrentHistoryIndex = HistoryData.Num() - 1;
	}

	// Update the current history data
	UpdateCurrentHistoryData();
}

void FHistoryManager::UpdateHistoryData()
{
	// Update the current history data
	UpdateCurrentHistoryData();
}

bool FHistoryManager::CanGoForward() const
{
	// User can go forward if there are items in the history data list, 
	// and the current history index isn't the last index in the list
	return HistoryData.Num() > 0 && CurrentHistoryIndex < HistoryData.Num() - 1;
}

bool FHistoryManager::CanGoBack() const
{
	// User can go back if there are items in the history data list,
	// and the current history index isn't the first index in the list
	return HistoryData.Num() > 0 && CurrentHistoryIndex > 0;
}

FText FHistoryManager::GetBackDesc() const
{
	if ( CanGoBack() )
	{
		return HistoryData[CurrentHistoryIndex - 1].HistoryDesc;
	}
	return FText::GetEmpty();
}

FText FHistoryManager::GetForwardDesc() const
{
	if ( CanGoForward() )
	{
		return HistoryData[CurrentHistoryIndex + 1].HistoryDesc;
	}
	return FText::GetEmpty();
}

void FHistoryManager::GetAvailableHistoryMenuItems(bool bGetPrior, FMenuBuilder& MenuBuilder)
{
	const FText HistoryHeadingString = (bGetPrior)? LOCTEXT("BackHistory", "Back History") : LOCTEXT("NextHistory", "Next History");
	MenuBuilder.BeginSection("HistoryBackNext", HistoryHeadingString);
	{
		if (HistoryData.Num() > 1)
		{
			// if there is at least 2 history items...

			// Start index is the first snapshot we should make a menu item out of
			int32 StartIndex = 0;
			// EndIndex is the last snapshot we should make a menu item out of
			int32 EndIndex = CurrentHistoryIndex;

			if (!bGetPrior)
			{
				// Need to return only items on or after the current history index
				StartIndex = CurrentHistoryIndex;
				EndIndex = HistoryData.Num() - 1;
			}

			// Check to make sure the start and end indices are within the bounds of the history list
			if (StartIndex < HistoryData.Num() && EndIndex != -1)
			{
				// Get all menu items between and including the start index and end index
				for (int32 HistoryIdx = StartIndex; HistoryIdx <= EndIndex; ++HistoryIdx)
				{
					MenuBuilder.AddMenuEntry(
						HistoryData[HistoryIdx].HistoryDesc,
						FText(),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateRaw( this, &FHistoryManager::ExecuteJumpToHistory, HistoryIdx )
							)
						);
				}
			}
		}
	}
	MenuBuilder.EndSection();
}

void FHistoryManager::ApplyCurrentHistoryData()
{
	if ( CurrentHistoryIndex >= 0 && CurrentHistoryIndex < HistoryData.Num())
	{
		OnApplyHistoryData.ExecuteIfBound( HistoryData[CurrentHistoryIndex] );
	}
}

void FHistoryManager::UpdateCurrentHistoryData()
{
	if ( CurrentHistoryIndex >= 0 && CurrentHistoryIndex < HistoryData.Num())
	{
		OnUpdateHistoryData.ExecuteIfBound( HistoryData[CurrentHistoryIndex] );
	}
}

void FHistoryManager::ExecuteJumpToHistory(int32 HistoryIndex)
{
	if (HistoryIndex >= 0 && HistoryIndex < HistoryData.Num())
	{
		// if the history index is valid, set the current history index to the history index requested by the user
		CurrentHistoryIndex = HistoryIndex;

		// Update the owner
		ApplyCurrentHistoryData();
	}
}

#undef LOCTEXT_NAMESPACE
