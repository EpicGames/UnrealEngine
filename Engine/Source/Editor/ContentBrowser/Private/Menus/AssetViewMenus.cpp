// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserCommands.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserMenuUtils.h"
#include "ContentBrowserStyle.h"
#include "Misc/DelayedAutoRegister.h"
#include "SAssetView.h"
#include "SContentBrowser.h"
#include "SDocumentationToolTip.h"
#include "SFilterList.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "UObject/NameTypes.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

namespace UE::Editor::ContentBrowser
{
	namespace Private
	{
		TSharedRef<SWidget> GetExtendedToolTipMouseWheel()
		{
			constexpr float ExtendedToolTipPadding = 8.f;
			return SNew(SBox)
					.Padding(ExtendedToolTipPadding)
					[
						SNew(SRichTextBlock)
						.DecoratorStyleSet(&FAppStyle::Get())
						.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("RichTextBlock.DarkText"))
						.Text(LOCTEXT("CustomSizeToolTip",
							"Hold <RichTextBlock.BoldDarkText>Ctrl</> and use the <RichTextBlock.BoldDarkText>Scroll Wheel</> to scale the thumbnails."
							"\nHold <RichTextBlock.BoldDarkText>Ctrl+Shift</> and use the <RichTextBlock.BoldDarkText>Scroll Wheel</> to cycle through predefined thumbnail sizes."))
					];
		}

		TSharedRef<IToolTip> GetThumbnailSizeToolTip()
		{
			return SNew(SToolTip)
					.BorderImage(FAppStyle::GetBrush("ToolTip.BrightBackground"))
					.TextMargin(FMargin(1.f, 0.f))
					.Content()
					[
						SNew(SDocumentationToolTip)
						.OverrideExtendedToolTipContent(GetExtendedToolTipMouseWheel())
						[
							SNew(STextBlock)
							.ColorAndOpacity(FLinearColor::Black)
							.Text(LOCTEXT("ThumbnailSizeToolTip", "Adjust the size of thumbnails."))
						]
					];
		}

		TSharedRef<SWidget> GetThumbnailSizeWidget()
		{
			return SNew(STextBlock)
				.Text(LOCTEXT("ThumbnailSize", "Thumbnail Size"))
				.ToolTip(GetThumbnailSizeToolTip());
		}
	}
}

FDelayedAutoRegisterHelper SAssetView::AssetViewOptionsMenuRegistration(
	EDelayedRegisterRunPhase::EndOfEngineInit,
	[]
	{
		using namespace UE::Editor::ContentBrowser;

		FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

		static const FName AssetViewOptionsMenuName("ContentBrowser.AssetViewOptions");
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(AssetViewOptionsMenuName);
		Menu->bCloseSelfOnly = true;

		Menu->AddDynamicSection(NAME_Default, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			FName OwningContentBrowserName = NAME_None;
			FFiltersAdditionalParams FilterParams;
			if (UContentBrowserAssetViewContextMenuContext* Context = InMenu->FindContext<UContentBrowserAssetViewContextMenuContext>())
			{
				if (Context->AssetView.IsValid())
				{
					auto SetEntryOrder = [](FToolMenuEntry& InEntry)
					{
						// New Style, >5.6: (entry name, entry to place after) map. If the named entry isn't found, the entry will not have an explicit order.
						static const TMap<FName, FName> NewEntryOrder = {
							{ "ShowFolders", "ShowFavorite" },
							{ "ShowEmptyFolders", "ShowFolders" },
							{ "OrganizeFolders", "ShowEmptyFolders" }
						};

						// <5.6 Style: (entry name, entry to place after) map. If the named entry isn't found, the entry will not have an explicit order.
						static const TMap<FName, FName> EntryOrder = {
							{ "ShowAllFolder", "FilterRecursively" }
						};

						if (const FName* InsertAfterEntry = IsNewStyleEnabled() ? NewEntryOrder.Find(InEntry.Name) : EntryOrder.Find(InEntry.Name))
						{
							InEntry.InsertPosition = FToolMenuInsert(*InsertAfterEntry, EToolMenuInsertType::After);
						}
					};

					const TSharedRef<SAssetView> AssetView = Context->AssetView.Pin().ToSharedRef();

					// ViewType Section
					{
						FToolMenuSection& ViewTypeSection = InMenu->FindOrAddSection(
							"AssetViewType",
							LOCTEXT("ViewTypeHeading", "View Type"));

						// Use commands in the new CB
						if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
						{
							ViewTypeSection.AddMenuEntryWithCommandList(FContentBrowserCommands::Get().GridViewShortcut, AssetView->Commands);
							ViewTypeSection.AddMenuEntryWithCommandList(FContentBrowserCommands::Get().ListViewShortcut, AssetView->Commands);
							ViewTypeSection.AddMenuEntryWithCommandList(FContentBrowserCommands::Get().ColumnViewShortcut, AssetView->Commands);
						}
						else
						{
							ViewTypeSection.AddMenuEntry(
								"TileView",
								LOCTEXT("TileViewOption", "Grid"),
								LOCTEXT("TileViewOptionToolTip", "View assets as tiles in a grid."),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(AssetView, &SAssetView::SetCurrentViewTypeFromMenu, EAssetViewType::Tile),
									FCanExecuteAction(),
									FIsActionChecked::CreateSP(AssetView, &SAssetView::IsCurrentViewType, EAssetViewType::Tile)
								),
								EUserInterfaceActionType::RadioButton
							);

							ViewTypeSection.AddMenuEntry(
								"ListView",
								LOCTEXT("ListViewOption", "List"),
								LOCTEXT("ListViewOptionToolTip", "View assets in a list with thumbnails."),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(AssetView, &SAssetView::SetCurrentViewTypeFromMenu, EAssetViewType::List),
									FCanExecuteAction(),
									FIsActionChecked::CreateSP(AssetView, &SAssetView::IsCurrentViewType, EAssetViewType::List)
								),
								EUserInterfaceActionType::RadioButton
							);
						}

						if (!UE::Editor::ContentBrowser::IsNewStyleEnabled())
						{
							ViewTypeSection.AddMenuEntry(
								"ColumnView",
								LOCTEXT("ColumnViewOption", "Columns"),
								LOCTEXT("ColumnViewOptionToolTip", "View assets in a list with columns of details."),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(AssetView, &SAssetView::SetCurrentViewTypeFromMenu, EAssetViewType::Column),
									FCanExecuteAction(),
									FIsActionChecked::CreateSP(AssetView, &SAssetView::IsCurrentViewType, EAssetViewType::Column)
								),
								EUserInterfaceActionType::RadioButton
							);
						}

						const FText CustomViewLabel = AssetView->ViewExtender
							? AssetView->ViewExtender->GetViewDisplayName()
							: LOCTEXT("CustomViewOption", "Custom");

						const FText CustomViewTooltip = AssetView->ViewExtender
							? AssetView->ViewExtender->GetViewTooltipText()
							: LOCTEXT("CustomViewOptionToolTip", "A user specified custom view.");

						ViewTypeSection.AddMenuEntry(
							"CustomView",
							CustomViewLabel,
							CustomViewTooltip,
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(AssetView, &SAssetView::SetCurrentViewTypeFromMenu, EAssetViewType::Custom),
								FCanExecuteAction(),
								FIsActionChecked::CreateSP(AssetView, &SAssetView::IsCurrentViewType, EAssetViewType::Custom),
								FIsActionButtonVisible::CreateSP(AssetView, &SAssetView::IsCustomViewSet)
							),
							EUserInterfaceActionType::RadioButton
						);
					}

					if (!IsNewStyleEnabled())
					{
						// Filter Bar Section
						if (TSharedPtr<SFilterList> FilterBarPinned = AssetView->FilterBar.Pin())
						{
							FToolMenuSection& Section = InMenu->FindOrAddSection(
								"FilterBar",
								LOCTEXT("FilterBarHeading", "Filter Display"),
								FToolMenuInsert("AssetViewType", EToolMenuInsertType::After));;

							Section.AddMenuEntry(
								"VerticalLayout",
								LOCTEXT("FilterListVerticalLayout", "Vertical"),
								LOCTEXT("FilterListVerticalLayoutToolTip", "Swap to a vertical layout for the filter bar"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateLambda([FilterBarPinned]()
									{
										if (FilterBarPinned->GetFilterLayout() != EFilterBarLayout::Vertical)
										{
											FilterBarPinned->SetFilterLayout(EFilterBarLayout::Vertical);
										}
									}),
									FCanExecuteAction(),
									FIsActionChecked::CreateLambda([FilterBarPinned]()
									{
										return FilterBarPinned->GetFilterLayout() == EFilterBarLayout::Vertical;
									})),
								EUserInterfaceActionType::RadioButton
							);

							Section.AddMenuEntry(
								"HorizontalLayout",
								LOCTEXT("FilterListHorizontalLayout", "Horizontal"),
								LOCTEXT("FilterListHorizontalLayoutToolTip", "Swap to a Horizontal layout for the filter bar"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateLambda([FilterBarPinned]()
									{
										if (FilterBarPinned->GetFilterLayout() != EFilterBarLayout::Horizontal)
										{
											FilterBarPinned->SetFilterLayout(EFilterBarLayout::Horizontal);
										}
									}),
									FCanExecuteAction(),
									FIsActionChecked::CreateLambda([FilterBarPinned]()
									{
										return FilterBarPinned->GetFilterLayout() == EFilterBarLayout::Horizontal;
									})),
								EUserInterfaceActionType::RadioButton
							);
						}
					}

					// Thumbnails Section
					{
						const FToolMenuInsert ThumbnailSectionInsert("Search", EToolMenuInsertType::After);
						FToolMenuSection& ThumbnailSection = InMenu->FindOrAddSection(
							"AssetThumbnails",
							LOCTEXT("ThumbnailsHeading", "Thumbnails"),
							ThumbnailSectionInsert);

						auto CreateThumbnailSizeSubMenu = [AssetView = &AssetView.Get()](UToolMenu* SubMenu)
						{
							FToolMenuSection& SizeSection = SubMenu->FindOrAddSection("ThumbnailSizes");

							for (int32 EnumValue = static_cast<int32>(EThumbnailSize::Tiny); EnumValue < static_cast<int32>(EThumbnailSize::MAX); ++EnumValue)
							{
								if (!IsNewStyleEnabled())
								{
									if ((EThumbnailSize)EnumValue == EThumbnailSize::XLarge)
									{
										continue;
									}
								}

								SizeSection.AddMenuEntry(
									FName(FString::Printf(TEXT("ThumbnailSizeValue_%d"), EnumValue)),
									SAssetView::ThumbnailSizeToDisplayName((EThumbnailSize)EnumValue),
									FText::GetEmpty(),
									FSlateIcon(),
									FUIAction(
										FExecuteAction::CreateSP(AssetView, &SAssetView::OnThumbnailSizeChanged, (EThumbnailSize)EnumValue),
										FCanExecuteAction::CreateSP(AssetView, &SAssetView::IsThumbnailScalingAllowed),
										FIsActionChecked::CreateSP(AssetView, &SAssetView::IsThumbnailSizeChecked, (EThumbnailSize)EnumValue)
									),
									EUserInterfaceActionType::RadioButton
								);
							}
						};

						ThumbnailSection.AddEntry(FToolMenuEntry::InitSubMenu(
							"ThumbnailSize",
							FToolUIActionChoice(),
							Private::GetThumbnailSizeWidget(),
							FNewToolMenuDelegate::CreateSPLambda(AssetView, CreateThumbnailSizeSubMenu),
							false
						));

						ThumbnailSection.AddMenuEntry(
							"ThumbnailEditMode",
							LOCTEXT("ThumbnailEditModeOption", "Thumbnail Edit Mode"),
							LOCTEXT("ThumbnailEditModeOptionToolTip", "Toggle thumbnail editing mode. When in this mode you can rotate the camera on 3D thumbnails by dragging them."),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(AssetView, &SAssetView::ToggleThumbnailEditMode),
								FCanExecuteAction::CreateSP(AssetView, &SAssetView::IsThumbnailEditModeAllowed),
								FIsActionChecked::CreateSP(AssetView, &SAssetView::IsThumbnailEditMode)
							),
							EUserInterfaceActionType::ToggleButton
						);

						ThumbnailSection.AddMenuEntry(
							"RealTimeThumbnails",
							LOCTEXT("RealTimeThumbnailsOption", "Real-Time Thumbnails"),
							LOCTEXT("RealTimeThumbnailsOptionToolTip", "Renders the assets thumbnails in real-time"),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(AssetView, &SAssetView::ToggleRealTimeThumbnails),
								FCanExecuteAction::CreateSP(AssetView, &SAssetView::CanShowRealTimeThumbnails),
								FIsActionChecked::CreateSP(AssetView, &SAssetView::IsShowingRealTimeThumbnails)
							),
							EUserInterfaceActionType::ToggleButton
						);
					}

					// AssetDetails Section
					if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
					{
						const FToolMenuInsert AssetDetailsSectionInsert("Search", EToolMenuInsertType::After);
						FToolMenuSection& AssetDetailsSection = InMenu->FindOrAddSection(
							"AssetDetails",
							LOCTEXT("AssetDetailsName", "Asset Details"),
							AssetDetailsSectionInsert);

						AssetDetailsSection.AddMenuEntry(
							"ThumbnailTooltip",
							LOCTEXT("ThumbnailTooltipExpanded", "Always Expand Tooltips"),
							LOCTEXT("ThumbnailTooltipExpandedTooltip", "Toggle Asset and Folder tooltip expansion default state"),
							FSlateIcon(),
							FUIAction(
									FExecuteAction::CreateSP(AssetView, &SAssetView::ToggleTooltipExpandedState),
									FCanExecuteAction(),
									FIsActionChecked::CreateSP(AssetView, &SAssetView::IsTooltipExpandedByDefault)
								),
							EUserInterfaceActionType::ToggleButton);
					}

					// Show/View Section
					{
						FToolMenuSection& ShowSection = InMenu->FindOrAddSection(
							IsNewStyleEnabled() ? "Show" : "View",
							IsNewStyleEnabled() ? LOCTEXT("ShowHeading", "Show") : LOCTEXT("ViewHeading", "View"));

						const FText ShowFoldersLabel = IsNewStyleEnabled() ? LOCTEXT("ShowFoldersOption_NewStyle", "Folders") :LOCTEXT("ShowFoldersOption", "Show Folders");
						SetEntryOrder(
							ShowSection.AddMenuEntry(
								"ShowFolders",
								ShowFoldersLabel,
								LOCTEXT("ShowFoldersOptionToolTip", "Show folders in the view as well as assets?"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(AssetView, &SAssetView::ToggleShowFolders),
									FCanExecuteAction::CreateSP(AssetView, &SAssetView::IsToggleShowFoldersAllowed),
									FIsActionChecked::CreateSP(AssetView, &SAssetView::IsShowingFolders)
								),
								EUserInterfaceActionType::ToggleButton
							)
						);

						const FText ShowEmptyFoldersLabel = IsNewStyleEnabled() ? LOCTEXT("ShowEmptyFoldersOption_NewStyle", "Empty Folders") :LOCTEXT("ShowEmptyFoldersOption", "Show Empty Folders");
						SetEntryOrder(
							ShowSection.AddMenuEntry(
								"ShowEmptyFolders",
								ShowEmptyFoldersLabel,
								LOCTEXT("ShowEmptyFoldersOptionToolTip", "Show empty folders in the view as well as assets?"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(AssetView, &SAssetView::ToggleShowEmptyFolders),
									FCanExecuteAction::CreateSP(AssetView, &SAssetView::IsToggleShowEmptyFoldersAllowed),
									FIsActionChecked::CreateSP(AssetView, &SAssetView::IsShowingEmptyFolders)
								),
								EUserInterfaceActionType::ToggleButton
							)
						);

						const FText ShowFavoriteLabel = IsNewStyleEnabled() ?  LOCTEXT("ShowFavoriteOptions_NewStyle", "Favorites") : LOCTEXT("ShowFavoriteOptions", "Show Favorites");
						ShowSection.AddMenuEntry(
							"ShowFavorite",
							ShowFavoriteLabel,
							LOCTEXT("ShowFavoriteOptionToolTip", "Show the favorite folders in the view?"),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(AssetView, &SAssetView::ToggleShowFavorites),
								FCanExecuteAction::CreateSP(AssetView, &SAssetView::IsToggleShowFavoritesAllowed),
								FIsActionChecked::CreateSP(AssetView, &SAssetView::IsShowingFavorites)
							),
							EUserInterfaceActionType::ToggleButton
						);

						if (!IsNewStyleEnabled())
						{
							ShowSection.AddMenuEntry(
								"FilterRecursively",
								LOCTEXT("FilterRecursivelyOption", "Filter Recursively"),
								LOCTEXT("FilterRecursivelyOptionToolTip", "Should filters apply recursively in the view?"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(AssetView, &SAssetView::ToggleFilteringRecursively),
									FCanExecuteAction::CreateSP(AssetView, &SAssetView::IsToggleFilteringRecursivelyAllowed),
									FIsActionChecked::CreateSP(AssetView, &SAssetView::IsFilteringRecursively)
								),
								EUserInterfaceActionType::ToggleButton
							);
						}

						SetEntryOrder(
							ShowSection.AddMenuEntry(
								"OrganizeFolders",
								LOCTEXT("OrganizeFoldersOption", "Organize Folders"),
								LOCTEXT("OrganizeFoldersOptionToolTip", "Organize folders in the view?"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(AssetView, &SAssetView::ToggleOrganizeFolders),
									FCanExecuteAction(),
									FIsActionChecked::CreateSP(AssetView, &SAssetView::IsOrganizingFolders)
								),
								EUserInterfaceActionType::ToggleButton
							)
						);

						if (AssetView->bShowPathViewFilters)
						{
							ShowSection.AddSubMenu(
								"PathViewFilters",
								LOCTEXT("PathViewFilters", "Path View Filters"),
								LOCTEXT("PathViewFilters_ToolTip", "Path View Filters"),
								FNewToolMenuDelegate());
						}
					}

					// Content Section
					{
						FToolMenuSection& ContentSection = InMenu->FindOrAddSection(
							"Content",
							IsNewStyleEnabled() ? FText::GetEmpty() : LOCTEXT("ContentHeading", "Content"),
							FToolMenuInsert(IsNewStyleEnabled() ? "Show" : "View", EToolMenuInsertType::After));

						if (IsNewStyleEnabled())
						{
							ContentSection.AddSeparator("ContentSeparator");
						}

						const FText ShowAllFolderLabel = IsNewStyleEnabled() ?  LOCTEXT("ShowAllFolderOption_NewStyle", "All Folder") : LOCTEXT("ShowAllFolderOption", "Show All Folder");
						FToolMenuSection& ShowAllFolderSection = IsNewStyleEnabled() ? ContentSection : *InMenu->FindSection("View");
						SetEntryOrder(
							ShowAllFolderSection.AddMenuEntry(
								"ShowAllFolder",
								ShowAllFolderLabel,
								LOCTEXT("ShowAllFolderOptionToolTip", "Show the all folder in the view?"),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateSP(AssetView, &SAssetView::ToggleShowAllFolder),
									FCanExecuteAction(),
									FIsActionChecked::CreateSP(AssetView, &SAssetView::IsShowingAllFolder)
								),
								EUserInterfaceActionType::ToggleButton
							)
						);
					}

					// Search Section
					{
						FToolMenuSection& SearchSection = InMenu->FindOrAddSection(
							"Search",
							LOCTEXT("SearchHeading", "Search"),
							FToolMenuInsert("Content", EToolMenuInsertType::After));

						SearchSection.AddMenuEntry(
							"IncludeClassName",
							LOCTEXT("IncludeClassNameOption", "Search Asset Class Names"),
							LOCTEXT("IncludeClassesNameOptionTooltip", "Include asset type names in search criteria?  (e.g. Blueprint, Texture, Sound)"),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(AssetView, &SAssetView::ToggleIncludeClassNames),
								FCanExecuteAction::CreateSP(AssetView, &SAssetView::IsToggleIncludeClassNamesAllowed),
								FIsActionChecked::CreateSP(AssetView, &SAssetView::IsIncludingClassNames)
							),
							EUserInterfaceActionType::ToggleButton
						);

						SearchSection.AddMenuEntry(
							"IncludeAssetPath",
							LOCTEXT("IncludeAssetPathOption", "Search Asset Path"),
							LOCTEXT("IncludeAssetPathOptionTooltip", "Include entire asset path in search criteria?"),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(AssetView, &SAssetView::ToggleIncludeAssetPaths),
								FCanExecuteAction::CreateSP(AssetView, &SAssetView::IsToggleIncludeAssetPathsAllowed),
								FIsActionChecked::CreateSP(AssetView, &SAssetView::IsIncludingAssetPaths)
							),
							EUserInterfaceActionType::ToggleButton
						);

						SearchSection.AddMenuEntry(
							"IncludeCollectionName",
							LOCTEXT("IncludeCollectionNameOption", "Search Collection Names"),
							LOCTEXT("IncludeCollectionNameOptionTooltip", "Include Collection names in search criteria?"),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(AssetView, &SAssetView::ToggleIncludeCollectionNames),
								FCanExecuteAction::CreateSP(AssetView, &SAssetView::IsToggleIncludeCollectionNamesAllowed),
								FIsActionChecked::CreateSP(AssetView, &SAssetView::IsIncludingCollectionNames)
							),
							EUserInterfaceActionType::ToggleButton
						);
					}

					if (IsNewStyleEnabled())
					{
						// Manage Section
						{
							// @note: This section is used elsewhere - @see: SContentBrowser::ExtendViewOptionsMenu
							FToolMenuSection& Section = InMenu->FindOrAddSection("Manage", LOCTEXT("ManageHeading", "Manage"), FToolMenuInsert("Search", EToolMenuInsertType::After));
						}
					}

					// Asset Columns/List Section
					if (AssetView->GetColumnViewVisibility() == EVisibility::Visible || (IsNewStyleEnabled() && AssetView->GetListViewVisibility() == EVisibility::Visible))
					{
						{
							FToolMenuSection& Section = InMenu->FindOrAddSection("AssetColumns", LOCTEXT("ToggleColumnsHeading", "Columns"), FToolMenuInsert("Search", EToolMenuInsertType::After));

							Section.AddMenuEntry(
								"ResetColumns",
								LOCTEXT("ResetColumns", "Reset Columns"),
								LOCTEXT("ResetColumnsToolTip", "Reset all columns to be visible again."),
								FSlateIcon(),
								FUIAction(FExecuteAction::CreateSP(AssetView, &SAssetView::ResetColumns)),
								EUserInterfaceActionType::Button
							);
						}
					}

					// Asset Columns Section
					if (AssetView->GetColumnViewVisibility() == EVisibility::Visible)
					{
						{
							FToolMenuSection& Section = InMenu->FindOrAddSection("AssetColumns", LOCTEXT("ToggleColumnsHeading", "Columns"), FToolMenuInsert("Search", EToolMenuInsertType::After));

							Section.AddMenuEntry(
								"ExportColumns",
								LOCTEXT("ExportColumns", "Export to CSV"),
								LOCTEXT("ExportColumnsToolTip", "Export column data to CSV."),
								FSlateIcon(),
								FUIAction(FExecuteAction::CreateSP(AssetView, &SAssetView::ExportColumns)),
								EUserInterfaceActionType::Button
							);
						}
					}

					AssetView->PopulateFilterAdditionalParams(FilterParams);
				}

				if (Context->OwningContentBrowser.IsValid())
				{
					OwningContentBrowserName = Context->OwningContentBrowser.Pin()->GetInstanceName();
				}
			}

			ContentBrowserMenuUtils::AddFiltersToMenu(InMenu, OwningContentBrowserName, FilterParams);
		}));
	});

FDelayedAutoRegisterHelper SAssetView::ToolBarMenuExtensionRegistration(
	EDelayedRegisterRunPhase::EndOfEngineInit,
	[]
	{
		using namespace UE::Editor::ContentBrowser;

		// @todo: Remove when old style is removed, all extensions are for the new style so early-out if it's not enabled
		if (!IsNewStyleEnabled())
		{
			return;
		}

		static const FName ToolBarName("ContentBrowser.ToolBar");
		if (UToolMenu* ToolBarMenu = UToolMenus::Get()->ExtendMenu(ToolBarName))
		{
			FToolMenuSection& SortSection = ToolBarMenu->FindOrAddSection("Sort");
			SortSection.InsertPosition = FToolMenuInsert("Search", EToolMenuInsertType::After);
			SortSection.ResizeParams.Wrapping.Allow = true;
			SortSection.ResizeParams.Wrapping.Priority = 10;

			SortSection.AddDynamicEntry(
				"Sort",
				FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					if (const UContentBrowserToolbarMenuContext* Context = InSection.FindContext<UContentBrowserToolbarMenuContext>())
					{
						if (Context->AssetView.IsValid())
						{
							const TSharedRef<SAssetView> AssetView = Context->AssetView.Pin().ToSharedRef();

							const TAttribute<FSlateIcon> GetIcon = TAttribute<FSlateIcon>::Create(
								TAttribute<FSlateIcon>::FGetter::CreateSPLambda(
									AssetView,
									[AssetView = &AssetView.Get()]()
									{
										EColumnSortMode::Type ActiveSortMode = EColumnSortMode::Ascending;
										if (const TSharedPtr<FAssetViewSortManager> StrongSortManager = AssetView->GetSortManager().Pin())
										{
											// @note: we don't use Secondary priorities, so this should change if/when we do
											ActiveSortMode = StrongSortManager->GetSortMode(EColumnSortPriority::Primary);
										}

										return ActiveSortMode == EColumnSortMode::Ascending
											? FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortDown")
											: FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.SortUp");
									}));

							FToolUIAction SortAction;
							SortAction.IsActionVisibleDelegate = FToolMenuIsActionButtonVisible::CreateSPLambda(
								AssetView,
								[AssetView = &AssetView.Get()](const FToolMenuContext&)
								{
									// Sort button only visible in Tile view mode - others have their own sorting UI representation.
									return AssetView->IsCurrentViewType(EAssetViewType::Tile);
								});

							FToolMenuEntry& SortEntry = InSection.AddEntry(
								FToolMenuEntry::InitComboButton(
									"Sort",
									FToolUIActionChoice(SortAction),
									FNewToolMenuChoice(
										FNewToolMenuDelegate::CreateSP(
											AssetView,
											&SAssetView::PopulateSortingButtonMenu)),
									FText::GetEmpty(), // Empty as button...
									LOCTEXT("SortToolTip", "Sorting options for the current asset view."),
									GetIcon));
						}
					}
				}));
		}
	});

FDelayedAutoRegisterHelper SAssetView::NavigationBarMenuExtensionRegistration(
	EDelayedRegisterRunPhase::EndOfEngineInit,
	[]
	{
		static const FName NavigationBarName("ContentBrowser.NavigationBar");
		if (UToolMenu* NavigationBarMenu = UToolMenus::Get()->ExtendMenu(NavigationBarName))
		{
			static bool bIsNewStyleEnabled = UE::Editor::ContentBrowser::IsNewStyleEnabled();

			if (bIsNewStyleEnabled)
			{
				FToolMenuSection& AssetViewInfoSection = NavigationBarMenu->FindOrAddSection("AssetViewInfo", { }, FToolMenuInsert("History", EToolMenuInsertType::Last));
				AssetViewInfoSection.Alignment = EToolMenuSectionAlign::Last;

				AssetViewInfoSection.AddDynamicEntry(
					"AssetViewInfo",
					FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
					{
						if (const UContentBrowserToolbarMenuContext* Context = InSection.FindContext<UContentBrowserToolbarMenuContext>())
						{
							if (Context->AssetView.IsValid())
							{
								const TSharedRef<SAssetView> AssetView = Context->AssetView.Pin().ToSharedRef();

								FToolMenuEntry& AssetViewInfoEntry = InSection.AddEntry(
									FToolMenuEntry::InitWidget(
										"AssetViewInfo",
										SNew(SBox)
										.Padding(FMargin(4, 0, 4, 0))
										[
											SNew(SHorizontalBox)
											+ SHorizontalBox::Slot()
											.HAlign(HAlign_Left)
											.FillWidth(1.0f)
											[
												SNew(STextBlock)
												.Text(AssetView, &SAssetView::GetAssetCountText)
											]
											+ SHorizontalBox::Slot()
											.HAlign(HAlign_Right)
											.FillWidth(1.0f)
											[
												SNew(STextBlock)
												.Text(AssetView, &SAssetView::GetMajorityAssetTypeText)
												.Visibility(AssetView, &SAssetView::GetMajorityAssetTypeTextVisibility)
												.ToolTipText(AssetView, &SAssetView::GetMajorityAssetTypeToolTipText)
											]
										],
										FText::GetEmpty()));
								AssetViewInfoEntry.WidgetData.StyleParams.VerticalAlignment = VAlign_Center;
								AssetViewInfoEntry.WidgetData.StyleParams.HorizontalAlignment = HAlign_Fill;
								AssetViewInfoEntry.WidgetData.StyleParams.FillSize = 1.0f;
								AssetViewInfoEntry.WidgetData.StyleParams.SizeRule = FSizeParam::SizeRule_Stretch;
							}
						}
					}));
			}
		}
	});

#undef LOCTEXT_NAMESPACE
