// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserStyle.h"
#include "Misc/DelayedAutoRegister.h"
#include "SActionButton.h"
#include "SAssetSearchBox.h"
#include "SContentBrowser.h"
#include "SFilterList.h"
#include "SNavigationBar.h"
#include "SPositiveActionButton.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "UObject/NameTypes.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

FDelayedAutoRegisterHelper SContentBrowser::NavigationBarMenuRegistration(
	EDelayedRegisterRunPhase::EndOfEngineInit,
	[]
	{
		FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

		static const FName NavigationBarMenuName("ContentBrowser.NavigationBar");
		UToolMenu* ToolBar = UToolMenus::Get()->RegisterMenu(NavigationBarMenuName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		ToolBar->SetStyleSet(&UE::ContentBrowser::Private::FContentBrowserStyle::Get());
		ToolBar->StyleName = "ContentBrowser.ToolBar";

		// @note: The navigation bar is registered here, but only used for extensions (@see: AssetViewMenus.cpp)
	});

FDelayedAutoRegisterHelper SContentBrowser::AddNewContextMenuRegistration(
	EDelayedRegisterRunPhase::EndOfEngineInit,
	[]
	{
        FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

        static const FName AddNewContextMenuName("ContentBrowser.AddNewContextMenu");
        UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(AddNewContextMenuName);
        Menu->SetStyleSet(&UE::ContentBrowser::Private::FContentBrowserStyle::Get());
		Menu->StyleName = "ContentBrowser.AddNewMenu";

        Menu->AddDynamicSection("DynamicSection_Common", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
        {
            TSharedPtr<SContentBrowser> ContentBrowser;
            if (const UContentBrowserMenuContext* MenuContext = InMenu->FindContext<UContentBrowserMenuContext>())
            {
                ContentBrowser = MenuContext->ContentBrowser.Pin();
            }
            else
            {
                if (const UContentBrowserToolbarMenuContext* ToolbarContext  = InMenu->FindContext<UContentBrowserToolbarMenuContext>())
                {
                    ContentBrowser = ToolbarContext->ContentBrowser.Pin();
                }
            }

            if (ContentBrowser)
            {
                ContentBrowser->PopulateAddNewContextMenu(InMenu);
            }
        }));
    });

FDelayedAutoRegisterHelper SContentBrowser::FolderContextMenuRegistration(
	EDelayedRegisterRunPhase::EndOfEngineInit,
	[]
	{
		FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

		static const FName FolderContextMenuName("ContentBrowser.FolderContextMenu");
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(FolderContextMenuName);
		Menu->bCloseSelfOnly = true;

		Menu->AddDynamicSection(NAME_Default, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			UContentBrowserFolderContext* Context = InMenu->FindContext<UContentBrowserFolderContext>();
			if (Context && Context->ContentBrowser.IsValid())
			{
				Context->ContentBrowser.Pin()->PopulateFolderContextMenu(InMenu);
			}
		}));
	});

FDelayedAutoRegisterHelper SContentBrowser::PathViewFiltersMenuRegistration(
	EDelayedRegisterRunPhase::EndOfEngineInit,
	[]
	{
        FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

        static const FName PathViewFiltersMenuName("ContentBrowser.AssetViewOptions.PathViewFilters");
        UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(PathViewFiltersMenuName);

        Menu->AddDynamicSection(NAME_Default, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
        {
            if (const UContentBrowserAssetViewContextMenuContext* AssetViewContext = InMenu->FindContext<UContentBrowserAssetViewContextMenuContext>())
            {
                if (const TSharedPtr<SContentBrowser> ContentBrowser = AssetViewContext->OwningContentBrowser.Pin())
                {
                    ContentBrowser->PopulatePathViewFiltersMenu(InMenu);
                }
            }
            else if (const UContentBrowserMenuContext* ContentBrowserContext = InMenu->FindContext<UContentBrowserMenuContext>())
            {
                if (const TSharedPtr<SContentBrowser> ContentBrowser = ContentBrowserContext->ContentBrowser.Pin())
                {
                    ContentBrowser->PopulatePathViewFiltersMenu(InMenu);
                }
            }
        }));
    });

FDelayedAutoRegisterHelper SContentBrowser::ToolBarMenuRegistration(
	EDelayedRegisterRunPhase::EndOfEngineInit,
	[]
	{
        FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);

		static const FName ToolBarMenuName("ContentBrowser.ToolBar");
        UToolMenu* ToolBar = UToolMenus::Get()->RegisterMenu(ToolBarMenuName, NAME_None,
        	UE::Editor::ContentBrowser::IsNewStyleEnabled() ? EMultiBoxType::SlimWrappingToolBar : EMultiBoxType::SlimHorizontalToolBar);
        ToolBar->SetStyleSet(&UE::ContentBrowser::Private::FContentBrowserStyle::Get());
        ToolBar->StyleName = ToolBarMenuName; // Style name is the same as the menu entry name

		FToolMenuSection& NewSection = ToolBar->FindOrAddSection("New");

        NewSection.AddDynamicEntry(
            "New",
            FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
            {
                if (UContentBrowserToolbarMenuContext* Context = InSection.FindContext<UContentBrowserToolbarMenuContext>())
                {
                    if (Context->ContentBrowser.IsValid())
                    {
                        const TSharedRef<SContentBrowser> ContentBrowser = Context->ContentBrowser.Pin().ToSharedRef();

                        const TSharedRef<SPositiveActionButton> NewButton =
                            SNew(SPositiveActionButton)
                            .OnGetMenuContent_Lambda([Context]
                            {
                                return Context->ContentBrowser.Pin()->MakeAddNewContextMenu(EContentBrowserDataMenuContext_AddNewMenuDomain::Toolbar, Context);
                            })
                            .ToolTipText(ContentBrowser, &SContentBrowser::GetAddNewToolTipText)
                            .IsEnabled(ContentBrowser, &SContentBrowser::IsAddNewEnabled)
                            .AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserNewAsset")))
                            .Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
                            .Text(LOCTEXT("AddAssetButton", "Add"));

						InSection.AddEntry(
							FToolMenuEntry::InitWidget(
								"NewButton",
								NewButton,
								FText::GetEmpty(),
								true,
								false
							));
                    }
                }
            }));

		FToolMenuSection& SaveSection = ToolBar->FindOrAddSection("Save", {}, FToolMenuInsert("New", EToolMenuInsertType::After));
		
        SaveSection.AddDynamicEntry(
            "Save",
            FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
            {
                if (const UContentBrowserToolbarMenuContext* Context = InSection.FindContext<UContentBrowserToolbarMenuContext>())
                {
                    if (Context->ContentBrowser.IsValid())
                    {
                        const TSharedRef<SContentBrowser> ContentBrowser = Context->ContentBrowser.Pin().ToSharedRef();

                        TSharedPtr<SWidget> SaveButton = nullptr;
                        if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
                        {
                            SaveButton =
                                SNew(SActionButton)
                                .ToolTipText(LOCTEXT("SaveDirtyPackagesTooltip", "Save all modified assets."))
                                .OnClicked(ContentBrowser, &SContentBrowser::OnSaveClicked)
                                .AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserSaveDirtyPackages")))
                                .Icon(FAppStyle::Get().GetBrush("MainFrame.SaveAll"))
                                .Text(LOCTEXT("SaveAll", "Save All"));
                        }
                        else
                        {
                            SaveButton =
                                SNew(SButton)
                                .ButtonStyle(FAppStyle::Get(), "SimpleButton")
                                .ToolTipText(LOCTEXT("SaveDirtyPackagesTooltip", "Save all modified assets."))
                                .ContentPadding(2.f)
                                .OnClicked(ContentBrowser, &SContentBrowser::OnSaveClicked)
                                .AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserSaveDirtyPackages")))
                                [
                                    SNew(SHorizontalBox)
                                    // Save All Icon
                                    + SHorizontalBox::Slot()
                                    .AutoWidth()
                                    .HAlign(HAlign_Center)
                                    .VAlign(VAlign_Center)
                                    [
                                        SNew(SImage)
                                        .Image(FAppStyle::Get().GetBrush("MainFrame.SaveAll"))
                                        .ColorAndOpacity(FSlateColor::UseForeground())
                                    ]

                                    // Save All Text
                                    + SHorizontalBox::Slot()
                                    .Padding(FMargin(3, 0, 0, 0))
                                    .VAlign(VAlign_Center)
                                    .AutoWidth()
                                    [
                                        SNew(STextBlock)
                                        .TextStyle(FAppStyle::Get(), "NormalText")
                                        .Text(LOCTEXT("SaveAll", "Save All"))
                                    ]
                                ];
                        }

                    	InSection.AddEntry(
							FToolMenuEntry::InitWidget(
								"SaveButton",
								SaveButton.ToSharedRef(),
								FText::GetEmpty(),
								true,
								false
							));
                    }
                }
            }));

        if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
        {
			FToolMenuSection& HistorySection = ToolBar->FindOrAddSection("History", {}, FToolMenuInsert("Save", EToolMenuInsertType::After));
        	HistorySection.ResizeParams.Wrapping.Allow = true;
        	HistorySection.ResizeParams.Wrapping.Priority = 5;
			HistorySection.AddDynamicEntry(
				"History",
				FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
				{
					if (!UE::Editor::ContentBrowser::IsNewStyleEnabled())
					{
						return;
					}

					if (const UContentBrowserToolbarMenuContext* Context = InSection.FindContext<UContentBrowserToolbarMenuContext>())
					{
						if (Context->ContentBrowser.IsValid())
						{
							const TSharedRef<SContentBrowser> ContentBrowser = Context->ContentBrowser.Pin().ToSharedRef();

							const TSharedRef<SActionButton> HistoryBackButton =
								SNew(SActionButton)
								.ActionButtonType(EActionButtonType::Simple)
								.ToolTipText(ContentBrowser, &SContentBrowser::GetHistoryBackTooltip)
								.IsEnabled(ContentBrowser, &SContentBrowser::IsBackEnabled)
								.OnClicked(ContentBrowser, &SContentBrowser::BackClicked)
								.Icon(FAppStyle::Get().GetBrush("Icons.CircleArrowLeft"))
								.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserHistoryBack")));

							InSection.AddEntry(
								FToolMenuEntry::InitWidget(
									"HistoryBackButton",
									HistoryBackButton,
									FText::GetEmpty()
								));

							const TSharedRef<SActionButton> HistoryForwardButton =
								SNew(SActionButton)
								.ActionButtonType(EActionButtonType::Simple)
								.ToolTipText(ContentBrowser, &SContentBrowser::GetHistoryForwardTooltip)
								.IsEnabled(ContentBrowser, &SContentBrowser::IsForwardEnabled)
								.OnClicked(ContentBrowser, &SContentBrowser::ForwardClicked)
								.Icon(FAppStyle::Get().GetBrush("Icons.CircleArrowRight"))
								.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserHistoryForward")));

							InSection.AddEntry(
								FToolMenuEntry::InitWidget(
									"HistoryForwardButton",
									HistoryForwardButton,
									FText::GetEmpty()
								));
						}
					}
				}));

	        FToolMenuSection& PathSection = ToolBar->FindOrAddSection("Path", {}, FToolMenuInsert("History", EToolMenuInsertType::After));
			PathSection.ResizeParams.Wrapping.Allow = true;
        	PathSection.ResizeParams.Wrapping.Mode = UE::Slate::PrioritizedWrapBox::EWrapMode::Parent;
			PathSection.ResizeParams.Wrapping.Priority = 5;

	        PathSection.AddDynamicEntry(
				"Path",
				FNewToolMenuSectionDelegate::CreateLambda(
				[](FToolMenuSection& InSection)
				{
					if (!UE::Editor::ContentBrowser::IsNewStyleEnabled())
					{
						return;
					}

					if (const UContentBrowserToolbarMenuContext* Context = InSection.FindContext<UContentBrowserToolbarMenuContext>())
					{
						if (Context->ContentBrowser.IsValid())
						{
							const TSharedRef<SContentBrowser> ContentBrowser = Context->ContentBrowser.Pin().ToSharedRef();

							// Should have been initialized in SContentBrowser::CreateAssetView
							if (!ensure(ContentBrowser->LegacyContentSourceWidgets->NavigationBar.IsValid()))
							{
								return;
							}

							FToolMenuEntry& PathBoxEntry =
								InSection.AddEntry(
									FToolMenuEntry::InitWidget(
										"PathBox",
										ContentBrowser->LegacyContentSourceWidgets->NavigationBar.ToSharedRef(),
										FText::GetEmpty()));

							PathBoxEntry.WidgetData.StyleParams.SizeRule = FSizeParam::SizeRule_StretchContent;
							PathBoxEntry.WidgetData.StyleParams.FillSize = 1.0f;
							PathBoxEntry.WidgetData.StyleParams.DesiredHeightOverride = 24.0f; // Match the height of the other widgets
						}
					}
				}));

			FToolMenuSection& FilterSection = ToolBar->FindOrAddSection("Filter", {}, FToolMenuInsert("Path", EToolMenuInsertType::After));
        	FilterSection.ResizeParams.Wrapping.Allow = true;
        	FilterSection.ResizeParams.Wrapping.ForceNewLine = true;
        	FilterSection.ResizeParams.Wrapping.Priority = 10;

        	FilterSection.AddDynamicEntry(
				"Filter",
				FNewToolMenuSectionDelegate::CreateLambda(
					[](FToolMenuSection& InSection)
					{
						if (const UContentBrowserToolbarMenuContext* Context = InSection.FindContext<UContentBrowserToolbarMenuContext>())
						{
							if (Context->ContentBrowser.IsValid())
							{
								const TSharedRef<SContentBrowser> ContentBrowser = Context->ContentBrowser.Pin().ToSharedRef();

								if (!ContentBrowser->FilterComboButton.IsValid())
								{
									return;
								}

								FToolMenuEntry& FilterEntry = InSection.AddEntry(
									FToolMenuEntry::InitWidget(
										"Filter",
										SNew(SBox)
										.HAlign(HAlign_Left)
										.VAlign(VAlign_Center)
										[
											ContentBrowser->FilterComboButton.ToSharedRef()
										],
										LOCTEXT("Filter", "Filter")
									));

								FilterEntry.WidgetData.StyleParams.HorizontalAlignment = HAlign_Left;
								FilterEntry.WidgetData.StyleParams.SizeRule = FSizeParam::ESizeRule::SizeRule_Auto;

								// ... no label when in the toolbar
								FilterEntry.ToolBarData.LabelOverride = FText::GetEmpty();
							}
						}
					}));

			FToolMenuSection& SearchSection = ToolBar->FindOrAddSection("Search", {}, FToolMenuInsert("Filter", EToolMenuInsertType::After));
        	SearchSection.ResizeParams.Wrapping.Allow = true;
        	SearchSection.ResizeParams.Wrapping.Mode = UE::Slate::PrioritizedWrapBox::EWrapMode::Parent;
        	SearchSection.ResizeParams.Wrapping.Priority = 10;

        	SearchSection.AddDynamicEntry(
                "Search",
                FNewToolMenuSectionDelegate::CreateLambda(
                [](FToolMenuSection& InSection)
                {
                    constexpr float SearchBoxMinWidth = 180.0f;
                    constexpr float SearchBoxMaxWidth = 640.0f;

                    if (const UContentBrowserToolbarMenuContext* Context = InSection.FindContext<UContentBrowserToolbarMenuContext>())
                    {
                        if (Context->ContentBrowser.IsValid())
                        {
                            const TSharedRef<SContentBrowser> ContentBrowser = Context->ContentBrowser.Pin().ToSharedRef();

                            // Should have been initialized in SContentBrowser::CreateAssetView
                            if (!ensure(ContentBrowser->LegacyContentSourceWidgets->SearchBoxPtr.IsValid()))
                            {
                                return;
                            }

                            FToolMenuEntry& SearchBoxEntry =
                                InSection.AddEntry(
                                FToolMenuEntry::InitWidget(
                                    "SearchBox",
                                    ContentBrowser->LegacyContentSourceWidgets->SearchBoxPtr.ToSharedRef(),
                                    FText::GetEmpty()));

                        	ContentBrowser->LegacyContentSourceWidgets->SearchBoxSizeSwitcher->SetSizeRange(FInt16Range(static_cast<int16>(FMath::FloorToInt(SearchBoxMaxWidth))));

							SearchBoxEntry.WidgetData.StyleParams.SizeRule = FSizeParam::SizeRule_StretchContent;
                        	SearchBoxEntry.WidgetData.StyleParams.FillSize = 1.0f;
							SearchBoxEntry.WidgetData.StyleParams.MinimumSize = SearchBoxMinWidth;
							SearchBoxEntry.WidgetData.StyleParams.MaximumSize = TAttribute<float>::Create(
								TAttribute<float>::FGetter::CreateSPLambda(
									ContentBrowser,
									[SizeSwitcher = ContentBrowser->LegacyContentSourceWidgets->SearchBoxSizeSwitcher.ToWeakPtr()]() -> float
									{
										if (SizeSwitcher.IsValid())
										{
											return SizeSwitcher.Pin()->GetDesiredSizeOverride();
										}

										return 0.0f;
									}));

                        	SearchBoxEntry.WidgetData.StyleParams.DesiredWidthOverride = TAttribute<FOptionalSize>::Create(
                        		TAttribute<FOptionalSize>::FGetter::CreateSPLambda(
                        			ContentBrowser,
                        			[SizeSwitcher = ContentBrowser->LegacyContentSourceWidgets->SearchBoxSizeSwitcher.ToWeakPtr()]() -> FOptionalSize
                        			{
                        				if (SizeSwitcher.IsValid())
                        				{
                        					return FOptionalSize(SizeSwitcher.Pin()->GetDesiredSizeOverride());
                        				}

                        				return FOptionalSize();
                        			}));
                        }
                    }
                }));

        	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
        	{
        		// Filters (plural) is the individual tag widgets, not the dropdown menu
				FToolMenuSection& FiltersSection = ToolBar->FindOrAddSection("Filters", {}, FToolMenuInsert("Sort", EToolMenuInsertType::After));
				FiltersSection.ResizeParams.Wrapping.Allow = true;
				FiltersSection.ResizeParams.Wrapping.Priority = 20;
				FiltersSection.ResizeParams.Wrapping.Mode = UE::Slate::PrioritizedWrapBox::EWrapMode::Parent; // Always wrap when too long, don't wait to hit the Preferred size threshold
				FiltersSection.ResizeParams.Wrapping.VerticalOverflowBehavior = UE::Slate::PrioritizedWrapBox::EVerticalOverflowBehavior::ExpandProportional;

				FiltersSection.AddDynamicEntry(
					"Filters",
					FNewToolMenuSectionDelegate::CreateLambda(
					[](FToolMenuSection& InSection)
					{
						if (const UContentBrowserToolbarMenuContext* Context = InSection.FindContext<UContentBrowserToolbarMenuContext>())
						{
							if (Context->ContentBrowser.IsValid())
							{
								const TSharedRef<SContentBrowser> ContentBrowser = Context->ContentBrowser.Pin().ToSharedRef();

								// Should have been initialized in SContentBrowser::CreateAssetView
								if (!ensure(ContentBrowser->LegacyContentSourceWidgets->FilterListPtr.IsValid()))
								{
									return;
								}

								FToolMenuEntry& FilterListEntry =
									InSection.AddEntry(
									FToolMenuEntry::InitWidget(
										"FilterList",
										SNew(SBox)
										.Padding(FMargin(-3))
										[
											ContentBrowser->LegacyContentSourceWidgets->FilterListPtr.ToSharedRef()											
										],
										FText::GetEmpty()));

								const TAttribute<EVisibility> FilterListVisibility = TAttribute<EVisibility>::Create(
									TAttribute<EVisibility>::FGetter::CreateSPLambda(
										ContentBrowser,
										[WeakContentBrowser = ContentBrowser.ToWeakPtr()]() -> EVisibility
										{
											if (WeakContentBrowser.IsValid())
											{
												const TSharedRef<SContentBrowser> StrongContentBrowser = WeakContentBrowser.Pin().ToSharedRef();
												const bool bIsHorizontal = StrongContentBrowser->GetFilterLayout() == EFilterBarLayout::Horizontal;
												const bool bHasAnyFilters = StrongContentBrowser->LegacyContentSourceWidgets
													&& StrongContentBrowser->LegacyContentSourceWidgets->FilterListPtr.IsValid()
													&& StrongContentBrowser->LegacyContentSourceWidgets->FilterListPtr->HasAnyFilters();
												
												return bIsHorizontal && bHasAnyFilters
													? EVisibility::Visible
													: EVisibility::Collapsed;
											}

											return EVisibility::Collapsed;
										}));

								FilterListEntry.Visibility = FilterListVisibility;
								FilterListEntry.WidgetData.StyleParams.SizeRule = FSizeParam::SizeRule_StretchContent;
								FilterListEntry.WidgetData.StyleParams.FillSizeMin = 0.1f;
								FilterListEntry.WidgetData.StyleParams.FillSize = 1.0f;
							}
						}
					}));
        	}

            FToolMenuSection& SettingsSection = ToolBar->FindOrAddSection("Settings", { }, FToolMenuInsert("Search", EToolMenuInsertType::After));
            SettingsSection.Alignment = EToolMenuSectionAlign::Last;

            SettingsSection.AddDynamicEntry(
	            "Settings",
	            FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	            {
	                if (const UContentBrowserToolbarMenuContext* Context = InSection.FindContext<UContentBrowserToolbarMenuContext>())
	                {
	                    if (Context->ContentBrowser.IsValid())
	                    {
	                        const TSharedRef<SContentBrowser> ContentBrowser = Context->ContentBrowser.Pin().ToSharedRef();

	                        const TSharedRef<SWidget> LockToggleButton =
	                            SNew(SActionButton)
	                            .ActionButtonType(EActionButtonType::Simple)
	                            .Icon(ContentBrowser, &SContentBrowser::GetLockIconBrush)
	                            .IconColorAndOpacity(FSlateColor::UseStyle())
	                            .ToolTipText(LOCTEXT("LockToggleTooltip", "Toggle lock. If locked, this browser will ignore Find in Content Browser requests."))
	                            .OnClicked(ContentBrowser, &SContentBrowser::ToggleLockClicked)
	                            .AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserLock")));

							FToolMenuEntry& LockEntry = InSection.AddEntry(
								FToolMenuEntry::InitWidget(
									"Lock",
									LockToggleButton,
									FText::GetEmpty()
								));
	                    }
	                }
	            }));
        }
    });

#undef LOCTEXT_NAMESPACE
