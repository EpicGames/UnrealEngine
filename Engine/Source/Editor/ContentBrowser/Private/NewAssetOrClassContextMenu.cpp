// Copyright Epic Games, Inc. All Rights Reserved.

#include "NewAssetOrClassContextMenu.h"

#include "ContentBrowserConfig.h"
#include "ContentBrowserDataMenuContexts.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserStyle.h"
#include "Framework/Commands/UIAction.h"
#include "IContentBrowserDataModule.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Settings/ContentBrowserSettings.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

void FNewAssetOrClassContextMenu::MakeContextMenu(
	UToolMenu* Menu, 
	const TArray<FName>& InSelectedPaths,
	const FOnNewFolderRequested& InOnNewFolderRequested, 
	const FOnGetContentRequested& InOnGetContentRequested
	)
{
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

	const FName FirstSelectedPath = (InSelectedPaths.Num() > 0) ? InSelectedPaths[0] : FName();
	const bool bIsValidNewFolderPath = ContentBrowserData->CanCreateFolder(FirstSelectedPath, nullptr);
	const bool bHasSinglePathSelected = InSelectedPaths.Num() == 1;

	auto CanExecuteFolderActions = [bHasSinglePathSelected, bIsValidNewFolderPath]() -> bool
	{
		// We can execute folder actions when we only have a single path selected, and that path is a valid path for creating a folder
		return bHasSinglePathSelected && bIsValidNewFolderPath;
	};

	const FCanExecuteAction CanExecuteFolderActionsDelegate = FCanExecuteAction::CreateLambda(CanExecuteFolderActions);

	// Get Content

	const FText GetContentSectionLabel =
		UE::Editor::ContentBrowser::IsNewStyleEnabled()
		? LOCTEXT("GetContentMenuHeading_NewStyle", "Get")
		: LOCTEXT("GetContentMenuHeading", "Get Content");

	FToolMenuSection& GetContentSection = Menu->AddSection("ContentBrowserGetContent", GetContentSectionLabel);
	UContentBrowserDataMenuContext_AddNewMenu* AddNewMenuContext = Menu->FindContext<UContentBrowserDataMenuContext_AddNewMenu>();

	if ( InOnGetContentRequested.IsBound() )
	{
		if (AddNewMenuContext && AddNewMenuContext->bCanBeModified && AddNewMenuContext->bContainsValidPackagePath)
		{
			GetContentSection.AddMenuEntry(
				"GetContent",
				LOCTEXT( "GetContentText", "Add Feature or Content Pack..." ),
				LOCTEXT( "GetContentTooltip", "Add features and content packs to the project." ),
				FSlateIcon(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetStyleSetName(), "ContentBrowser.AddContent" ),
				FUIAction( FExecuteAction::CreateStatic( &FNewAssetOrClassContextMenu::ExecuteGetContent, InOnGetContentRequested ) )
				);
		}
	}

	// If a single folder is selected and it is not valid for creating a new folder (The All Folder), don't create a context menu
	if (bHasSinglePathSelected && !bIsValidNewFolderPath)
	{
		return;
	}

	bool bDisplayFolders = GetDefault<UContentBrowserSettings>()->DisplayFolders;
	// check to see if we have an instance config that overrides the default in UContentBrowserSettings
	if (AddNewMenuContext && AddNewMenuContext->OwningInstanceConfig)
	{
		bDisplayFolders = AddNewMenuContext->OwningInstanceConfig->bShowFolders;
	}

	// New Folder
	if(InOnNewFolderRequested.IsBound() && bDisplayFolders)
	{
		{
			FToolMenuSection* Section = nullptr;
			if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
			{
				Section = &Menu->FindOrAddSection("ContentBrowserNewAsset");
			}
			else
			{
				Section = &Menu->AddSection("ContentBrowserNewFolder", LOCTEXT("FolderMenuHeading", "Folder") );
			}

			check(Section);

			FText NewFolderToolTip;
			if(bHasSinglePathSelected)
			{
				if(bIsValidNewFolderPath)
				{
					NewFolderToolTip = FText::Format(
						LOCTEXT("NewFolderTooltip_CreateIn", "Create a new folder in {0}."),
						IContentBrowserDataModule::Get().GetSubsystem()->ConvertVirtualPathToDisplay(FirstSelectedPath, EContentBrowserItemTypeFilter::IncludeFolders));
				}
				else
				{
					NewFolderToolTip = FText::Format(
						LOCTEXT("NewFolderTooltip_InvalidPath", "Cannot create new folders in {0}."),
						IContentBrowserDataModule::Get().GetSubsystem()->ConvertVirtualPathToDisplay(FirstSelectedPath, EContentBrowserItemTypeFilter::IncludeFolders));
				}
			}
			else
			{
				NewFolderToolTip = LOCTEXT("NewFolderTooltip_InvalidNumberOfPaths", "Can only create folders when there is a single path selected.");
			}

			Section->AddMenuEntry(
				"NewFolder",
				LOCTEXT("NewFolderLabel", "New Folder"),
				NewFolderToolTip,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.NewFolderIcon"),
				FUIAction(
					FExecuteAction::CreateStatic(&FNewAssetOrClassContextMenu::ExecuteNewFolder, FirstSelectedPath, InOnNewFolderRequested),
					CanExecuteFolderActionsDelegate
					)
				);
		}
	}
}

void FNewAssetOrClassContextMenu::ExecuteNewFolder(FName InPath, FOnNewFolderRequested InOnNewFolderRequested)
{
	if (ensure(!InPath.IsNone()))
	{
		InOnNewFolderRequested.ExecuteIfBound(InPath.ToString());
	}
}

void FNewAssetOrClassContextMenu::ExecuteGetContent( FOnGetContentRequested InOnGetContentRequested )
{
	InOnGetContentRequested.ExecuteIfBound();
}

#undef LOCTEXT_NAMESPACE
