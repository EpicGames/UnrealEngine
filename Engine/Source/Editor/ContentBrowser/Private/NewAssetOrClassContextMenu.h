// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "UObject/NameTypes.h"

class FString;
class UToolMenu;

class FNewAssetOrClassContextMenu
{
public:
	DECLARE_DELEGATE_OneParam( FOnNewFolderRequested, const FString& /*SelectedPath*/ );
	DECLARE_DELEGATE( FOnGetContentRequested )

	/** Makes the context menu widget */
	static void MakeContextMenu(
		UToolMenu* Menu, 
		const TArray<FName>& InSelectedPaths, 
		const FOnNewFolderRequested& InOnNewFolderRequested, 
		const FOnGetContentRequested& InOnGetContentRequested
		);

private:
	/** Create a new folder at the specified path */
	static void ExecuteNewFolder(FName InPath, FOnNewFolderRequested InOnNewFolderRequested);

	/** Handle when the "Get Content" button is clicked */
	static void ExecuteGetContent( FOnGetContentRequested InOnGetContentRequested );
};
