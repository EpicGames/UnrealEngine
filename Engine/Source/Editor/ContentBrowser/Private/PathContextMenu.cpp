// Copyright Epic Games, Inc. All Rights Reserved.

#include "PathContextMenu.h"

#include "AssetViewUtils.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserCommands.h"
#include "ContentBrowserDataFilter.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "ContentBrowserSingleton.h"
#include "ContentBrowserStyle.h"
#include "ContentBrowserUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/SlateDelegates.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformCrt.h"
#include "IContentBrowserDataModule.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "SlotBase.h"
#include "SourceControlOperations.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

FPathContextMenu::FPathContextMenu(const TWeakPtr<SWidget>& InParentContent)
	: ParentContent(InParentContent)
{
}

void FPathContextMenu::SetOnRenameFolderRequested(const FOnRenameFolderRequested& InOnRenameFolderRequested)
{
	OnRenameFolderRequested = InOnRenameFolderRequested;
}

void FPathContextMenu::SetOnFolderDeleted(const FOnFolderDeleted& InOnFolderDeleted)
{
	OnFolderDeleted = InOnFolderDeleted;
}

void FPathContextMenu::SetOnFolderFavoriteToggled(const FOnFolderFavoriteToggled& InOnFolderFavoriteToggled)
{
	OnFolderFavoriteToggled = InOnFolderFavoriteToggled;
}

void FPathContextMenu::SetOnFolderFavoriteEdited(const FOnFolderFavoriteEdited& InOnFolderFavoriteEdited)
{
	OnFolderFavoriteEdited = InOnFolderFavoriteEdited;
}

void FPathContextMenu::SetOnPrivateContentEditToggled(const FOnPrivateContentEditToggled& InOnPrivateContentEditToggled)
{
	OnPrivateContentEditToggled = InOnPrivateContentEditToggled;
}

const TArray<FContentBrowserItem>& FPathContextMenu::GetSelectedFolders() const
{
	return SelectedFolders;
}

void FPathContextMenu::SetSelectedFolders(const TArray<FContentBrowserItem>& InSelectedFolders)
{
	SelectedFolders = InSelectedFolders;
}

TSharedRef<FExtender> FPathContextMenu::MakePathViewContextMenuExtender(const TArray<FString>& InSelectedPaths)
{
	// Get all menu extenders for this context menu from the content browser module
	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>( TEXT("ContentBrowser") );
	TArray<FContentBrowserMenuExtender_SelectedPaths> MenuExtenderDelegates = ContentBrowserModule.GetAllPathViewContextMenuExtenders();

	TArray<TSharedPtr<FExtender>> Extenders;
	for (int32 i = 0; i < MenuExtenderDelegates.Num(); ++i)
	{
		if (MenuExtenderDelegates[i].IsBound())
		{
			Extenders.Add(MenuExtenderDelegates[i].Execute( InSelectedPaths ));
		}
	}
	TSharedPtr<FExtender> MenuExtender = FExtender::Combine(Extenders);
	return MenuExtender.ToSharedRef();
}

void FPathContextMenu::MakePathViewContextMenu(UToolMenu* Menu)
{
	UContentBrowserFolderContext* Context = Menu->FindContext<UContentBrowserFolderContext>();

	// Only add something if at least one folder is selected
	if ( SelectedFolders.Num() > 0 )
	{
		// Common operations section //
		{
			FToolMenuSection& Section = Menu->AddSection("PathViewFolderOptions", LOCTEXT("PathViewOptionsMenuHeading", "Folder Options") );

			{
				FText NewAssetToolTip;
				if(SelectedFolders.Num() == 1)
				{
					if(Context->bCanBeModified)
					{
						NewAssetToolTip = FText::Format(LOCTEXT("NewAssetTooltip_CreateIn", "Create a new item in {0}."), IContentBrowserDataModule::Get().GetSubsystem()->ConvertVirtualPathToDisplay(SelectedFolders[0]));
					}
				}
				else
				{
					NewAssetToolTip = LOCTEXT("NewAssetTooltip_InvalidNumberOfPaths", "Can only create items when there is a single path selected.");
				}

				// New Asset (submenu)
				if (Context->bCanBeModified)
				{
					FToolMenuEntry& NewAssetEntry = Section.AddSubMenu(
						"NewAsset",
						LOCTEXT("AddImportLabel", "Add/Import Content"),
						NewAssetToolTip,
						FNewToolMenuDelegate::CreateSP(this, &FPathContextMenu::MakeNewAssetSubMenu),
						FUIAction(),
						EUserInterfaceActionType::Button,
						false,
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Import")
					);

					NewAssetEntry.SubMenuData.Style.StyleSet = &UE::ContentBrowser::Private::FContentBrowserStyle::Get();
				}
			}

			// Explore
			if (!Context->bNoFolderOnDisk)
			{
				Section.AddMenuEntry(
					"Explore",
					ContentBrowserUtils::GetExploreFolderText(),
					LOCTEXT("ExploreTooltip", "Finds this folder on disk."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.BrowseContent"),
					FUIAction( FExecuteAction::CreateSP( this, &FPathContextMenu::ExecuteExplore ) )
					);
			}

			if (!Context->GetSelectedPackagePaths().IsEmpty())
			{
				Section.AddMenuEntry(
					"CopyPath",
					LOCTEXT("CopyFolderPath", "Copy Path"),
					LOCTEXT("CopyFolderTooltip", "Copy the paths of the selected folder(s)"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
					FExecuteAction::CreateSP(this, &FPathContextMenu::CopySelectedFolder)
				);
			}

			if (Context->bCanBeModified)
			{
				Section.AddMenuEntry(FGenericCommands::Get().Rename,
					LOCTEXT("RenameFolder", "Rename"),
					LOCTEXT("RenameFolderTooltip", "Rename the selected folder.")
					);
			}

			// If any colors have already been set, display color options as a sub menu
			if ( ContentBrowserUtils::HasCustomColors() )
			{
				// Set Color (submenu)
				Section.AddSubMenu(
					"SetColor",
					LOCTEXT("SetColor", "Set Color"),
					LOCTEXT("SetColorTooltip", "Sets the color this folder should appear as."),
					FNewToolMenuDelegate::CreateSP( this, &FPathContextMenu::MakeSetColorSubMenu ),
					false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Color")
					);
			}
			else
			{
				// Set Color
				Section.AddMenuEntry(
					"SetColor",
					LOCTEXT("SetColor", "Set Color"),
					LOCTEXT("SetColorTooltip", "Sets the color this folder should appear as."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Color"),
					FUIAction( FExecuteAction::CreateSP( this, &FPathContextMenu::ExecutePickColor ) )
					);
			}			

			FString SelectedFolderPath = SelectedFolders[0].GetVirtualPath().ToString();
			FContentBrowserItemPath SelectedFolderItemPath(SelectedFolders[0].GetVirtualPath(), EContentBrowserPathType::Virtual);
			// If this folder is already favorited, show the option to remove from favorites
			if (ContentBrowserUtils::IsFavoriteFolder(SelectedFolderItemPath))
			{
				// Remove from favorites
				Section.AddMenuEntry(
					"RemoveFromFavorites",
					LOCTEXT("RemoveFromFavorites", "Remove From Favorites"),
					LOCTEXT("RemoveFromFavoritesTooltip", "Removes this folder from the favorites section."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "PropertyWindow.Favorites_Disabled"),
					FUIAction(FExecuteAction::CreateSP(this, &FPathContextMenu::ExecuteFavorite))
				);
				
				if (SelectedFolders.Num() == 1)
				{
					// Edit favorite
					Section.AddMenuEntry(
						"EditFavorite",
						LOCTEXT("EditFavorite", "Edit Favorite Item"),
						LOCTEXT("EditFavoriteTooltip", "Edit and configure favorite item options"),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
						FUIAction(FExecuteAction::CreateSP(this, &FPathContextMenu::ExecuteEditFavorite))
					);
				}
			}
			else
			{
				// Add to favorites
				Section.AddMenuEntry(
					"AddToFavorites",
					LOCTEXT("AddToFavorites", "Add To Favorites"),
					LOCTEXT("AddToFavoritesTooltip", "Adds this folder to the favorites section for easy access."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Star"),
					FUIAction(FExecuteAction::CreateSP(this, &FPathContextMenu::ExecuteFavorite))
				);
			}

			static const IConsoleVariable* EnablePublicAssetFeatureCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("AssetTools.EnablePublicAssetFeature"));
			const bool bIsPublicAssetUIEnabled = EnablePublicAssetFeatureCVar && EnablePublicAssetFeatureCVar->GetBool();

			FStringView SelectedFolderPathView(SelectedFolderPath);
			if (bIsPublicAssetUIEnabled && FContentBrowserSingleton::Get().IsFolderShowPrivateContentToggleable(SelectedFolderPathView))
			{
				if (FContentBrowserSingleton::Get().IsShowingPrivateContent(SelectedFolderPathView))
				{
					Section.AddMenuEntry(
						"DisallowPrivateContentEditing",
						LOCTEXT("DisallowPrivateContentEditing", "Disallow Private Content Editing"),
						LOCTEXT("DisallowPrivateContentEditingTooltip", "Hides Private Content and prevents editing the Public/Private state of content in this folder"),
						FSlateIcon(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetStyleSetName(), "ContentBrowser.PrivateContentEdit"),
						FUIAction(FExecuteAction::CreateSP(this, &FPathContextMenu::ExecutePrivateContentEdit))
					);
				}
				else
				{
					Section.AddMenuEntry(
						"AllowPrivateContentEditing",
						LOCTEXT("AllowPrivateContentEditing", "Allow Private Content Editing"),
						LOCTEXT("AllowPrivateContentEditingTooltip", "Reveals Private Content and allows editing the Public/Private state of content in this folder"),
						FSlateIcon(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetStyleSetName(), "ContentBrowser.PrivateContentEdit"),
						FUIAction(FExecuteAction::CreateSP(this, &FPathContextMenu::ExecutePrivateContentEdit))
					);
				}
			}
		}

		if(Context->bCanBeModified)
		{
			// Bulk operations section //
			{
				FToolMenuSection& Section = Menu->AddSection("PathContextBulkOperations", LOCTEXT("AssetTreeBulkMenuHeading", "Bulk Operations") );

				// Save
				Section.AddMenuEntry(FContentBrowserCommands::Get().SaveAllCurrentFolder,
					LOCTEXT("SaveFolder", "Save All"),
					LOCTEXT("SaveFolderTooltip", "Saves all modified assets in this folder.")
					);

				// Resave
				Section.AddMenuEntry(FContentBrowserCommands::Get().ResaveAllCurrentFolder);

				// Delete
				Section.AddMenuEntry(FGenericCommands::Get().Delete,
					LOCTEXT("DeleteFolder", "Delete"),
					TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &FPathContextMenu::GetDeleteToolTip))
					);
			}
		}
	}
}

void FPathContextMenu::MakeNewAssetSubMenu(UToolMenu* Menu)
{
	UToolMenus::Get()->AssembleMenuHierarchy(Menu, UToolMenus::Get()->CollectHierarchy("ContentBrowser.AddNewContextMenu"));
}

void FPathContextMenu::MakeSetColorSubMenu(UToolMenu* Menu)
{
	{
		FToolMenuSection& Section = Menu->AddSection("Section");

		// New Color
		Section.AddMenuEntry(
			"NewColor",
			LOCTEXT("NewColor", "New Color"),
			LOCTEXT("NewColorTooltip", "Changes the color this folder should appear as."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FPathContextMenu::ExecutePickColor))
		);

		// Clear Color (only required if any of the selection has one)
		if (SelectedHasCustomColors())
		{
			Section.AddMenuEntry(
				"ClearColor",
				LOCTEXT("ClearColor", "Clear Color"),
				LOCTEXT("ClearColorTooltip", "Resets the color this folder appears as."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FPathContextMenu::ExecuteResetColor))
			);
		}
	}

	// Add all the custom colors the user has chosen so far
	TArray< FLinearColor > CustomColors;
	if ( ContentBrowserUtils::HasCustomColors( &CustomColors ) )
	{	
		{
			FToolMenuSection& Section = Menu->AddSection("PathContextCustomColors", LOCTEXT("CustomColorsExistingColors", "Existing Colors") );
			for ( int32 ColorIndex = 0; ColorIndex < CustomColors.Num(); ColorIndex++ )
			{
				const FLinearColor& Color = CustomColors[ ColorIndex ];
				Section.AddEntry(FToolMenuEntry::InitWidget(
					NAME_None,
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2, 0, 0, 0)
					[
						SNew(SButton)
						.ButtonStyle( FAppStyle::Get(), "Menu.Button" )
						.OnClicked( this, &FPathContextMenu::OnColorClicked, Color )
						[
							SNew(SColorBlock)
							.Color( Color )
							.Size( FVector2D(77,16) )
						]
					],
					FText::GetEmpty(),
					/*bNoIndent=*/true
				));
			}
		}
	}
}

void FPathContextMenu::ExecuteExplore()
{
	if (TSharedPtr<SWidget> PinnedParentContent = ParentContent.Pin())
	{
		ContentBrowserUtils::ExploreFolders(SelectedFolders, PinnedParentContent.ToSharedRef());
	}
}

bool FPathContextMenu::CanExecuteRename() const
{
	return SelectedFolders.Num() == 1 && SelectedFolders[0].CanRename(nullptr);
}

void FPathContextMenu::ExecuteRename(EContentBrowserViewContext ViewContext)
{
	check(SelectedFolders.Num() == 1);
	if (OnRenameFolderRequested.IsBound())
	{
		OnRenameFolderRequested.Execute(SelectedFolders[0], ViewContext);
	}
}

void FPathContextMenu::ExecuteResetColor()
{
	ResetColors();
}

void FPathContextMenu::ExecutePickColor()
{
	if (SelectedFolders.Num() == 0)
	{
		return;
	}

	// Spawn a color picker, so the user can select which color they want
	FLinearColor InitialColor = ContentBrowserUtils::GetDefaultColor();
	if (SelectedFolders.Num() > 0)
	{
		// Make sure an color entry exists for all the paths, otherwise they won't update in realtime with the widget color
		for (const FContentBrowserItem& SelectedItem : SelectedFolders)
		{
			const FString Path = SelectedItem.GetInvariantPath().ToString();

			TOptional<FLinearColor> Color = ContentBrowserUtils::GetPathColor(Path);
			if (Color.IsSet())
			{
				// Default the color to the first valid entry
				InitialColor = Color.GetValue();
				break;
			}
		}
	}

	FColorPickerArgs PickerArgs = FColorPickerArgs(InitialColor, FOnLinearColorValueChanged::CreateSP(this, &FPathContextMenu::OnLinearColorValueChanged));
	PickerArgs.bIsModal = false;
	PickerArgs.ParentWidget = ParentContent.Pin();

	OpenColorPicker(PickerArgs);
}

void FPathContextMenu::ExecuteFavorite()
{
	TArray<FString> PathsToUpdate;
	for (const FContentBrowserItem& SelectedItem : SelectedFolders)
	{
		PathsToUpdate.Add(SelectedItem.GetVirtualPath().ToString());
	}

	OnFolderFavoriteToggled.ExecuteIfBound(PathsToUpdate);
}

void FPathContextMenu::ExecuteEditFavorite()
{
	if (SelectedFolders.Num() == 1)
	{
		OnFolderFavoriteEdited.ExecuteIfBound(SelectedFolders[0].GetItemPath());
	}
}

void FPathContextMenu::ExecutePrivateContentEdit()
{
	TArray<FString> PathsToUpdate;
	for (const FContentBrowserItem& SelectedItem : SelectedFolders)
	{
		PathsToUpdate.Add(SelectedItem.GetVirtualPath().ToString());
	}

	OnPrivateContentEditToggled.ExecuteIfBound(PathsToUpdate);
}

void FPathContextMenu::OnLinearColorValueChanged(const FLinearColor InColor)
{
	OnColorClicked(InColor);
}

FReply FPathContextMenu::OnColorClicked( const FLinearColor InColor )
{
	// Make sure a color entry exists for all the paths, otherwise it can't save correctly
	for (const FContentBrowserItem& SelectedItem : SelectedFolders)
	{
		const FString Path = SelectedItem.GetInvariantPath().ToString();
		ContentBrowserUtils::SetPathColor(Path, InColor);
	}

	// Dismiss the menu here, as we can't make the 'clear' option appear if a folder has just had a color set for the first time
	FSlateApplication::Get().DismissAllMenus();

	return FReply::Handled();
}

void FPathContextMenu::ResetColors()
{
	// Clear the custom colors for all the selected paths
	for (const FContentBrowserItem& SelectedItem : SelectedFolders)
	{
		ContentBrowserUtils::SetPathColor(SelectedItem.GetInvariantPath().ToString(), TOptional<FLinearColor>());
	}
}

void FPathContextMenu::ExecuteSaveFolder()
{
	SaveFilesWithinSelectedFolders(EContentBrowserItemSaveFlags::SaveOnlyIfDirty | EContentBrowserItemSaveFlags::SaveOnlyIfLoaded);
}

void FPathContextMenu::ExecuteResaveFolder()
{
	SaveFilesWithinSelectedFolders(EContentBrowserItemSaveFlags::None);
}

void FPathContextMenu::CopySelectedFolder()
{
	CopySelectedFoldersToClipoard();
}

void FPathContextMenu::SaveFilesWithinSelectedFolders(EContentBrowserItemSaveFlags InSaveFlags)
{
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

	// Batch these by their data sources
	TMap<UContentBrowserDataSource*, TArray<FContentBrowserItemData>> SourcesAndItems;
	for (const FContentBrowserItem& SelectedItem : SelectedFolders)
	{
		FContentBrowserDataFilter SubFileFilter;
		SubFileFilter.bRecursivePaths = true;
		SubFileFilter.ItemTypeFilter = EContentBrowserItemTypeFilter::IncludeFiles;

		// Get the file items within this folder
		ContentBrowserData->EnumerateItemsUnderPath(SelectedItem.GetVirtualPath(), SubFileFilter, [InSaveFlags , &SourcesAndItems](FContentBrowserItemData&& InFileItem)
		{
			if (UContentBrowserDataSource* ItemDataSource = InFileItem.GetOwnerDataSource())
			{
				if (ItemDataSource->CanSaveItem(InFileItem, InSaveFlags, nullptr))
				{
					TArray<FContentBrowserItemData>& ItemsForSource = SourcesAndItems.FindOrAdd(ItemDataSource);
					ItemsForSource.Add(MoveTemp(InFileItem));
				}
			}

			return true;
		});
	}

	// Execute the operation now
	for (const auto& SourceAndItemsPair : SourcesAndItems)
	{
		SourceAndItemsPair.Key->BulkSaveItems(SourceAndItemsPair.Value, InSaveFlags);
	}
}

void FPathContextMenu::CopySelectedFoldersToClipoard()
{
	ContentBrowserUtils::CopyFolderReferencesToClipboard(SelectedFolders);
}

bool FPathContextMenu::CanExecuteDelete() const
{
	bool bCanDelete = false;
	for (const FContentBrowserItem& SelectedItem : SelectedFolders)
	{
		bCanDelete |= SelectedItem.CanDelete();
	}
	return bCanDelete;
}

FText FPathContextMenu::GetDeleteToolTip() const
{
	FText ErrorMessage;
	bool bCanDelete = false;
	for (const FContentBrowserItem& SelectedItem : SelectedFolders)
	{
		bCanDelete |= SelectedItem.CanDelete(&ErrorMessage);
	}

	if (!bCanDelete && !ErrorMessage.IsEmpty())
	{
		return ErrorMessage;
	}

	return LOCTEXT("DeleteFolderTooltip", "Removes this folder and all assets it contains.");
}

void FPathContextMenu::ExecuteDelete()
{
	TSharedPtr<SWidget> ParentContentPtr = ParentContent.Pin();
	if (ParentContentPtr && SelectedFolders.Num() > 0)
	{
		ExecuteDeleteFolderConfirmed();
	}
}

FReply FPathContextMenu::ExecuteDeleteFolderConfirmed()
{
	// Batch these by their data sources
	TMap<UContentBrowserDataSource*, TArray<FContentBrowserItemData>> SourcesAndItems;
	for (const FContentBrowserItem& SelectedItem : SelectedFolders)
	{
		FContentBrowserItem::FItemDataArrayView ItemDataArray = SelectedItem.GetInternalItems();
		for (const FContentBrowserItemData& ItemData : ItemDataArray)
		{
			if (UContentBrowserDataSource* ItemDataSource = ItemData.GetOwnerDataSource())
			{
				FText DeleteErrorMsg;
				if (ItemDataSource->CanDeleteItem(ItemData, &DeleteErrorMsg))
				{
					TArray<FContentBrowserItemData>& ItemsForSource = SourcesAndItems.FindOrAdd(ItemDataSource);
					ItemsForSource.Add(ItemData);
				}
				else
				{
					AssetViewUtils::ShowErrorNotifcation(DeleteErrorMsg);
				}
			}
		}
	}

	// Execute the operation now
	bool bDidDelete = false;
	for (const auto& SourceAndItemsPair : SourcesAndItems)
	{
		bDidDelete |= SourceAndItemsPair.Key->BulkDeleteItems(SourceAndItemsPair.Value);
	}

	if (bDidDelete)
	{
		ResetColors();
		OnFolderDeleted.ExecuteIfBound();
	}

	return FReply::Handled();
}

bool FPathContextMenu::SelectedHasCustomColors() const
{
	for (const FContentBrowserItem& SelectedItem : SelectedFolders)
	{
		if (const TOptional<FLinearColor> Color = ContentBrowserUtils::GetPathColor(SelectedItem.GetInvariantPath().ToString()))
		{
			// Ignore any that are the default color, in case the user used the deprecated SaveColor with bForce
			if (!Color->Equals(ContentBrowserUtils::GetDefaultColor()))
			{
				return true;
			}
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
