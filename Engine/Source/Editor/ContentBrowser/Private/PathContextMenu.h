// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserItemData.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"

struct FContentBrowserFavoriteItem;
class FExtender;
class FString;
class SWidget;
class SWindow;
class UToolMenu;

enum class EContentBrowserViewContext : uint8;

class FPathContextMenu : public TSharedFromThis<FPathContextMenu>
{
public:
	/** Constructor */
	FPathContextMenu(const TWeakPtr<SWidget>& InParentContent);

	/** Delegate for when the context menu requests a rename of a folder */
	DECLARE_DELEGATE_TwoParams(FOnRenameFolderRequested, const FContentBrowserItem& /*FolderToRename*/, EContentBrowserViewContext /*ViewContext*/);
	void SetOnRenameFolderRequested(const FOnRenameFolderRequested& InOnRenameFolderRequested);

	/** Delegate for when the context menu has successfully deleted a folder */
	DECLARE_DELEGATE(FOnFolderDeleted)
	void SetOnFolderDeleted(const FOnFolderDeleted& InOnFolderDeleted);

	/** Delegate for when the context menu has successfully toggled the favorite status of a folder */
	DECLARE_DELEGATE_OneParam(FOnFolderFavoriteToggled, const TArray<FString>& /*FoldersToToggle*/)
	void SetOnFolderFavoriteToggled(const FOnFolderFavoriteToggled& InOnFolderFavoriteToggled);
	
	/** Delegate for when the context menu has successfully edited a favorite folder */
	DECLARE_DELEGATE_OneParam(FOnFolderFavoriteEdited, const FContentBrowserItemPath& /*FolderToEdit*/)
	void SetOnFolderFavoriteEdited(const FOnFolderFavoriteEdited& InOnFolderFavoriteEdited);

	/* Delegate for when the context menu has succesfully toggled the private content edit mode of a folder*/
	DECLARE_DELEGATE_OneParam(FOnPrivateContentEditToggled, const TArray<FString>& /*FolderVirtualPaths*/)
	void SetOnPrivateContentEditToggled(const FOnPrivateContentEditToggled& InOnPrivateContentEditableToggled);

	/** Gets the currently selected folders */
	const TArray<FContentBrowserItem>& GetSelectedFolders() const;

	/** Sets the currently selected folders */
	void SetSelectedFolders(const TArray<FContentBrowserItem>& InSelectedFolders);

	/** Makes the asset tree context menu extender */
	TSharedRef<FExtender> MakePathViewContextMenuExtender(const TArray<FString>& InSelectedPaths);

	/** Makes the asset tree context menu widget */
	void MakePathViewContextMenu(UToolMenu* Menu);

	/** Makes the new asset submenu */
	void MakeNewAssetSubMenu(UToolMenu* Menu);

	/** Makes the set color submenu */
	void MakeSetColorSubMenu(UToolMenu* Menu);

	/** Handler for when "Explore" is selected */
	void ExecuteExplore();

	/** Handler to check to see if a rename command is allowed */
	bool CanExecuteRename() const;

	/** Handler for Rename */
	void ExecuteRename(EContentBrowserViewContext ViewContext);

	/** Handler to check to see if a delete command is allowed */
	bool CanExecuteDelete() const;

	/** Handler for Delete */
	void ExecuteDelete();

	/** Handler for when reset color is selected */
	void ExecuteResetColor();

	/** Handler for when new or set color is selected */
	void ExecutePickColor();

	/** Handler for favoriting */
	void ExecuteFavorite();
	
	/** Handler for favorite editing */
	void ExecuteEditFavorite();

	/* Handler for enabling private content editing*/
	void ExecutePrivateContentEdit();

	/** Handler for when "Save" is selected */
	void ExecuteSaveFolder();

	/** Handler for when "Resave" is selected */
	void ExecuteResaveFolder();

	/** Handler for when "Copy AssetPath" is selected */
	void CopySelectedFolder();

	/** Handler for when "Delete" is selected and the delete was confirmed */
	FReply ExecuteDeleteFolderConfirmed();

	/** Get the parent widget for which this menu was summoned. */
	TSharedPtr<SWidget> GetParentContent() const { return ParentContent.Pin(); }

private:
	/** Get tooltip for delete */
	FText GetDeleteToolTip() const;

	void SaveFilesWithinSelectedFolders(EContentBrowserItemSaveFlags InSaveFlags);
	
	void CopySelectedFoldersToClipoard();

	/** Checks to see if any of the selected paths use custom colors */
	bool SelectedHasCustomColors() const;

	/** Callback when the color picker dialog changed the color. */
	void OnLinearColorValueChanged(const FLinearColor InColor);

	/** Callback when the color is picked from the set color submenu */
	FReply OnColorClicked( const FLinearColor InColor );

	/** Resets the colors of the selected paths */
	void ResetColors();

private:
	TArray<FContentBrowserItem> SelectedFolders;
	TWeakPtr<SWidget> ParentContent;
	FOnRenameFolderRequested OnRenameFolderRequested;
	FOnFolderDeleted OnFolderDeleted;
	FOnFolderFavoriteToggled OnFolderFavoriteToggled;
	FOnFolderFavoriteEdited OnFolderFavoriteEdited;
	FOnPrivateContentEditToggled OnPrivateContentEditToggled;
};
