// Copyright Epic Games, Inc. All Rights Reserved.

#include "PathViewTypes.h"

#include "Algo/Copy.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserItemData.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UnrealTemplate.h"

FTreeItem::FTreeItem(FContentBrowserItem&& InItem)
	: Item(MoveTemp(InItem))
{
	checkf(Item.IsFolder(), TEXT("FTreeItem must be constructed from a folder item!"));
}

FTreeItem::FTreeItem(const FContentBrowserItem& InItem)
	: Item(InItem)
{
	checkf(Item.IsFolder(), TEXT("FTreeItem must be constructed from a folder item!"));
}

FTreeItem::FTreeItem(FContentBrowserItemData&& InItemData)
	: Item(MoveTemp(InItemData))
{
	checkf(Item.IsFolder(), TEXT("FTreeItem must be constructed from a folder item!"));
}

FTreeItem::FTreeItem(const FContentBrowserItemData& InItemData)
	: Item(InItemData)
{
	checkf(Item.IsFolder(), TEXT("FTreeItem must be constructed from a folder item!"));
}

void FTreeItem::AppendItemData(const FContentBrowserItem& InItem)
{
	checkf(InItem.IsFolder(), TEXT("FTreeItem can only contain folder items!"));
	Item.Append(InItem);
}

void FTreeItem::AppendItemData(const FContentBrowserItemData& InItemData)
{
	checkf(InItemData.IsFolder(), TEXT("FTreeItem can only contain folder items!"));
	Item.Append(InItemData);
}

void FTreeItem::RemoveItemData(const FContentBrowserItem& InItem)
{
	checkf(InItem.IsFolder(), TEXT("FTreeItem can only contain folder items!"));
	Item.Remove(InItem);
}

void FTreeItem::RemoveItemData(const FContentBrowserItemData& InItemData)
{
	checkf(InItemData.IsFolder(), TEXT("FTreeItem can only contain folder items!"));
	Item.Remove(InItemData);
}

FContentBrowserItemData FTreeItem::RemoveItemData(const FContentBrowserMinimalItemData& InItemKey)
{
	return Item.TryRemove(InItemKey);
}

void FTreeItem::SetItemData(FContentBrowserItem InItem)
{
	Item = MoveTemp(InItem);
}

const FContentBrowserItem& FTreeItem::GetItem() const
{
	return Item;
}

void FTreeItem::SetVisible(bool bInIsVisible)
{
	bIsVisible = bInIsVisible;
}

void FTreeItem::SetHasVisibleDescendants(bool bValue)
{
	bHasVisibleDescendants = bValue;
}

bool FTreeItem::GetHasVisibleDescendants() const
{
	return bHasVisibleDescendants;
}

bool FTreeItem::IsVisible() const
{
	return bIsVisible || bHasVisibleDescendants;
}

FSimpleMulticastDelegate& FTreeItem::OnRenameRequested()
{
	return RenameRequestedEvent;
}

bool FTreeItem::IsNamingFolder() const
{
	return bNamingFolder;
}

void FTreeItem::SetNamingFolder(const bool InNamingFolder)
{
	bNamingFolder = InNamingFolder;
}

bool FTreeItem::IsChildOf(const FTreeItem& InParent)
{
	TSharedPtr<FTreeItem> CurrentParent = Parent.Pin();
	while (CurrentParent.IsValid())
	{
		if (CurrentParent.Get() == &InParent)
		{
			return true;
		}

		CurrentParent = CurrentParent->Parent.Pin();
	}

	return false;
}

void FTreeItem::AddChild(const TSharedRef<FTreeItem>& InChild)
{
	checkSlow(!AllChildren.Contains(InChild));
	AllChildren.Add(InChild);
	InChild->Parent = AsWeak();
	bChildrenRequireSort = true;
}

void FTreeItem::RemoveChild(const TSharedRef<FTreeItem>& InChild)
{
	if (InChild->Parent == AsWeak())
	{
		AllChildren.Remove(InChild);
		InChild->Parent = nullptr;
	}
}

void FTreeItem::RemoveAllChildren()
{
	AllChildren.Reset();
}

TConstArrayView<TSharedPtr<FTreeItem>> FTreeItem::GetChildren() const
{
	return AllChildren;
}

TSharedPtr<FTreeItem> FTreeItem::GetChild(const FName InChildFolderName) const
{
	for (const TSharedPtr<FTreeItem>& Child : AllChildren)
	{
		if (Child->Item.GetItemName() == InChildFolderName)
		{
			return Child;
		}
	}

	return nullptr;
}

TSharedPtr<FTreeItem> FTreeItem::GetParent() const
{
	return Parent.Pin();
}

TSharedPtr<FTreeItem> FTreeItem::FindItemRecursive(const FName InFullPath)
{
	if (InFullPath == Item.GetVirtualPath())
	{
		return SharedThis(this);
	}

	for (const TSharedPtr<FTreeItem>& Child : AllChildren)
	{
		if (TSharedPtr<FTreeItem> ChildItem = Child->FindItemRecursive(InFullPath))
		{
			return ChildItem;
		}
	}

	return nullptr;
}

void FTreeItem::ForAllChildrenRecursive(TFunctionRef<void(const TSharedRef<FTreeItem>&)> Functor)
{
	for (const TSharedPtr<FTreeItem>& Child : AllChildren)
	{
		if (Child.IsValid())
		{
			Functor(Child.ToSharedRef());
			Child->ForAllChildrenRecursive(Functor);
		}
	}
}

void FTreeItem::RequestSortChildren()
{
	bChildrenRequireSort = true;
}

void FTreeItem::GetSortedVisibleChildren(TArray<TSharedPtr<FTreeItem>>& OutChildren)
{
	if (bChildrenRequireSort)
	{
		UE::PathView::DefaultSort(AllChildren);
		bChildrenRequireSort = false;
	}
	OutChildren.Reset();
	Algo::CopyIf(AllChildren, OutChildren, UE_PROJECTION_MEMBER(FTreeItem, IsVisible));
}

bool FTreeItem::IsDisplayOnlyFolder() const
{
	return GetItem().IsDisplayOnlyFolder();
}
