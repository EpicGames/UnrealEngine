// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "ContentBrowserItem.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FContentBrowserItemData;
class FTreeItem;

// Sorts array of tree items
DECLARE_DELEGATE_TwoParams(FSortTreeItemChildrenDelegate, const FTreeItem* /* OptionalOwner */, TArray<TSharedPtr<FTreeItem>>& /* Items to sort */);

namespace UE::PathView
{
void DefaultSort(TArray<TSharedPtr<FTreeItem>>& InChildren);
}

/** A folder item shown in the asset tree */
class FTreeItem : public TSharedFromThis<FTreeItem>
{
public:
	FTreeItem() = default;

	explicit FTreeItem(FContentBrowserItem&& InItem);
	explicit FTreeItem(const FContentBrowserItem& InItem);

	explicit FTreeItem(FContentBrowserItemData&& InItemData);
	explicit FTreeItem(const FContentBrowserItemData& InItemData);

	// Replace the current item data or pending item data with the given item data
	void SetItemData(FContentBrowserItem InItem);

	void AppendItemData(const FContentBrowserItem& InItem);
	void AppendItemData(const FContentBrowserItemData& InItemData);
	void RemoveItemData(const FContentBrowserItem& InItem);
	void RemoveItemData(const FContentBrowserItemData& InItemData);
	FContentBrowserItemData RemoveItemData(const FContentBrowserMinimalItemData& InItemKey);

	/** Get the underlying Content Browser item */
	const FContentBrowserItem& GetItem() const;

	/** Get the event fired whenever a rename is requested */
	FSimpleMulticastDelegate& OnRenameRequested();

	/** Set whether the item passes current filtering and should be displayed */
	void SetVisible(bool bInIsVisible);

	/** Set whether this item has any descendants which explicitly passed filtering */
	void SetHasVisibleDescendants(bool bValue);

	/** Return whether this item has any descendants which explicitly passed filtering */
	bool GetHasVisibleDescendants() const;

	/** Returns whether the item passes current filtering and should be displayed, or whether any of its descendants did */
	bool IsVisible() const;

	/** True if this folder is in the process of being named */
	bool IsNamingFolder() const;

	/** Set whether this folder is in the process of being named */
	void SetNamingFolder(const bool InNamingFolder);

	/** Returns true if this item is a child of the specified item */
	bool IsChildOf(const FTreeItem& InParent);

	/** Add a child item and link its Parent backreference. */
	void AddChild(const TSharedRef<FTreeItem>& InChild);

	/** Remove a child item and unlink its Parent backreference. */
	void RemoveChild(const TSharedRef<FTreeItem>& InChild);

	/** Remove all children when recycling this item. They may be re-added later. */
	void RemoveAllChildren();

	/** Get a view of all the direct children of this node*/
	TConstArrayView<TSharedPtr<FTreeItem>> GetChildren() const;

	/** Returns the child item by name or NULL if the child does not exist */
	TSharedPtr<FTreeItem> GetChild(const FName InChildFolderName) const;

	/** Returns the parent item if any */
	TSharedPtr<FTreeItem> GetParent() const;

	/** Finds the child who's path matches the one specified */
	TSharedPtr<FTreeItem> FindItemRecursive(const FName InFullPath);

	/** Execute the given functor on all children of this item recursively */
	void ForAllChildrenRecursive(TFunctionRef<void(const TSharedRef<FTreeItem>&)> Functor);

	/** Request that the children be sorted the next time someone calls SortChildrenIfNeeded */
	void RequestSortChildren();

	/** Sort the children if necessary and populate the output parameter with a copy */
	void GetSortedVisibleChildren(TArray<TSharedPtr<FTreeItem>>& OutChildren);

	/** Represents a folder that does not correspond to a mounted location */
	bool IsDisplayOnlyFolder() const;

private:
	/** The children of this tree item */
	TArray<TSharedPtr<FTreeItem>> AllChildren;

	/** The parent folder for this item */
	TWeakPtr<FTreeItem> Parent;

	/** Underlying Content Browser item data */
	FContentBrowserItem Item;

	/** Broadcasts whenever a rename is requested */
	FSimpleMulticastDelegate RenameRequestedEvent;

	/** If true, this folder is in the process of being named */
	bool bNamingFolder = false;

	/** If true, the children of this item need sorting */
	bool bChildrenRequireSort = false;

	/** Whether this node has passed the current set of filters in use */
	bool bIsVisible = true;

	/** Whether this node has any descendants that are visible, so this node needs to be shown too */
	bool bHasVisibleDescendants = true;
};
