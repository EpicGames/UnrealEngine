// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAssetDialog.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetViewUtils.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "ContentBrowserCommands.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserItemData.h"
#include "ContentBrowserPluginFilters.h"
#include "ContentBrowserSingleton.h"
#include "ContentBrowserStyle.h"
#include "ContentBrowserUtils.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "ICollectionSource.h"
#include "Editor/EditorEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "IContentBrowserDataModule.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Layout/Children.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "SAssetPicker.h"
#include "SAssetView.h"
#include "SPathPicker.h"
#include "SPathView.h"
#include "SPrimaryButton.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "SNavigationBar.h"
#include "SWarningOrErrorBox.h"
#include "Experimental/ContentBrowserExtensionUtils.h"
#include "Styling/AppStyle.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateStructs.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/Class.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

class FExtender;
class SWidget;
struct FGeometry;
struct FWorldContext;

#define LOCTEXT_NAMESPACE "ContentBrowser"

SAssetDialog::SAssetDialog()
	: DialogType(EAssetDialogType::Open)
	, ExistingAssetPolicy(ESaveAssetDialogExistingAssetPolicy::Disallow)
	, bLastInputValidityCheckSuccessful(false)
	, bPendingFocusNextFrame(true)
	, bValidAssetsChosen(false)
{
}

SAssetDialog::~SAssetDialog()
{
	if (!bValidAssetsChosen)
	{
		OnAssetDialogCancelled.ExecuteIfBound();
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SAssetDialog::Construct(const FArguments& InArgs, const FSharedAssetDialogConfig& InConfig)
{
	DialogType = InConfig.GetDialogType();

	AssetClassNames = InConfig.AssetClassNames;

	HistoryManager.SetOnApplyHistoryData(FOnApplyHistoryData::CreateSP(this, &SAssetDialog::OnApplyHistoryData));
	HistoryManager.SetOnUpdateHistoryData(FOnUpdateHistoryData::CreateSP(this, &SAssetDialog::OnUpdateHistoryData));

	RecentDirectories.MaxItems = 30;

	const FString DefaultPath = InConfig.DefaultPath;

	RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SAssetDialog::SetFocusPostConstruct ) );

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.DefaultPath = DefaultPath;
	PathPickerConfig.bFocusSearchBoxWhenOpened = false;
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &SAssetDialog::SetCurrentlySelectedPath, EContentBrowserPathType::Virtual);
	PathPickerConfig.SetPathsDelegates.Add(&SetPathsDelegate);
	PathPickerConfig.OnGetFolderContextMenu = FOnGetFolderContextMenu::CreateSP(this, &SAssetDialog::OnGetFolderContextMenu, EAssetDialogCommandContext::PathView);
	PathPickerConfig.bOnPathSelectedPassesVirtualPaths = true;

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Append(AssetClassNames);
	AssetPickerConfig.bAllowDragging = false;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Tile;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SAssetDialog::OnAssetSelected);
	AssetPickerConfig.OnAssetsActivated = FOnAssetsActivated::CreateSP(this, &SAssetDialog::OnAssetsActivated);
	AssetPickerConfig.SetFilterDelegates.Add(&SetFilterDelegate);
	AssetPickerConfig.GetCurrentSelectionDelegates.Add(&GetCurrentSelectionDelegate);
	AssetPickerConfig.SaveSettingsName = TEXT("AssetDialog");
	AssetPickerConfig.bCanShowFolders = true;
	AssetPickerConfig.bCanShowDevelopersFolder = true;
	AssetPickerConfig.OnFolderEntered = FOnPathSelected::CreateSP(this, &SAssetDialog::SetCurrentlySelectedPath, EContentBrowserPathType::Virtual);
	AssetPickerConfig.OnGetAssetContextMenu = FOnGetAssetContextMenu::CreateSP(this, &SAssetDialog::OnGetAssetContextMenu);
	AssetPickerConfig.OnGetFolderContextMenu = FOnGetFolderContextMenu::CreateSP(this, &SAssetDialog::OnGetFolderContextMenu, EAssetDialogCommandContext::AssetView);

	OnPathSelected = InConfig.OnPathSelected;

	// Open and save specific configuration
	FText ConfirmButtonText;
	bool bIncludeNameBox = false;
	switch (DialogType)
	{
	case EAssetDialogType::Open:
	{
		const FOpenAssetDialogConfig& OpenAssetConfig = static_cast<const FOpenAssetDialogConfig&>(InConfig);
		PathPickerConfig.bAllowContextMenu = true;
		ConfirmButtonText = LOCTEXT("AssetDialogOpenButton", "Open");
		AssetPickerConfig.SelectionMode = OpenAssetConfig.bAllowMultipleSelection ? ESelectionMode::Multi : ESelectionMode::Single;
		AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
		bIncludeNameBox = false;
		break;
	}

	case EAssetDialogType::Save:
	{
		const FSaveAssetDialogConfig& SaveAssetConfig = static_cast<const FSaveAssetDialogConfig&>(InConfig);
		PathPickerConfig.bAllowContextMenu = true;
		PathPickerConfig.bAllowReadOnlyFolders = false;
		ConfirmButtonText = LOCTEXT("AssetDialogSaveButton", "Save");
		AssetPickerConfig.SelectionMode = ESelectionMode::Single;
		AssetPickerConfig.bFocusSearchBoxWhenOpened = false;
		AssetPickerConfig.bCanShowReadOnlyFolders = false;
		bIncludeNameBox = true;
		ExistingAssetPolicy = SaveAssetConfig.ExistingAssetPolicy;
		SetCurrentlyEnteredAssetName(SaveAssetConfig.DefaultAssetName);
		break;
	}

	default:
		ensureMsgf(0, TEXT("AssetDialog type %d is not supported."), DialogType);
		break;
	}

	PathPicker = StaticCastSharedRef<SPathPicker>(FContentBrowserSingleton::Get().CreatePathPicker(PathPickerConfig));

	TArray<FString> SelectedVirtualPaths = PathPicker->GetPaths();

	// The path view defers its population (and therefore the application of DefaultPath) until its
	// first paint, so GetPaths() is empty here even when the caller supplied a path. Seed the selection
	// from DefaultPath explicitly; otherwise the fallbacks below would override it with the global default.
	if (SelectedVirtualPaths.Num() == 0 && !DefaultPath.IsEmpty())
	{
		const FName VirtualDefaultPath = IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual(*DefaultPath);
		PathPicker->SetPaths({ VirtualDefaultPath.ToString() });
		SelectedVirtualPaths = PathPicker->GetPaths();
	}

	if (SelectedVirtualPaths.Num() == 0)
	{
		// No paths selected, choose PathView's default selection
		const TSharedPtr<SPathView>& PathView = PathPicker->GetPathView();
		const TArray<FName> DefaultPathsToSelect = PathView->GetDefaultPathsToSelect();
		if (DefaultPathsToSelect.Num() > 0)
		{
			// Try select path
			PathPicker->SetPaths({ DefaultPathsToSelect[0].ToString() });

			// Get paths that were successfully selected
			SelectedVirtualPaths = PathPicker->GetPaths();
		}

		if (SelectedVirtualPaths.Num() == 0)
		{
			// No paths selected, choose selection based on first root folder displayed in PathView
			const TArray<FName> RootPathItemNames = PathView->GetRootPathItemNames();
			if (RootPathItemNames.Num() > 0)
			{
				// Try select path
				PathPicker->SetPaths({ FString(TEXT("/")) + RootPathItemNames[0].ToString() });

				// Get paths that were successfully selected
				SelectedVirtualPaths = PathPicker->GetPaths();
			}
		}
	}

	// Update AssetPickerConfig's selection to match PathPicker
	if (SelectedVirtualPaths.Num() > 0)
	{
		AssetPickerConfig.Filter.PackagePaths = { FName(*SelectedVirtualPaths[0]) };
	}

	AssetPicker = StaticCastSharedRef<SAssetPicker>(FContentBrowserSingleton::Get().CreateAssetPicker(AssetPickerConfig));

	FContentBrowserCommands::Register();
	BindCommands();

	// The root widget in this dialog.
	TSharedRef<SVerticalBox> MainVerticalBox = SNew(SVerticalBox);

	// Navigation & History
	MainVerticalBox->AddSlot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SHorizontalBox)
				// History Back Button
				+SHorizontalBox::Slot()
				.Padding(10, 0, 0, 0)
				.AutoWidth()
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(this, &SAssetDialog::GetBackTooltip )
					.ContentPadding( FMargin(1, 0) )
					.OnClicked(this, &SAssetDialog::OnBackClicked)
					.IsEnabled(this, &SAssetDialog::IsBackEnabled)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.CircleArrowLeft"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
				// History Forward Button
				+ SHorizontalBox::Slot()
				.Padding(2, 0, 0, 0)
				.AutoWidth()
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText( this, &SAssetDialog::GetForwardTooltip )
					.ContentPadding( FMargin(1, 0) )
					.OnClicked(this, &SAssetDialog::OnForwardClicked)
					.IsEnabled(this, &SAssetDialog::IsForwardEnabled)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.CircleArrowRight"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				.FillWidth(1.0f)
				.Padding(2, 0, 0, 0)
				[
					SAssignNew(NavigationBar, SNavigationBar)
					.OnPathClicked(this, &SAssetDialog::SetCurrentlySelectedPath, EContentBrowserPathType::Virtual)
					.OnNavigateToPath(this, &SAssetDialog::OnPathTextEdited)
					.OnGetEditPathAsText(this, &SAssetDialog::OnGetEditPathAsText)
					.OnCompletePrefix(this, &SAssetDialog::OnCompletePathPrefix)
					.GetPathMenuContent(this, &SAssetDialog::OnGetCrumbDelimiterContent)
					.GetComboOptions(this, &SAssetDialog::GetRecentPaths)
				]
			]
		];
	
	// Path/Asset view
	MainVerticalBox->AddSlot()
		.FillHeight(1)
		.Padding(0, 0, 0, 4)
		[
			SNew(SSplitter)
		
			+SSplitter::Slot()
			.Value(0.25f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					PathPicker.ToSharedRef()
				]
			]

			+SSplitter::Slot()
			.Value(0.75f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					AssetPicker.ToSharedRef()
				]
			]
		];

	// Input error strip, if we are using a name box
	if (bIncludeNameBox)
	{
		// Name Error label
		MainVerticalBox->AddSlot()
		.AutoHeight()
		[
			SNew(SWarningOrErrorBox)
			.Padding(FMargin(8.0f, 4.0f, 4.0f, 4.0f))
			.IconSize(FVector2D(16,16))
			.MessageStyle(EMessageStyle::Error)
			.Message(this, &SAssetDialog::GetNameErrorLabelText)
			.Visibility(this, &SAssetDialog::GetNameErrorLabelVisibility)
		];
	}

	TSharedRef<SVerticalBox> LabelsBox = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1)
		.VAlign(VAlign_Center)
		.Padding(0, 4, 0, 4)
		[
			SNew(STextBlock).Text(LOCTEXT("PathBoxLabel", "Path:"))
		];

	TSharedRef<SVerticalBox> ContentBox = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1)
		.VAlign(VAlign_Center)
		.Padding(0, 4, 0, 4)
		[
			SAssignNew(PathText, STextBlock)
			.Text(this, &SAssetDialog::GetPathNameText)
		];

	if (bIncludeNameBox)
	{
		LabelsBox->AddSlot()
			.FillHeight(1)
			.VAlign(VAlign_Center)
			.Padding(0, 4, 0, 4)
			[
				SNew(STextBlock).Text(LOCTEXT("NameBoxLabel", "Name:"))
			];

		ContentBox->AddSlot()
			.AutoHeight() 
			.VAlign(VAlign_Center)
			.Padding(0, 0, 0, 0)
			[
				SAssignNew(NameEditableText, SEditableTextBox)
				.Text(this, &SAssetDialog::GetAssetNameText)
				.OnTextCommitted(this, &SAssetDialog::OnAssetNameTextCommited)
				.OnTextChanged(this, &SAssetDialog::OnAssetNameTextCommited, ETextCommit::Default)
				.SelectAllTextWhenFocused(true)
			];
	}

	// Buttons and asset name
	TSharedRef<SHorizontalBox> ButtonsAndNameBox = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		.Padding(0.f, 0.f, 4.f, 0.f)
		[
			LabelsBox
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		.VAlign(VAlign_Bottom)
		.Padding(4.f, 0.f) 
		[
			ContentBox
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Bottom)
		.Padding(4.f, 0.f) 
		[
			SNew(SPrimaryButton)
			.Text(ConfirmButtonText)
			.IsEnabled(this, &SAssetDialog::IsConfirmButtonEnabled)
			.OnClicked(this, &SAssetDialog::OnConfirmClicked)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Bottom)
		.Padding(4.f, 0.f, 0.0f, 0.0f) 
		[
			SNew(SButton)
			.TextStyle(FAppStyle::Get(), "DialogButtonText")
			.Text(LOCTEXT("AssetDialogCancelButton", "Cancel"))
			.OnClicked(this, &SAssetDialog::OnCancelClicked)
		];

	MainVerticalBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(16.f, 4.f, 16.f, 16.f)
		[
			ButtonsAndNameBox
		];

	ChildSlot
	[
		MainVerticalBox
	];

	// Sync path selection with the path picker
	if (SelectedVirtualPaths.Num() > 0)
	{
		SetCurrentlySelectedPath(SelectedVirtualPaths[0], EContentBrowserPathType::Virtual);
	}
	else
	{
		SetCurrentlySelectedPath(DefaultPath, EContentBrowserPathType::Internal);
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FReply SAssetDialog::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if ( InKeyEvent.GetKey() == EKeys::Escape )
	{
		CloseDialog();
		return FReply::Handled();
	}
	else
	{
		if (FSlateApplication::Get().HasFocusedDescendants(PathPicker->GetPathView().ToSharedRef()))
		{
			if (PathViewCommands->ProcessCommandBindings(InKeyEvent))
			{
				return FReply::Handled();
			}
		}
		else if (FSlateApplication::Get().HasFocusedDescendants(AssetPicker->GetAssetView().ToSharedRef()))
		{
			if (AssetViewCommands->ProcessCommandBindings(InKeyEvent))
			{
				return FReply::Handled();
			}
		}
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

FReply SAssetDialog::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Mouse back and forward buttons traverse history
	if (MouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton)
	{
		HistoryManager.GoBack();
		return FReply::Handled();
	}
	if (MouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton2)
	{
		HistoryManager.GoForward();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SAssetDialog::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	// Mouse back and forward buttons traverse history
	if (InMouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton)
	{
		HistoryManager.GoBack();
		return FReply::Handled();
	}
	if (InMouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton2)
	{
		HistoryManager.GoForward();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SAssetDialog::BindCommands()
{
	auto MapActions = [this](EAssetDialogCommandContext CommandContext, TSharedPtr<FUICommandList>& OutCommands)
	{
		OutCommands = MakeShared<FUICommandList>();

		OutCommands->MapAction(FGenericCommands::Get().Rename, FUIAction(
			FExecuteAction::CreateSP(this, &SAssetDialog::ExecuteRename, CommandContext),
			FCanExecuteAction::CreateSP(this, &SAssetDialog::CanExecuteRename, CommandContext)
		));

		OutCommands->MapAction(FGenericCommands::Get().Delete, FUIAction(
			FExecuteAction::CreateSP(this, &SAssetDialog::ExecuteDelete, CommandContext),
			FCanExecuteAction::CreateSP(this, &SAssetDialog::CanExecuteDelete, CommandContext)
		));

		OutCommands->MapAction(FContentBrowserCommands::Get().CreateNewFolder, FUIAction(
			FExecuteAction::CreateSP(this, &SAssetDialog::ExecuteCreateNewFolder, CommandContext),
			FCanExecuteAction::CreateSP(this, &SAssetDialog::CanExecuteCreateNewFolder, CommandContext)
		));
	};

	MapActions(EAssetDialogCommandContext::AssetView, AssetViewCommands);
	MapActions(EAssetDialogCommandContext::PathView, PathViewCommands);
}

bool SAssetDialog::CanExecuteRename(EAssetDialogCommandContext CommandContext) const
{
	switch (CommandContext)
	{
		case EAssetDialogCommandContext::AssetView: return ContentBrowserUtils::CanRenameFromAssetView(AssetPicker->GetAssetView());
		case EAssetDialogCommandContext::PathView: return ContentBrowserUtils::CanRenameFromPathView(PathPicker->GetPathView());
	}

	return false;
}

void SAssetDialog::ExecuteRename(EAssetDialogCommandContext CommandContext)
{
	switch (CommandContext)
	{
	case EAssetDialogCommandContext::AssetView:
	{
		const TArray<FContentBrowserItem> SelectedItems = AssetPicker->GetAssetView()->GetSelectedItems();
		if (SelectedItems.Num() == 1)
		{
			AssetPicker->GetAssetView()->RenameItem(SelectedItems[0]);
		}
		break;
	}
	case EAssetDialogCommandContext::PathView:
	{
		const TArray<FContentBrowserItem> SelectedFolders = PathPicker->GetPathView()->GetSelectedFolderItems();
		if (SelectedFolders.Num() == 1)
		{
			PathPicker->GetPathView()->RenameFolderItem(SelectedFolders[0]);
		}
		break;
	}
	}
}

bool SAssetDialog::CanExecuteDelete(EAssetDialogCommandContext CommandContext) const
{
	switch (CommandContext)
	{
		case EAssetDialogCommandContext::AssetView: return ContentBrowserUtils::CanDeleteFromAssetView(AssetPicker->GetAssetView());
		case EAssetDialogCommandContext::PathView: return ContentBrowserUtils::CanDeleteFromPathView(PathPicker->GetPathView());
	}

	return false;
}

void SAssetDialog::ExecuteDelete(EAssetDialogCommandContext CommandContext)
{
	// Don't allow asset deletion during PIE
	if (GIsEditor)
	{
		UEditorEngine* Editor = GEditor;
		FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext();
		if (PIEWorldContext)
		{
			FNotificationInfo Notification(LOCTEXT("CannotDeleteAssetInPIE", "Assets cannot be deleted while in PIE."));
			Notification.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Notification);
			return;
		}
	}

	if (CommandContext != EAssetDialogCommandContext::PathView)
	{
		const TArray<FContentBrowserItem> SelectedFiles = AssetPicker->GetAssetView()->GetSelectedFileItems();

		// Batch these by their data sources
		TMap<UContentBrowserDataSource*, TArray<FContentBrowserItemData>> SourcesAndItems;
		for (const FContentBrowserItem& SelectedItem : SelectedFiles)
		{
			FContentBrowserItem::FItemDataArrayView ItemDataArray = SelectedItem.GetInternalItems();
			for (const FContentBrowserItemData& ItemData : ItemDataArray)
			{
				if (UContentBrowserDataSource* ItemDataSource = ItemData.GetOwnerDataSource())
				{
					FText DeleteErrorMsg;
					if (ItemDataSource->CanDeleteItem(ItemData, &DeleteErrorMsg))
					{
						TArray<FContentBrowserItemData>& ItemsForSource = SourcesAndItems.FindOrAdd(ItemDataSource);
						ItemsForSource.Add(ItemData);
					}
					else
					{
						AssetViewUtils::ShowErrorNotifcation(DeleteErrorMsg);
					}
				}
			}
		}

		// Execute the operation now
		for (const auto& SourceAndItemsPair : SourcesAndItems)
		{
			SourceAndItemsPair.Key->BulkDeleteItems(SourceAndItemsPair.Value);
		}
	}

	// List selected folders that can be deleted
	FText FirstFolderDisplayName;
	TArray<FString> SelectedFolderInternalPaths;
	{
		const TArray<FContentBrowserItem> SelectedFolderItems = (CommandContext == EAssetDialogCommandContext::PathView) ?
			PathPicker->GetPathView()->GetSelectedFolderItems() :
			AssetPicker->GetAssetView()->GetSelectedFolderItems();

		for (const FContentBrowserItem& SelectedItem : SelectedFolderItems)
		{
			if (SelectedItem.CanDelete())
			{
				// Only internal folders supported currently
				const FName ConvertedPath = SelectedItem.GetInternalPath();
				if (!ConvertedPath.IsNone())
				{
					if (SelectedFolderInternalPaths.Num() == 0)
					{
						FirstFolderDisplayName = SelectedItem.GetDisplayName();
					}

					SelectedFolderInternalPaths.Add(ConvertedPath.ToString());
				}
			}
		}
	}

	if (SelectedFolderInternalPaths.Num() > 0)
	{
		const bool bResetSelection = CommandContext == EAssetDialogCommandContext::PathView;
		ExecuteDeleteFolderConfirmed(SelectedFolderInternalPaths, bResetSelection);
	}
}

FReply SAssetDialog::ExecuteDeleteFolderConfirmed(const TArray<FString> SelectedFolderInternalPaths, const bool bResetSelection)
{
	if (SelectedFolderInternalPaths.Num() > 0)
	{
		if (ContentBrowserUtils::DeleteFolders(SelectedFolderInternalPaths))
		{
			if (bResetSelection)
			{
				// Since the contents of the asset view have just been deleted, set the default selected paths
				SelectDefaultPaths();
			}
		}
	}

	return FReply::Handled();
}

void SAssetDialog::SelectDefaultPaths()
{
	const TArray<FName> DefaultVirtualPathsToSelect = PathPicker->GetPathView()->GetDefaultPathsToSelect();

	TArray<FString> DefaultSelectedPaths;
	DefaultSelectedPaths.Reserve(DefaultVirtualPathsToSelect.Num());
	for (const FName DefaultVirtualPathToSelect : DefaultVirtualPathsToSelect)
	{
		DefaultSelectedPaths.Add(DefaultVirtualPathToSelect.ToString());
	}

	PathPicker->GetPathView()->SetSelectedPaths(DefaultSelectedPaths);

	FAssetViewContentSources DefaultContentSources(DefaultVirtualPathsToSelect);
	AssetPicker->GetAssetView()->SetContentSources(DefaultContentSources);
}

void SAssetDialog::ExecuteExplore(EAssetDialogCommandContext Widget)
{
	TArray<FContentBrowserItem> SelectedItems;
	TSharedPtr<SWidget> ParentContent;
	switch (Widget)
	{
	case EAssetDialogCommandContext::AssetView:
		SelectedItems = AssetPicker->GetAssetView()->GetSelectedItems();
		ParentContent = AssetPicker->GetAssetView();
		break;
	case EAssetDialogCommandContext::PathView:
		SelectedItems = PathPicker->GetPathView()->GetSelectedFolderItems();
		ParentContent = PathPicker->GetPathView();
		break;
	default:
		return;
	}
	ContentBrowserUtils::ExploreFolders(SelectedItems, ParentContent.ToSharedRef());
}

bool SAssetDialog::CanExecuteExplore(EAssetDialogCommandContext Widget)
{
	TArray<FContentBrowserItem> SelectedItems;
	switch (Widget)
	{
	case EAssetDialogCommandContext::AssetView:
		SelectedItems = AssetPicker->GetAssetView()->GetSelectedItems();
		break;
	case EAssetDialogCommandContext::PathView:
		SelectedItems = PathPicker->GetPathView()->GetSelectedFolderItems();
		break;
	default:
		return false;
	}
	return ContentBrowserUtils::CanExploreFolders(SelectedItems);
}

FContentBrowserItem SAssetDialog::GetCreateNewFolderParent(EAssetDialogCommandContext CommandContext) const
{
	if (CommandContext == EAssetDialogCommandContext::AssetView)
	{
		TArray<FContentBrowserItem> SelectedItems = AssetPicker->GetAssetView()->GetSelectedItems();
		if (!SelectedItems.IsEmpty())
		{
			if (SelectedItems.Num() == 1 && SelectedItems[0].IsFolder())
			{
				return SelectedItems[0];
			}
			return {};
		}
		// If nothing is selected in the AssetView, use the parent folder from the PathView.
	}

	TArray<FContentBrowserItem> SelectedFolders = PathPicker->GetPathView()->GetSelectedFolderItems();
	// We can only create folders when we have a single path selected
	if (SelectedFolders.Num() == 1)
	{
		return SelectedFolders[0];
	}

	return {};
}

bool SAssetDialog::CanExecuteCreateNewFolder(EAssetDialogCommandContext CommandContext) const
{
	const FContentBrowserItem CreateNewFolderParent = GetCreateNewFolderParent(CommandContext);
	if (!CreateNewFolderParent.IsValid())
	{
		return false;
	}

	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	return ContentBrowserData->CanCreateFolder(CreateNewFolderParent.GetVirtualPath(), nullptr);
}

void SAssetDialog::ExecuteCreateNewFolder(EAssetDialogCommandContext CommandContext)
{
	const FContentBrowserItem CreateNewFolderParent = GetCreateNewFolderParent(CommandContext);
	if (!CreateNewFolderParent.IsValid())
	{
		return;
	}

	FOnCreateNewFolder OnCreateNewFolder;
	switch (CommandContext)
	{
	case EAssetDialogCommandContext::AssetView:
		OnCreateNewFolder = FOnCreateNewFolder::CreateSP(AssetPicker->GetAssetView().ToSharedRef(), &SAssetView::NewFolderItemRequested);
		break;
	case EAssetDialogCommandContext::PathView:
		OnCreateNewFolder = FOnCreateNewFolder::CreateSP(PathPicker->GetPathView().ToSharedRef(), &SPathView::NewFolderItemRequested);
		break;
	}

	PathPicker->CreateNewFolder(CreateNewFolderParent.GetVirtualPath().ToString(), OnCreateNewFolder);
}

TSharedPtr<SWidget> SAssetDialog::OnGetFolderContextMenu(const TArray<FString>& SelectedPaths, FContentBrowserMenuExtender_SelectedPaths InMenuExtender, FOnCreateNewFolder InOnCreateNewFolder, EAssetDialogCommandContext CommandContext)
{
	TSharedPtr<const FUICommandList> Commands;
	switch (CommandContext)
	{
	case EAssetDialogCommandContext::AssetView:
		Commands = AssetViewCommands;
		break;
	case EAssetDialogCommandContext::PathView:
		Commands = PathViewCommands;
		break;
	}

	TSharedPtr<FExtender> Extender;
	if (InMenuExtender.IsBound())
	{
		Extender = InMenuExtender.Execute(SelectedPaths);
	}

	FMenuBuilder MenuBuilder(true /*bInShouldCloseWindowAfterMenuSelection*/, Commands, Extender);
	SetupContextMenuContent(CommandContext, MenuBuilder);

	return MenuBuilder.MakeWidget();
}

TSharedPtr<SWidget> SAssetDialog::OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets)
{
	FMenuBuilder MenuBuilder(true /*bInShouldCloseWindowAfterMenuSelection*/, AssetViewCommands);

	SetupContextMenuContent(EAssetDialogCommandContext::AssetView, MenuBuilder);

	return MenuBuilder.MakeWidget();
}

void SAssetDialog::SetupContextMenuContent(EAssetDialogCommandContext CommandContext, FMenuBuilder& MenuBuilder)
{
	const FName ContentBrowserStyleSetName = UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetStyleSetName();

	FText NewFolderToolTip;

	const FContentBrowserItem CreateNewFolderParent = GetCreateNewFolderParent(CommandContext);
	if (CreateNewFolderParent.IsValid())
	{
		if (CanExecuteCreateNewFolder(CommandContext))
		{
			NewFolderToolTip = FText::Format(
				LOCTEXT("NewFolderTooltip_CreateIn", "Create a new folder in {0}."),
				IContentBrowserDataModule::Get().GetSubsystem()->ConvertVirtualPathToDisplay(CreateNewFolderParent));
		}
		else
		{
			NewFolderToolTip = FText::Format(
				LOCTEXT("NewFolderTooltip_InvalidPath", "Cannot create new folders in {0}."),
				IContentBrowserDataModule::Get().GetSubsystem()->ConvertVirtualPathToDisplay(CreateNewFolderParent));
		}
	}
	else
	{
		NewFolderToolTip = FText(LOCTEXT("NewFolderTooltip_InvalidAction", "Can only create folders when there is a single path selected."));
	}

	MenuBuilder.BeginSection("AssetDialogOptions", LOCTEXT("AssetDialogMenuHeading", "Options"));

	MenuBuilder.AddMenuEntry(FContentBrowserCommands::Get().CreateNewFolder, NAME_None, LOCTEXT("NewFolder", "New Folder"), NewFolderToolTip, FSlateIcon(ContentBrowserStyleSetName, "ContentBrowser.NewFolderIcon"));
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename, NAME_None, LOCTEXT("RenameFolder", "Rename"), LOCTEXT("RenameFolderTooltip", "Rename the selected folder."), FSlateIcon(ContentBrowserStyleSetName, "ContentBrowser.AssetActions.Rename"));
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete, NAME_None, LOCTEXT("DeleteFolder", "Delete"), LOCTEXT("DeleteFolderTooltip", "Removes this folder and all assets it contains."));

	MenuBuilder.EndSection();

	if (CanExecuteExplore(CommandContext))
	{
		MenuBuilder.BeginSection("AssetDialogExplore", LOCTEXT("AssetDialogExploreHeading", "Explore"));
		MenuBuilder.AddMenuEntry(ContentBrowserUtils::GetExploreFolderText(),
			LOCTEXT("ExploreTooltip", "Finds this folder on disk."),
			FSlateIcon(ContentBrowserStyleSetName, "SystemWideCommands.FindInContentBrowser"),
			FUIAction(FExecuteAction::CreateSP(this, &SAssetDialog::ExecuteExplore, CommandContext)));
		MenuBuilder.EndSection();
	}
}

EActiveTimerReturnType SAssetDialog::SetFocusPostConstruct( double InCurrentTime, float InDeltaTime )
{
	FocusNameBox();
	return EActiveTimerReturnType::Stop;
}

void SAssetDialog::SetOnAssetsChosenForOpen(const FOnAssetsChosenForOpen& InOnAssetsChosenForOpen)
{
	OnAssetsChosenForOpen = InOnAssetsChosenForOpen;
}

void SAssetDialog::SetOnObjectPathChosenForSave(const FOnObjectPathChosenForSave& InOnObjectPathChosenForSave)
{
	OnObjectPathChosenForSave = InOnObjectPathChosenForSave;
}

void SAssetDialog::SetOnAssetDialogCancelled(const FOnAssetDialogCancelled& InOnAssetDialogCancelled)
{
	OnAssetDialogCancelled = InOnAssetDialogCancelled;
}

void SAssetDialog::FocusNameBox()
{
	if ( NameEditableText.IsValid() )
	{
		FSlateApplication::Get().SetKeyboardFocus(NameEditableText.ToSharedRef(), EFocusCause::SetDirectly);
	}
}

bool SAssetDialog::IsBackEnabled() const
{
	return HistoryManager.CanGoBack();
}

bool SAssetDialog::IsForwardEnabled() const
{
	return HistoryManager.CanGoForward();
}

FText SAssetDialog::GetBackTooltip() const
{
	if (HistoryManager.CanGoBack())
	{
		return FText::Format(LOCTEXT("HistoryBackTooltipFmt", "Back to {0}"), HistoryManager.GetBackDesc());
	}
	return FText::GetEmpty();
}

FText SAssetDialog::GetForwardTooltip() const
{
	if (HistoryManager.CanGoForward())
	{
		return FText::Format(LOCTEXT("HistoryForwardTooltipFmt", "Forward to {0}"), HistoryManager.GetForwardDesc());
	}
	return FText::GetEmpty();
}

FReply SAssetDialog::OnBackClicked()
{
	HistoryManager.GoBack();
	return FReply::Handled();
}

FReply SAssetDialog::OnForwardClicked()
{
	HistoryManager.GoForward();
	return FReply::Handled();
}

void SAssetDialog::OnApplyHistoryData(const FHistoryData& History)
{
	PathPicker->GetPathView()->ApplyHistoryData(History);
	AssetPicker->GetAssetView()->ApplyHistoryData(History);

	ContentBrowserUtils::UpdateNavigationBar(NavigationBar.ToSharedRef(), AssetPicker->GetAssetView().ToSharedRef(), PathPicker->GetPathView().ToSharedRef());
}

void SAssetDialog::OnUpdateHistoryData(FHistoryData& HistoryData) const
{
	if (!AssetPicker)
	{
		return;
	}

	const FAssetViewContentSources& ContentSources = AssetPicker->GetAssetView()->GetContentSources();
	const TArray<FContentBrowserItem> SelectedItems = AssetPicker->GetAssetView()->GetSelectedItems();

	const FText NewSource = ContentSources.HasVirtualPaths() ?
		IContentBrowserDataModule::Get().GetSubsystem()->ConvertVirtualPathToDisplay(ContentSources.GetVirtualPaths()[0], EContentBrowserItemTypeFilter::IncludeFolders) :
		(ContentSources.HasCollections() ? FText::FromName(ContentSources.GetCollections()[0].Name) : LOCTEXT("AllAssets", "All Assets"));

	HistoryData.HistoryDesc = NewSource;
	HistoryData.ContentSources = ContentSources;

	HistoryData.SelectionData.Reset();
	for (const FContentBrowserItem& SelectedItem : SelectedItems)
	{
		HistoryData.SelectionData.SelectedVirtualPaths.Add(SelectedItem.GetVirtualPath());
	}
}

FText SAssetDialog::GetAssetNameText() const
{
	return FText::FromString(CurrentlyEnteredAssetName);
}

FText SAssetDialog::GetPathNameText() const
{
	return IContentBrowserDataModule::Get().GetSubsystem()->ConvertVirtualPathToDisplay(GetCurrentSelectedVirtualPath(), EContentBrowserItemTypeFilter::IncludeFolders);
}

void SAssetDialog::OnAssetNameTextCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	SetCurrentlyEnteredAssetName(InText.ToString());

	if ( InCommitType == ETextCommit::OnEnter )
	{
		CommitObjectPathForSave();
	}
}

EVisibility SAssetDialog::GetNameErrorLabelVisibility() const
{
	return GetNameErrorLabelText().IsEmpty() ? EVisibility::Hidden : EVisibility::Visible;
}

FText SAssetDialog::GetNameErrorLabelText() const
{
	if (!bLastInputValidityCheckSuccessful)
	{
		return LastInputValidityErrorText;
	}

	return FText::GetEmpty();
}

FText SAssetDialog::OnGetEditPathAsText(const FString& Path) const
{
	const FAssetViewContentSources& ContentSources = AssetPicker->GetAssetView()->GetContentSources();
	if (ContentSources.HasCollections())
	{
		return FText::GetEmpty();
	}
	else if (ContentSources.HasVirtualPaths())
	{
		return ContentBrowserUtils::GetUserFacingPathFromVirtualPath(FName(Path));
	}
	return FText::GetEmpty();
}

void SAssetDialog::OnPathTextEdited(const FString& NewPath)
{
	FContentBrowserItem Item = ContentBrowserUtils::TryGetItemFromUserProvidedPath(NewPath);
	if (Item.IsValid())
	{
		SyncToItems({ Item });
	}
}

TArray<FNavigationBarComboOption> SAssetDialog::OnCompletePathPrefix(const FString& Prefix) const
{
	FStringView PrefixView = Prefix;
	
	// Strip to last path separator, but keep the very first path separator
	if (int32 Index = UE::String::FindLastChar(PrefixView, '/'); Index != INDEX_NONE)
	{	
		PrefixView.LeftInline(FMath::Max(1, Index));
	}

	// Find PrefixView in the available tree of data sources, get its direct children, and filter them by SuffixView
	TArray<FString> ChildPaths = ContentBrowserUtils::GetChildPathsFromUserProvidedPath(
		PrefixView,
		PathPicker->GetPathView()->GetContentBrowserItemCategoryFilter(),
		PathPicker->GetPathView()->GetContentBrowserItemAttributeFilter(),
		NAME_None,
		*PathPicker->GetPathView()
	);
	
	TArray<FNavigationBarComboOption> Results;
	for (const FString& ChildPath : ChildPaths)
	{
		Results.Emplace(FText::AsCultureInvariant(ChildPath), ChildPath);
	}
	return Results;
}

TSharedRef<SWidget> SAssetDialog::OnGetCrumbDelimiterContent(const FString& CrumbData) const
{
	TArray<FContentBrowserItem> SubItems = ContentBrowserUtils::GetChildItemsFromVirtualPath(
		*CrumbData,
		PathPicker->GetPathView()->GetContentBrowserItemCategoryFilter(),
		PathPicker->GetPathView()->GetContentBrowserItemAttributeFilter(),
		NAME_None,
		*PathPicker->GetPathView()
	);

	SubItems.Sort([](const FContentBrowserItem& ItemOne, const FContentBrowserItem& ItemTwo)
	{
		return ItemOne.GetDisplayName().CompareTo(ItemTwo.GetDisplayName()) < 0;
	});

	if (SubItems.Num() > 0)
	{
		FMenuBuilder MenuBuilder( true, nullptr );

		for (const FContentBrowserItem& SubItem : SubItems)
		{
			FName FolderBrushName = NAME_None;
			FName FolderShadowBrushName = NAME_None;
			ContentBrowserUtils::TryGetFolderBrushAndShadowNameSmall(SubItem, FolderBrushName, FolderShadowBrushName);

			FText EntryName = SubItem.GetDisplayName();
			FUIAction EntryAction = FUIAction(FExecuteAction::CreateSP(const_cast<SAssetDialog*>(this), &SAssetDialog::OnCrumbDelimiterItemClicked, SubItem.GetVirtualPath().ToString()));

			if (FolderBrushName != NAME_None)
			{
				FLinearColor FolderColor = UE::Editor::ContentBrowser::ExtensionUtils::GetFolderColor(SubItem).Get(ContentBrowserUtils::GetDefaultColor());
				
				FMenuEntryParams Params;
				Params.EntryWidget = ContentBrowserUtils::GetFolderWidgetForNavigationBar(EntryName, FolderBrushName, FolderColor);
				Params.DirectActions = EntryAction;
				MenuBuilder.AddMenuEntry(Params);
			}
			else
			{
				MenuBuilder.AddMenuEntry(
					EntryName,
					FText::GetEmpty(),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), FolderBrushName),
					EntryAction
				);
			}
		}

		return SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.MaxHeight(400.0f)
			[
				MenuBuilder.MakeWidget()
			];
	}

	return SNullWidget::NullWidget;
}

void SAssetDialog::OnCrumbDelimiterItemClicked(FString ClickedPath)
{
	SetCurrentlySelectedPath(ClickedPath, EContentBrowserPathType::Virtual);
}

TArray<FNavigationBarComboOption> SAssetDialog::GetRecentPaths() const
{
	TArray<FNavigationBarComboOption> RecentPaths;
	RecentPaths.Reserve(RecentDirectories.Num());
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	for (const FString& RecentDirectory : RecentDirectories)
	{
		FContentBrowserItem Item = ContentBrowserData->GetItemAtPath(FName(RecentDirectory), EContentBrowserItemTypeFilter::IncludeFolders);
		if (Item.IsValid())
		{
			RecentPaths.Emplace(ContentBrowserData->ConvertVirtualPathToDisplay(Item), RecentDirectory);
		}
	}
	return RecentPaths;
}

void SAssetDialog::SetCurrentlySelectedPath(const FString& NewPath, const EContentBrowserPathType InPathType)
{
	CurrentlySelectedPath = NewPath;
	CurrentlySelectedPathType = InPathType;

	const FName NewVirtualPath = InPathType == EContentBrowserPathType::Virtual ? FName(NewPath) : IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual(*NewPath);
	
	// Update Path View
	if (PathPicker)
	{
		TArray<FString> SelectedVirtualPaths = PathPicker->GetPaths();
		if (SelectedVirtualPaths.Num() == 0 || NewVirtualPath != *SelectedVirtualPaths[0])
		{
			SetPathsDelegate.Execute({ NewVirtualPath.ToString() });
		}
	}

	if (AssetPicker)
	{
		// The asset picker uses the FARFilter::PackagePaths field to fill content sources.
		// Thus, content sources must be checked rather than the filter itself.
		const FAssetViewContentSources& Sources = AssetPicker->GetAssetView()->GetContentSources();
		if (!Sources.HasVirtualPaths() || Sources.GetVirtualPaths()[0] != NewVirtualPath)
		{
			FARFilter NewFilter;

			NewFilter.ClassPaths.Append(AssetClassNames);
			NewFilter.PackagePaths.Add(NewVirtualPath);

			SetFilterDelegate.Execute(NewFilter);
		}
	}
	
	// Update Navigation Bar
	if (NavigationBar && AssetPicker && PathPicker)
	{
		// Calling this function will invalidate the const-ref parameter NewPath
		ContentBrowserUtils::UpdateNavigationBar(NavigationBar, AssetPicker->GetAssetView(), PathPicker->GetPathView());
	}
	
	UpdateInputValidity();

	HistoryManager.AddHistoryData();
	RecentDirectories.AddUnique(InPathType == EContentBrowserPathType::Virtual ? CurrentlySelectedPath : NewVirtualPath.ToString());

	OnPathSelected.ExecuteIfBound(NewPath);
}

void SAssetDialog::SyncToItems(TArrayView<const FContentBrowserItem> ItemsToSync, const bool bAllowImplicitSync)
{
	TArray<FContentBrowserItem> NewItemsToSync = ContentBrowserUtils::FilterOrAliasItems(ItemsToSync, *PathPicker->GetPathView());
	ItemsToSync = NewItemsToSync;
	
	// Tell the sources view first so the asset view will be up to date by the time we request the sync
	PathPicker->SyncToItems(ItemsToSync, bAllowImplicitSync);
	AssetPicker->SyncToItems(ItemsToSync, bAllowImplicitSync, true);
}

bool SAssetDialog::IsConfirmButtonEnabled() const
{
	switch (DialogType)
	{
	case EAssetDialogType::Open:
		return GetCurrentSelectionDelegate.Execute().Num() > 0;
	case EAssetDialogType::Save:
		return bLastInputValidityCheckSuccessful;
	default:
	    ensureMsgf(0, TEXT("AssetDialog type %d is not supported."), DialogType);
	    return false;
	}
}

FReply SAssetDialog::OnConfirmClicked()
{
	switch (DialogType)
	{
	case EAssetDialogType::Open:
	{
		TArray<FAssetData> SelectedAssets = GetCurrentSelectionDelegate.Execute();
		if (SelectedAssets.Num() > 0)
		{
			ChooseAssetsForOpen(SelectedAssets);
		}
		break;
	}

	case EAssetDialogType::Save:
		// @todo save asset validation (e.g. "asset already exists" check)
		CommitObjectPathForSave();
		break;

	default:
		ensureMsgf(0, TEXT("AssetDialog type %d is not supported."), DialogType);
		break;
	}
	return FReply::Handled();
}

FReply SAssetDialog::OnCancelClicked()
{
	CloseDialog();

	return FReply::Handled();
}

void SAssetDialog::OnAssetSelected(const FAssetData& AssetData)
{
	if ( AssetData.IsValid() )
	{
		SetCurrentlySelectedPath(AssetData.PackagePath.ToString(), EContentBrowserPathType::Internal);
		SetCurrentlyEnteredAssetName(AssetData.AssetName.ToString());
	}
}

void SAssetDialog::OnAssetsActivated(const TArray<FAssetData>& SelectedAssets, EAssetTypeActivationMethod::Type ActivationType)
{
	const bool bCorrectActivationMethod = (ActivationType == EAssetTypeActivationMethod::DoubleClicked || ActivationType == EAssetTypeActivationMethod::Opened);
	if (SelectedAssets.Num() > 0 && bCorrectActivationMethod)
	{
		switch (DialogType)
		{
		case EAssetDialogType::Open:
			ChooseAssetsForOpen(SelectedAssets);
			break;

		case EAssetDialogType::Save:
		{
			const FAssetData& AssetData = SelectedAssets[0];
			SetCurrentlySelectedPath(AssetData.PackagePath.ToString(), EContentBrowserPathType::Internal);
			SetCurrentlyEnteredAssetName(AssetData.AssetName.ToString());
			CommitObjectPathForSave();
			break;
		}

		default:
			ensureMsgf(0, TEXT("AssetDialog type %d is not supported."), DialogType);
			break;
		}
	}
}

void SAssetDialog::CloseDialog()
{
	TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());

	if (ContainingWindow.IsValid())
	{
		ContainingWindow->RequestDestroyWindow();
	}
}

void SAssetDialog::SetCurrentlyEnteredAssetName(const FString& NewName)
{
	CurrentlyEnteredAssetName = NewName;
	UpdateInputValidity();
}

void SAssetDialog::UpdateInputValidity()
{
	bLastInputValidityCheckSuccessful = true;

	if ( CurrentlyEnteredAssetName.IsEmpty() )
	{
		// No error text for an empty name. Just fail validity.
		LastInputValidityErrorText = FText::GetEmpty();
		bLastInputValidityCheckSuccessful = false;
	}

	if (bLastInputValidityCheckSuccessful)
	{
		if ( CurrentlySelectedPath.IsEmpty() )
		{
			LastInputValidityErrorText = LOCTEXT("AssetDialog_NoPathSelected", "You must select a path.");
			bLastInputValidityCheckSuccessful = false;
		}
		else if (CurrentlySelectedPathType == EContentBrowserPathType::Virtual)
		{
			FName ConvertedPath;
			const EContentBrowserPathType ConvertedType = IContentBrowserDataModule::Get().GetSubsystem()->TryConvertVirtualPath(CurrentlySelectedPath, ConvertedPath);

			bool bIsMountedInternalPath = false;
			if (ConvertedType == EContentBrowserPathType::Internal)
			{
				FString CheckPath = ConvertedPath.ToString();
				if (!CheckPath.EndsWith(TEXT("/")))
				{
					CheckPath += TEXT("/");
				}

				if (FPackageName::IsValidPath(CheckPath))
				{
					bIsMountedInternalPath = true;
				}
			}

			if (!bIsMountedInternalPath)
			{
				LastInputValidityErrorText = LOCTEXT("AssetDialog_VirtualPathSelected", "The selected folder cannot be modified.");
				bLastInputValidityCheckSuccessful = false;
			}
		}
	}

	if ( DialogType == EAssetDialogType::Save )
	{
		if ( bLastInputValidityCheckSuccessful )
		{
			const FString ObjectPath = GetObjectPathForSave();
			FText ErrorMessage;
			const bool bAllowExistingAsset = (ExistingAssetPolicy == ESaveAssetDialogExistingAssetPolicy::AllowButWarn);

			FTopLevelAssetPath AssetClassName = AssetClassNames.Num() == 1 ? AssetClassNames[0] : FTopLevelAssetPath();
			UClass* AssetClass = !AssetClassName.IsNull() ? FindObject<UClass>(AssetClassName, EFindObjectFlags::ExactClass) : nullptr;

			if ( !ContentBrowserUtils::IsValidObjectPathForCreate(ObjectPath, AssetClass, ErrorMessage, bAllowExistingAsset) )
			{
				LastInputValidityErrorText = ErrorMessage;
				bLastInputValidityCheckSuccessful = false;
			}
			else if(bAllowExistingAsset && AssetClassNames.Num() > 1) // If for some reason we have multiple names, perform additional logic here...
			{
				FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
				FAssetData ExistingAsset = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPath));
				if (ExistingAsset.IsValid() && !AssetClassNames.Contains(ExistingAsset.AssetClassPath))
				{
					const FString ObjectName = FPackageName::ObjectPathToObjectName(ObjectPath);
					LastInputValidityErrorText = FText::Format(LOCTEXT("AssetDialog_AssetAlreadyExists", "An asset of type '{0}' already exists at this location with the name '{1}'."), FText::FromString(ExistingAsset.AssetClassPath.ToString()), FText::FromString(ObjectName));
					bLastInputValidityCheckSuccessful = false;
				}
			}
		}
	}
}

FName SAssetDialog::GetCurrentSelectedVirtualPath() const
{
	if (CurrentlySelectedPathType == EContentBrowserPathType::Virtual)
	{
		return *CurrentlySelectedPath;
	}
	else
	{
		return IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual(*CurrentlySelectedPath);
	}
}

void SAssetDialog::ChooseAssetsForOpen(const TArray<FAssetData>& SelectedAssets)
{
	if ( ensure(DialogType == EAssetDialogType::Open) )
	{
		if (SelectedAssets.Num() > 0)
		{
			bValidAssetsChosen = true;
			OnAssetsChosenForOpen.ExecuteIfBound(SelectedAssets);
			CloseDialog();
		}
	}
}

FString SAssetDialog::GetObjectPathForSave() const
{
	FString Base = CurrentlySelectedPath;

	if (CurrentlySelectedPathType == EContentBrowserPathType::Virtual)
	{
		FName ConvertedPath;
		const EContentBrowserPathType ConvertedType = IContentBrowserDataModule::Get().GetSubsystem()->TryConvertVirtualPath(CurrentlySelectedPath, ConvertedPath);
		if (ConvertedType == EContentBrowserPathType::Internal)
		{
			Base = ConvertedPath.ToString();
		}
		else
		{
			return FString();
		}
	}

	return Base / CurrentlyEnteredAssetName + TEXT(".") + CurrentlyEnteredAssetName;
}

void SAssetDialog::CommitObjectPathForSave()
{
	if ( ensure(DialogType == EAssetDialogType::Save) )
	{
		if ( bLastInputValidityCheckSuccessful )
		{
			const FString ObjectPath = GetObjectPathForSave();

			bool bProceedWithSave = true;

			// If we were asked to warn on existing assets, do it now
			if ( ExistingAssetPolicy == ESaveAssetDialogExistingAssetPolicy::AllowButWarn )
			{
				FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
				FAssetData ExistingAsset = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPath));
				if ( ExistingAsset.IsValid() && AssetClassNames.Contains(ExistingAsset.AssetClassPath) )
				{
					EAppReturnType::Type ShouldReplace = FMessageDialog::Open( EAppMsgType::YesNo, FText::Format(LOCTEXT("ReplaceAssetMessage", "{0} already exists. Do you want to replace it?"), FText::FromString(CurrentlyEnteredAssetName)) );
					bProceedWithSave = (ShouldReplace == EAppReturnType::Yes);
				}
			}

			if ( bProceedWithSave )
			{
				bValidAssetsChosen = true;
				OnObjectPathChosenForSave.ExecuteIfBound(ObjectPath);
				CloseDialog();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
