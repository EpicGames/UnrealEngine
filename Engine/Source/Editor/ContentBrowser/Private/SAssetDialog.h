// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserDelegates.h"
#include "HistoryManager.h"
#include "HAL/Platform.h"
#include "IAssetTypeActions.h"
#include "IContentBrowserSingleton.h"
#include "SNavigationBar.h"
#include "Containers/MRUArray.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FMenuBuilder;
class FUICommandList;
class SAssetPicker;
class SEditableTextBox;
class SPathPicker;
class STextBlock;
class SWidget;
struct FAssetData;
struct FGeometry;
struct FKeyEvent;
struct FNavigationBarComboOption;
struct FTopLevelAssetPath;

enum class EAssetDialogCommandContext
{
	AssetView,
	PathView,
};

class SAssetDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAssetDialog){}

	SLATE_END_ARGS()

	SAssetDialog();
	virtual ~SAssetDialog();

	virtual void Construct(const FArguments& InArgs, const FSharedAssetDialogConfig& InConfig);

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	/** Sets the delegate handler for when an open operation is committed */
	void SetOnAssetsChosenForOpen(const FOnAssetsChosenForOpen& InOnAssetsChosenForOpen);

	/** Sets the delegate handler for when a save operation is committed */
	void SetOnObjectPathChosenForSave(const FOnObjectPathChosenForSave& InOnObjectPathChosenForSave);

	/** Sets the delegate handler for when the dialog is closed or cancelled */
	void SetOnAssetDialogCancelled(const FOnAssetDialogCancelled& InOnAssetDialogCancelled);

	/** Reset selected paths to default choices */
	void SelectDefaultPaths();

private:

	/** Used to focus the name box immediately following construction */
	EActiveTimerReturnType SetFocusPostConstruct( double InCurrentTime, float InDeltaTime );

	/** Moves keyboard focus to the name box if this is a save dialog */
	void FocusNameBox();


	// --- History Callbacks ---
	bool IsBackEnabled() const;
	bool IsForwardEnabled() const;

	FText GetBackTooltip() const;
	FText GetForwardTooltip() const;

	FReply OnBackClicked();
	FReply OnForwardClicked();

	/** Handles when the history manager shifts to a different history state. */
	void OnApplyHistoryData(const FHistoryData& History);
	/** Fills in the history item with the current state. */
	void OnUpdateHistoryData(FHistoryData& History) const;

	
	/** Gets the name to display in the asset name box */
	FText GetAssetNameText() const;

	/** Gets the name to display in the path text block */
	FText GetPathNameText() const;

	/** Fired when the asset name box text is commited */
	void OnAssetNameTextCommited(const FText& InText, ETextCommit::Type InCommitType);

	/** Gets the visibility of the name error label */
	EVisibility GetNameErrorLabelVisibility() const;

	/** Gets the text to display in the name error label */
	FText GetNameErrorLabelText() const;

	
	// --- Navigation Bar Behavior ---
	/** Returns how the currently shown location should be presented to the user as text when beginning to edit the path */
	FText OnGetEditPathAsText(const FString& Path) const;

	/** Respond to edited text from the navigation bar */
	void OnPathTextEdited(const FString& NewPath);

	/** Provides autocomplete for text-based path editing in the navigation bar */
	TArray<FNavigationBarComboOption> OnCompletePathPrefix(const FString& Prefix) const;
	
	/** Provide content for the navigation bar's breadcrumb menues. */
	TSharedRef<SWidget> OnGetCrumbDelimiterContent(const FString& CrumbData) const;
	void OnCrumbDelimiterItemClicked(FString ClickedPath);

	/** Get a list of recently visited paths for the navigation bar. */
	TArray<FNavigationBarComboOption> GetRecentPaths() const;

	
	// --- Selection ---
	/** Synchronizes all pickers & views to a new path state */
	void SetCurrentlySelectedPath(const FString& NewPath, const EContentBrowserPathType InPathType);

	/** Changes sources to show the specified items and selects them */
	void SyncToItems(TArrayView<const FContentBrowserItem> ItemsToSync, const bool bAllowImplicitSync = false);


	/** Determines if the confirm button (e.g. Open/Save) is enabled. */
	bool IsConfirmButtonEnabled() const;

	/** Handler for when the confirm button (e.g. Open/Save) is clicked */
	FReply OnConfirmClicked();

	/** Handler for when the cancel button is clicked */
	FReply OnCancelClicked();

	/** Handler for when an asset was selected in the asset picker */
	void OnAssetSelected(const FAssetData& AssetData);

	/* Handler for when an asset was double clicked in the asset picker */
	void OnAssetsActivated(const TArray<FAssetData>& SelectedAssets, EAssetTypeActivationMethod::Type ActivationType);


	/** Will generate the context menu if an asset or a folder is selected, either from the PathView or AssetView */
	TSharedPtr<SWidget> OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets);
	TSharedPtr<SWidget> OnGetFolderContextMenu(const TArray<FString>& SelectedPaths, FContentBrowserMenuExtender_SelectedPaths InMenuExtender, FOnCreateNewFolder InOnCreateNewFolder, EAssetDialogCommandContext CommandContext);

	/** Handler to check to see if a rename command is allowed */
	bool CanExecuteRename(EAssetDialogCommandContext CommandContext) const;

	/** Handler for Rename */
	void ExecuteRename(EAssetDialogCommandContext CommandContext);

	/** Handler to check to see if a delete command is allowed */
	bool CanExecuteDelete(EAssetDialogCommandContext CommandContext) const;

	/** Handler for Delete */
	void ExecuteDelete(EAssetDialogCommandContext CommandContext);
	FReply ExecuteDeleteFolderConfirmed(const TArray<FString> SelectedFolderInternalPaths, const bool bResetSelection);

	FContentBrowserItem GetCreateNewFolderParent(EAssetDialogCommandContext CommandContext) const;

	/** Handler to check to see if a create new folder command is allowed */
	bool CanExecuteCreateNewFolder(EAssetDialogCommandContext CommandContext) const;

	/** Handler for creating new folder */
	void ExecuteCreateNewFolder(EAssetDialogCommandContext CommandContext);

	/** Handler for show in explorer */
	void ExecuteExplore(EAssetDialogCommandContext Widget);
	bool CanExecuteExplore(EAssetDialogCommandContext Widget);

	/** Setup function for the context menu creation of folder and assets */
	void SetupContextMenuContent(EAssetDialogCommandContext CommandContext, FMenuBuilder& MenuBuilder);
	
	void BindCommands();

	/** Closes this dialog */
	void CloseDialog();

	void SetCurrentlyEnteredAssetName(const FString& NewName);

	FName GetCurrentSelectedVirtualPath() const;

	void UpdateInputValidity();

	/** Used to commit the assets that were selected for open in this dialog */
	void ChooseAssetsForOpen(const TArray<FAssetData>& SelectedAssets);

	FString GetObjectPathForSave() const;

	/** Used to commit the object path used for saving in this dialog */
	void CommitObjectPathForSave();

private:

	/** Whether this is an open or save dialog */
	EAssetDialogType::Type DialogType;

	/** Tracks the history data for the dialog */
	FHistoryManager HistoryManager;

	/** The most recently visited directories */
	TMRUArray<FString> RecentDirectories;

	/** Used to update the path view after it has been created */
	FSetPathPickerPathsDelegate SetPathsDelegate;

	/** Used to update the asset view after it has been created */
	FSetARFilterDelegate SetFilterDelegate;

	/** Used to get the currently selected assets */
	FGetCurrentSelectionDelegate GetCurrentSelectionDelegate;

	/** Only assets of these classes will show up */
	TArray<FTopLevelAssetPath> AssetClassNames;

	/** Fired when assets are chosen for open. Only fired in open dialogs. */
	FOnAssetsChosenForOpen OnAssetsChosenForOpen;

	/** Fired when an object path was chosen for save. Only fired in save dialogs. */
	FOnObjectPathChosenForSave OnObjectPathChosenForSave;

	/** Fired when the asset dialog is cancelled or closed */
	FOnAssetDialogCancelled OnAssetDialogCancelled;

	/** The name box. Only used in save dialogs. */
	TSharedPtr<SEditableTextBox> NameEditableText;

	/** The path box. */
	TSharedPtr<STextBlock> PathText;

	/** The object path of the asset to save. Only used in save dialogs. */
	FString CurrentlySelectedPath;

	EContentBrowserPathType CurrentlySelectedPathType = EContentBrowserPathType::None;

	/** The object name of the asset to save. Only used in save dialogs. */
	FString CurrentlyEnteredAssetName;

	/** The behavior when the user chooses an existing asset. Only used in save dialogs. */
	ESaveAssetDialogExistingAssetPolicy::Type ExistingAssetPolicy;

	/** The error text from the last validity check */
	FText LastInputValidityErrorText;

	/** True if the last validity check returned that the class name/path is valid for creation */
	bool bLastInputValidityCheckSuccessful;

	/** Used to focus the name box after opening the dialog */
	bool bPendingFocusNextFrame;

	/** Used to specify that valid assets were chosen */
	bool bValidAssetsChosen;

	/** Commands handled by this widget */
	TSharedPtr<FUICommandList> AssetViewCommands;
	TSharedPtr<FUICommandList> PathViewCommands;

	/** Path Picker used by the dialog */
	TSharedPtr<SPathPicker> PathPicker;

	/** Navigation Bar used by the dialog */
	TSharedPtr<SNavigationBar> NavigationBar;
	
	/** Asset Picker used by the dialog */
	TSharedPtr<SAssetPicker> AssetPicker;

	/** Callback for refreshing content */
	FOnPathSelected OnPathSelected;
};
