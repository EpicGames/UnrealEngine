// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAssetPicker.h"

#include "Algo/Transform.h"
#include "AssetRegistry/AssetData.h"
#include "AssetTextFilter.h"
#include "AssetThumbnail.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "ContentBrowserCommands.h"
#include "ContentBrowserDataFilter.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserStyle.h"
#include "ContentBrowserUtils.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Filters.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "FrontendFilters.h"
#include "HAL/PlatformCrt.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/WidgetPath.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "Misc/FilterCollection.h"
#include "Misc/Paths.h"
#include "PropertyHandle.h"
#include "SAssetSearchBox.h"
#include "SAssetView.h"
#include "SContentBrowser.h"
#include "SFilterList.h"
#include "SlotBase.h"
#include "SourceControlOperations.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/UnrealTemplate.h"
#include "Types/ISlateMetaData.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"

class SWidget;
class UObject;
struct FGeometry;

#define LOCTEXT_NAMESPACE "ContentBrowser"

SAssetPicker::~SAssetPicker()
{
	SaveSettings();
}

void SAssetPicker::Construct( const FArguments& InArgs )
{
	BindCommands();

	OnAssetsActivated = InArgs._AssetPickerConfig.OnAssetsActivated;
	OnAssetsDragged = InArgs._AssetPickerConfig.OnAssetsDragged;
	OnAssetSelected = InArgs._AssetPickerConfig.OnAssetSelected;
	OnAssetDoubleClicked = InArgs._AssetPickerConfig.OnAssetDoubleClicked;
	OnAssetEnterPressed = InArgs._AssetPickerConfig.OnAssetEnterPressed;
	bPendingFocusNextFrame = InArgs._AssetPickerConfig.bFocusSearchBoxWhenOpened;
	DefaultFilterMenuExpansion = InArgs._AssetPickerConfig.DefaultFilterMenuExpansion;
	SaveSettingsName = InArgs._AssetPickerConfig.SaveSettingsName;
	FilterBarSaveSettingsNameOverride = InArgs._AssetPickerConfig.FilterBarSaveSettingsNameOverride;
	OnFolderEnteredDelegate = InArgs._AssetPickerConfig.OnFolderEntered;
	OnGetAssetContextMenu = InArgs._AssetPickerConfig.OnGetAssetContextMenu;
	OnGetFolderContextMenu = InArgs._AssetPickerConfig.OnGetFolderContextMenu;
	OnSearchTextChangedDelegate = InArgs._AssetPickerConfig.OnSearchTextChanged;

	// Break up the incoming filter into a sources data and backend filter.
	TArray<FCollectionRef> CollectionsFilter = InArgs._AssetPickerConfig.CollectionsFilter;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (CollectionsFilter.IsEmpty() && !InArgs._AssetPickerConfig.Collections.IsEmpty())
	{
		// Fall back to the Collections field for backwards compatibility.
		CollectionsFilter.Reserve(InArgs._AssetPickerConfig.Collections.Num());
		Algo::Transform(
			InArgs._AssetPickerConfig.Collections,
			CollectionsFilter,
			[](const FCollectionNameType& Collection) { return FCollectionRef(FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(), Collection); });
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	CurrentContentSources = FAssetViewContentSources(InArgs._AssetPickerConfig.Filter.PackagePaths, MoveTemp(CollectionsFilter));
	CurrentBackendFilter = InArgs._AssetPickerConfig.Filter;
	CurrentBackendFilter.PackagePaths.Reset();

	bAllowRename = InArgs._AssetPickerConfig.bAllowRename;
	
	FOnGetContentBrowserItemContextMenu OnGetItemContextMenu;
	if (OnGetAssetContextMenu.IsBound() || OnGetFolderContextMenu.IsBound())
	{
		OnGetItemContextMenu = FOnGetContentBrowserItemContextMenu::CreateSP(this, &SAssetPicker::GetItemContextMenu);
	}

	if ( InArgs._AssetPickerConfig.bFocusSearchBoxWhenOpened )
	{
		RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SAssetPicker::SetFocusPostConstruct ) );
	}

	for (auto DelegateIt = InArgs._AssetPickerConfig.GetCurrentSelectionDelegates.CreateConstIterator(); DelegateIt; ++DelegateIt)
	{
		if ((*DelegateIt) != nullptr)
		{
			(**DelegateIt) = FGetCurrentSelectionDelegate::CreateSP(this, &SAssetPicker::GetCurrentSelection);
		}
	}

	for(auto DelegateIt = InArgs._AssetPickerConfig.SyncToAssetsDelegates.CreateConstIterator(); DelegateIt; ++DelegateIt)
	{
		if((*DelegateIt) != nullptr)
		{
			(**DelegateIt) = FSyncToAssetsDelegate::CreateSP(this, &SAssetPicker::SyncToAssets);
		}
	}

	for (auto DelegateIt = InArgs._AssetPickerConfig.SetFilterDelegates.CreateConstIterator(); DelegateIt; ++DelegateIt)
	{
		if ((*DelegateIt) != nullptr)
		{
			(**DelegateIt) = FSetARFilterDelegate::CreateSP(this, &SAssetPicker::SetNewBackendFilter);
		}
	}

	for (auto DelegateIt = InArgs._AssetPickerConfig.RefreshAssetViewDelegates.CreateConstIterator(); DelegateIt; ++DelegateIt)
	{
		if ((*DelegateIt) != nullptr)
		{
			(**DelegateIt) = FRefreshAssetViewDelegate::CreateSP(this, &SAssetPicker::RefreshAssetView);
		}
	}

	for (auto DelegateIt = InArgs._AssetPickerConfig.SaveAssetViewSettingsDelegates.CreateConstIterator(); DelegateIt; ++DelegateIt)
	{
		if ((*DelegateIt) != nullptr)
		{
			(**DelegateIt) = FSaveAssetViewSettingsDelegate::CreateSP(this, &SAssetPicker::SaveSettings);
		}
	}

	for (auto DelegateIt = InArgs._AssetPickerConfig.LoadAssetViewSettingsDelegates.CreateConstIterator(); DelegateIt; ++DelegateIt)
	{
		if ((*DelegateIt) != nullptr)
		{
			(**DelegateIt) = FLoadAssetViewSettingsDelegate::CreateSP(this, &SAssetPicker::LoadSettings);
		}
	}

	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	ChildSlot
	[
		VerticalBox
	];

	TAttribute< FText > HighlightText;
	EThumbnailLabel::Type ThumbnailLabel = InArgs._AssetPickerConfig.ThumbnailLabel;

	FrontendFilters = MakeShareable(new FAssetFilterCollectionType());

	TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	if (!InArgs._AssetPickerConfig.bAutohideSearchBar)
	{
		// Search box
		HighlightText = TAttribute< FText >(this, &SAssetPicker::GetHighlightedText);
		HorizontalBox->AddSlot()
		.FillWidth(1.0f)
		[
			SAssignNew( SearchBoxPtr, SAssetSearchBox )
			.HintText(NSLOCTEXT( "ContentBrowser", "SearchBoxHint", "Search Assets" ))
			.OnTextChanged( this, &SAssetPicker::OnSearchBoxChanged )
			.OnTextCommitted( this, &SAssetPicker::OnSearchBoxCommitted )
			.DelayChangeNotificationsWhileTyping( true )
			.OnKeyDownHandler( this, &SAssetPicker::HandleKeyDownFromSearchBox )
		];

		// The 'Other Developers' filter is always on by design.
		HorizontalBox->AddSlot()
		.AutoWidth()
		.Padding(4.f, 0.0f, 0.0f, 0.0f)
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "ToggleButtonCheckBox")
			.ToolTipText(this, &SAssetPicker::GetShowOtherDevelopersToolTip)
			.OnCheckStateChanged(this, &SAssetPicker::HandleShowOtherDevelopersCheckStateChanged)
			.IsChecked(this, &SAssetPicker::GetShowOtherDevelopersCheckState)
			.Padding(4.f)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.ColumnViewDeveloperFolderIcon"))
			]
		];
	}
	else
	{
		HorizontalBox->AddSlot()
		.FillWidth(1.0)
		[
			SNew(SSpacer)
		];
	}


	if (InArgs._AssetPickerConfig.bAddFilterUI)
	{		
		// We create available classes here. These are used to hide away the type filters in the filter list that don't match this list of classes
		TArray<UClass*> FilterClassList;
		for(auto Iter = CurrentBackendFilter.ClassPaths.CreateIterator(); Iter; ++Iter)
		{
			FTopLevelAssetPath ClassName = (*Iter);
			UClass* FilterClass = FindObject<UClass>(ClassName);
			if(FilterClass)
			{
				FilterClassList.AddUnique(FilterClass);
			}
		}		
		
		SAssignNew(FilterListPtr, SFilterList)
			.CommandList(Commands)
			.OnFilterChanged(this, &SAssetPicker::OnFilterChanged)
			.FrontendFilters(FrontendFilters)
			.InitialClassFilters(FilterClassList)
			.FilterBarIdentifier(FName(FilterBarSaveSettingsNameOverride.Get(SaveSettingsName)))
			.ExtraFrontendFilters(InArgs._AssetPickerConfig.ExtraFrontendFilters)
			.DefaultMenuExpansionCategory(ConvertAssetTypeCategoryToAssetCategoryPath(DefaultFilterMenuExpansion).Get(EAssetCategoryPaths::Basic))
			.OnExtendAddFilterMenu(InArgs._AssetPickerConfig.OnExtendAddFilterMenu)
			.bUseSectionsForCustomCategories(InArgs._AssetPickerConfig.bUseSectionsForCustomFilterCategories);
		
		FilterComboButtonPtr = StaticCastSharedRef<SComboButton>(SFilterList::MakeAddFilterButton(FilterListPtr.ToSharedRef()));
		
		HorizontalBox->InsertSlot(0)
		.AutoWidth()
		[
			FilterComboButtonPtr.ToSharedRef()
		];
	}
		
	VerticalBox->AddSlot()
	.AutoHeight()
	.Padding(8.f, 0.f, 8.f, 8.f)
	[
		HorizontalBox
	];

	// "None" button
	if (InArgs._AssetPickerConfig.bAllowNullSelection)
	{
		VerticalBox->AddSlot()
		.AutoHeight()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SButton)
						.ButtonStyle(UE::ContentBrowser::Private::FContentBrowserStyle::Get(), "ContentBrowser.NoneButton" )
						.TextStyle(UE::ContentBrowser::Private::FContentBrowserStyle::Get(), "ContentBrowser.NoneButtonText" )
						.Text( LOCTEXT("NoneButtonText", "( None )") )
						.ToolTipText( LOCTEXT("NoneButtonTooltip", "Clears the asset selection.") )
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.OnClicked(this, &SAssetPicker::OnNoneButtonClicked)
				]

			// Trailing separator
			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0, 0, 0, 4)
				[
					SNew(SSeparator)
						.Orientation(Orient_Horizontal)
				]
		];
	}

	// Asset view
	if (InArgs._AssetPickerConfig.bAddFilterUI)
	{
		VerticalBox->AddSlot()
		.AutoHeight()
		[
			FilterListPtr.ToSharedRef()
		];

		// Use the 'other developer' filter from the filter list widget. 
		HideOtherDevelopersFilter = StaticCastSharedPtr<FFilter_HideOtherDevelopers>(FilterListPtr->GetFrontendFilter(TEXT("HideOtherDevelopersBackend")));
	}
	else
	{
		// Filter UI is off, but the 'other developer' filter is a built-in feature.
		HideOtherDevelopersFilter = MakeShared<FFilter_HideOtherDevelopers>(nullptr, FName(SaveSettingsName));
		HideOtherDevelopersFilter->SetActiveInCollection(HideOtherDevelopersFilter.ToSharedRef(), false, *FrontendFilters);
	}

	// Make game-specific filter
	FOnShouldFilterAsset ShouldFilterAssetDelegate;
	{
		FAssetReferenceFilterContext AssetReferenceFilterContext;
		AssetReferenceFilterContext.AddReferencingAssets(InArgs._AssetPickerConfig.AdditionalReferencingAssets);
		AssetReferenceFilterContext.AddReferencingAssetsFromPropertyHandle(InArgs._AssetPickerConfig.PropertyHandle);

		TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = GEditor ? GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext) : nullptr;
		if (AssetReferenceFilter.IsValid())
		{
			FOnShouldFilterAsset ConfigFilter = InArgs._AssetPickerConfig.OnShouldFilterAsset;
			ShouldFilterAssetDelegate = FOnShouldFilterAsset::CreateLambda([ConfigFilter, AssetReferenceFilter](const FAssetData& AssetData) -> bool {
				if (!AssetReferenceFilter->PassesFilter(AssetData))
				{
					return true;
				}
				if (ConfigFilter.IsBound())
				{
					return ConfigFilter.Execute(AssetData);
				}
				return false;
			});
		}
		else
		{
			ShouldFilterAssetDelegate = InArgs._AssetPickerConfig.OnShouldFilterAsset;
		}
	}

	if (!InArgs._AssetPickerConfig.bAutohideSearchBar)
	{
		TextFilter = MakeShared<FAssetTextFilter>();
	}

	// clang-format off
	VerticalBox->AddSlot()
	.FillHeight(1.f)
	[
		SAssignNew(AssetViewPtr, SAssetView)
		.InitialCategoryFilter(EContentBrowserItemCategoryFilter::IncludeAssets)
		.SelectionMode( InArgs._AssetPickerConfig.SelectionMode )
		.OnShouldFilterAsset(ShouldFilterAssetDelegate)
		.OnNewItemRequested(this, &SAssetPicker::HandleNewItemRequested)
		.OnItemSelectionChanged(this, &SAssetPicker::HandleItemSelectionChanged)
		.OnItemsActivated(this, &SAssetPicker::HandleItemsActivated)
		.OnItemsDragged(this, &SAssetPicker::HandleItemsDragged)
		.OnGetItemContextMenu(OnGetItemContextMenu)
		.OnIsAssetValidForCustomToolTip(InArgs._AssetPickerConfig.OnIsAssetValidForCustomToolTip)
		.OnGetCustomAssetToolTip(InArgs._AssetPickerConfig.OnGetCustomAssetToolTip)
		.OnVisualizeAssetToolTip(InArgs._AssetPickerConfig.OnVisualizeAssetToolTip)
		.OnAssetToolTipClosing(InArgs._AssetPickerConfig.OnAssetToolTipClosing)
		.FrontendFilters(FrontendFilters)
		.TextFilter(TextFilter)
		.ShowRedirectors(InArgs._AssetPickerConfig.bAddFilterUI ? ContentBrowserUtils::ShouldShowRedirectors(FilterListPtr) : false)
		.InitialContentSources(CurrentContentSources)
		.InitialBackendFilter(CurrentBackendFilter)
		.InitialViewType(InArgs._AssetPickerConfig.InitialAssetViewType)
		.InitialAssetSelection(InArgs._AssetPickerConfig.InitialAssetSelection)
		.ShowBottomToolbar(InArgs._AssetPickerConfig.bShowBottomToolbar)
		.OnAssetTagWantsToBeDisplayed(InArgs._AssetPickerConfig.OnAssetTagWantsToBeDisplayed)
		.OnGetCustomSourceAssets(InArgs._AssetPickerConfig.OnGetCustomSourceAssets)
		.AllowDragging( InArgs._AssetPickerConfig.bAllowDragging )
		.CanShowClasses( InArgs._AssetPickerConfig.bCanShowClasses )
		.CanShowFolders( InArgs._AssetPickerConfig.bCanShowFolders )
		.CanShowReadOnlyFolders( InArgs._AssetPickerConfig.bCanShowReadOnlyFolders )
		.ShowPathInColumnView( InArgs._AssetPickerConfig.bShowPathInColumnView)
		.ShowTypeInColumnView( InArgs._AssetPickerConfig.bShowTypeInColumnView)
		.ShowViewOptions(false)  // We control this in the asset picker
		.SortByPathInColumnView( InArgs._AssetPickerConfig.bSortByPathInColumnView)
		.FilterRecursivelyWithBackendFilter( false )
		.CanShowRealTimeThumbnails( InArgs._AssetPickerConfig.bCanShowRealTimeThumbnails )
		.CanShowDevelopersFolder( InArgs._AssetPickerConfig.bCanShowDevelopersFolder )
		.ForceShowEngineContent( InArgs._AssetPickerConfig.bForceShowEngineContent )
		.ForceShowPluginContent( InArgs._AssetPickerConfig.bForceShowPluginContent )
		.HighlightedText( HighlightText )
		.ThumbnailLabel( ThumbnailLabel )
		.AssetShowWarningText( InArgs._AssetPickerConfig.AssetShowWarningText)
		.AssetShowWarningTextJustification( InArgs._AssetPickerConfig.AssetShowWarningTextJustification )
		.AssetShowWarningTextSelectable( InArgs._AssetPickerConfig.AssetShowWarningTextSelectable )
		.AllowFocusOnSync(false)	// Stop the asset view from stealing focus (we're in control of that)
		.HiddenColumnNames(InArgs._AssetPickerConfig.HiddenColumnNames)
		.ListHiddenColumnNames(InArgs._AssetPickerConfig.ListHiddenColumnNames)
		.CustomColumns(InArgs._AssetPickerConfig.CustomColumns)
		.OnColumnVisibilityChanged(InArgs._AssetPickerConfig.OnColumnVisibilityChanged)
		.OnSearchOptionsChanged(this, &SAssetPicker::HandleSearchSettingsChanged)
		.InitialThumbnailSize(InArgs._AssetPickerConfig.InitialThumbnailSize)
		.AssetViewOptionsProfile(InArgs._AssetPickerConfig.AssetViewOptionsProfile)
	];
	// clang-format on

	HorizontalBox->AddSlot()
	.AutoWidth()
	[
		SNew(SComboButton)
		.ContentPadding(0.f)
		.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
		.OnGetMenuContent(AssetViewPtr.ToSharedRef(), &SAssetView::GetViewButtonContent)
		.HasDownArrow(false)
		.ButtonContent()
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
		]	
	];

	InArgs._AssetPickerConfig.OnExtendAssetPickerTopBar.ExecuteIfBound(HorizontalBox);

	LoadSettings();

	if (TextFilter.IsValid())
	{
		bool bClassNamesProvided = (InArgs._AssetPickerConfig.Filter.ClassPaths.Num() != 1);
		TextFilter->SetIncludeClassName(bClassNamesProvided || AssetViewPtr->IsIncludingClassNames());
		TextFilter->SetIncludeAssetPath(AssetViewPtr->IsIncludingAssetPaths());
		TextFilter->SetIncludeCollectionNames(AssetViewPtr->IsIncludingCollectionNames());
	}

	AssetViewPtr->RequestSlowFullListRefresh();
}

void SAssetPicker::ClearSearchText()
{
	if (SearchBoxPtr.IsValid())
	{
		SearchBoxPtr->SetText(FText::GetEmpty());
	}
}

TSharedPtr<SFilterList> SAssetPicker::GetFilterList() const
{
	return FilterListPtr;
}

TSharedPtr<SWidget> SAssetPicker::GetSearchBox() const
{
	return SearchBoxPtr;
}

EActiveTimerReturnType SAssetPicker::SetFocusPostConstruct( double InCurrentTime, float InDeltaTime )
{
	if (SearchBoxPtr.IsValid())
	{
		FWidgetPath WidgetToFocusPath;
		FSlateApplication::Get().GeneratePathToWidgetUnchecked(SearchBoxPtr.ToSharedRef(), WidgetToFocusPath);
		FSlateApplication::Get().SetKeyboardFocus(WidgetToFocusPath, EFocusCause::SetDirectly);
		WidgetToFocusPath.GetWindow()->SetWidgetToFocusOnActivate(SearchBoxPtr);
		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

FReply SAssetPicker::HandleKeyDownFromSearchBox(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// Hide the filter list
	if (FilterComboButtonPtr.IsValid())
	{
		FilterComboButtonPtr->SetIsOpen(false);
	}

	// Up and down move thru the filtered list
	int32 SelectionDelta = 0;

	if (InKeyEvent.GetKey() == EKeys::Up)
	{
		SelectionDelta = -1;
	}
	else if (InKeyEvent.GetKey() == EKeys::Down)
	{
		SelectionDelta = +1;
	}

	if (SelectionDelta != 0)
	{
		AssetViewPtr->AdjustActiveSelection(SelectionDelta);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SAssetPicker::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Enter)
	{
		TArray<FContentBrowserItem> SelectionSet = AssetViewPtr->GetSelectedFileItems();
		HandleItemsActivated(SelectionSet, EAssetTypeActivationMethod::Opened);

		return FReply::Handled();
	}

	if (Commands->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SAssetPicker::FolderEntered(const FString& FolderPath)
{
	CurrentContentSources.SetVirtualPath(FName(*FolderPath));

	AssetViewPtr->SetContentSources(CurrentContentSources);

	OnFolderEnteredDelegate.ExecuteIfBound(FolderPath);
}

FText SAssetPicker::GetHighlightedText() const
{
	return TextFilter->GetRawFilterText();
}

void SAssetPicker::SetSearchBoxText(const FText& InSearchText)
{
	// Has anything changed? (need to test case as the operators are case-sensitive)
	if (!InSearchText.ToString().Equals(TextFilter->GetRawFilterText().ToString(), ESearchCase::CaseSensitive))
	{
		TextFilter->SetRawFilterText(InSearchText);
		if (InSearchText.IsEmpty())
		{
			AssetViewPtr->SetUserSearching(false);
		}
		else
		{
			AssetViewPtr->SetUserSearching(true);
		}
	}
}

void SAssetPicker::OnSearchBoxChanged(const FText& InSearchText)
{
	SetSearchBoxText( InSearchText );
	OnSearchTextChangedDelegate.ExecuteIfBound(InSearchText);
}

void SAssetPicker::OnSearchBoxCommitted(const FText& InSearchText, ETextCommit::Type CommitInfo)
{
	SetSearchBoxText( InSearchText );

	if (CommitInfo == ETextCommit::OnEnter)
	{
		TArray<FContentBrowserItem> SelectionSet = AssetViewPtr->GetSelectedFileItems();
		if ( SelectionSet.Num() == 0 )
		{
			AssetViewPtr->AdjustActiveSelection(1);
			SelectionSet = AssetViewPtr->GetSelectedFileItems();
		}
		HandleItemsActivated(SelectionSet, EAssetTypeActivationMethod::Opened);
	}
}

void SAssetPicker::SetNewBackendFilter(const FARFilter& NewFilter)
{
	CurrentContentSources.SetVirtualPaths(NewFilter.PackagePaths);
	if(AssetViewPtr.IsValid())
	{
		AssetViewPtr->SetContentSources(CurrentContentSources);
	}

	CurrentBackendFilter = NewFilter;
	CurrentBackendFilter.PackagePaths.Reset();

	// Update the Text filter too, since now class names may no longer matter
	if (TextFilter.IsValid())
	{
		TextFilter->SetIncludeClassName(NewFilter.ClassPaths.Num() != 1);
	}

	OnFilterChanged();
}

void SAssetPicker::OnFilterChanged()
{
	FARFilter Filter;
	TArray<TSharedRef<const FPathPermissionList>> CustomPermissionLists;
	
	if ( FilterListPtr.IsValid() )
	{
		Filter = FilterListPtr->GetCombinedBackendFilter(CustomPermissionLists);
	}
	else if (!IsShowingOtherDevelopersContent())
	{
		CustomPermissionLists.Add(HideOtherDevelopersFilter->GetPathPermissionList());
	}

	Filter.Append(CurrentBackendFilter);
	if (AssetViewPtr.IsValid())
	{
		AssetViewPtr->SetBackendFilter(Filter, &CustomPermissionLists);
	}
}

FReply SAssetPicker::OnNoneButtonClicked()
{
	OnAssetSelected.ExecuteIfBound(FAssetData());
	if (AssetViewPtr.IsValid())
	{
		AssetViewPtr->ClearSelection(true);
	}
	return FReply::Handled();
}

void SAssetPicker::HandleNewItemRequested(const FContentBrowserItem& NewItem)
{
	// Make sure we are showing the location of the new file (we may have created it in a folder)
	const FString ItemOwnerPath = FPaths::GetPath(NewItem.GetVirtualPath().ToString());
	FolderEntered(ItemOwnerPath);
}

void SAssetPicker::HandleItemSelectionChanged(const FContentBrowserItem& InSelectedItem, ESelectInfo::Type InSelectInfo)
{
	if (InSelectInfo != ESelectInfo::Direct)
	{
		FAssetData ItemAssetData;
		InSelectedItem.Legacy_TryGetAssetData(ItemAssetData);
		OnAssetSelected.ExecuteIfBound(ItemAssetData);
		
	}
}

void SAssetPicker::HandleItemsActivated(TArrayView<const FContentBrowserItem> ActivatedItems, EAssetTypeActivationMethod::Type ActivationMethod)
{
	FContentBrowserItem FirstActivatedFolder;

	TArray<FAssetData> ActivatedAssets;
	for (const FContentBrowserItem& ActivatedItem : ActivatedItems)
	{
		if (ActivatedItem.IsFile())
		{
			FAssetData ItemAssetData;
			if (ActivatedItem.Legacy_TryGetAssetData(ItemAssetData))
			{
				ActivatedAssets.Add(MoveTemp(ItemAssetData));
			}
		}

		if (ActivatedItem.IsFolder() && !FirstActivatedFolder.IsValid())
		{
			FirstActivatedFolder = ActivatedItem;
		}
	}

	if (FirstActivatedFolder.IsValid())
	{
		if (ActivatedAssets.Num() == 0)
		{
			FolderEntered(FirstActivatedFolder.GetVirtualPath().ToString());
		}
		return;
	}

	if (ActivatedAssets.Num() == 0)
	{
		return;
	}

	if (ActivationMethod == EAssetTypeActivationMethod::DoubleClicked)
	{
		if (ActivatedAssets.Num() == 1)
		{
			OnAssetDoubleClicked.ExecuteIfBound(ActivatedAssets[0]);
		}
	}
	else if (ActivationMethod == EAssetTypeActivationMethod::Opened)
	{
		OnAssetEnterPressed.ExecuteIfBound(ActivatedAssets);
	}

	OnAssetsActivated.ExecuteIfBound( ActivatedAssets, ActivationMethod );
}

void SAssetPicker::HandleItemsDragged(TArrayView<const FContentBrowserItem> DraggedItems)
{
	TArray<FAssetData> DraggedAssets;
	for (const FContentBrowserItem& DraggedItem : DraggedItems)
	{
		if (DraggedItem.IsFile())
		{
			FAssetData ItemAssetData;
			if (DraggedItem.Legacy_TryGetAssetData(ItemAssetData))
			{
				DraggedAssets.Add(MoveTemp(ItemAssetData));
			}
		}
	}

	if (DraggedAssets.Num() == 0)
	{
		return;
	}

	OnAssetsDragged.ExecuteIfBound(DraggedAssets);
}

void SAssetPicker::SyncToAssets(const TArray<FAssetData>& AssetDataList)
{
	AssetViewPtr->SyncToLegacy(AssetDataList, TArray<FString>());
}

void SAssetPicker::SyncToItems(TArrayView<const FContentBrowserItem> ItemsToSync, const bool bAllowImplicitSync, const bool bDisableFiltersThatHideAssets)
{
	if (bDisableFiltersThatHideAssets)
	{
		if (FilterListPtr)
		{
			FilterListPtr->DisableFiltersThatHideItems(ItemsToSync);
		}
		
		if (SearchBoxPtr)
		{
			SetSearchBoxText(FText::GetEmpty());
            SearchBoxPtr->SetText(FText::GetEmpty());
            SearchBoxPtr->SetError(FText::GetEmpty());	
		}
	}
	if (AssetViewPtr)
	{
		AssetViewPtr->SyncToItems(ItemsToSync, bAllowImplicitSync);
	}
}

TArray< FAssetData > SAssetPicker::GetCurrentSelection()
{
	return AssetViewPtr->GetSelectedAssets();
}

void SAssetPicker::RefreshAssetView(bool bRefreshSources)
{
	if (bRefreshSources)
	{
		AssetViewPtr->RequestSlowFullListRefresh();
	}
	else
	{
		AssetViewPtr->RequestQuickFrontendListRefresh();
	}
}

bool SAssetPicker::IsShowingOtherDevelopersContent() const
{
	if (FilterListPtr.IsValid())
	{
		// OtherDevelopersFilter is an inverse filter, so if it is active we are not showing other developers content
		return FilterListPtr->IsFrontendFilterActive(HideOtherDevelopersFilter);
	}
	else
	{
		for (const TSharedPtr<IFilter<FAssetFilterType>>& Filter : *FrontendFilters.Get())
		{
			if (Filter == HideOtherDevelopersFilter)
			{
				return false;
			}
		}
		return true;
	}
}

FText SAssetPicker::GetShowOtherDevelopersToolTip() const
{
	// NOTE: This documents the filter effect rather than the button action.
	if (IsShowingOtherDevelopersContent())
	{
		return LOCTEXT("ShowingOtherDevelopersFilterTooltipText", "Showing Other Developers Assets");
	}
	else
	{
		return LOCTEXT("HidingOtherDevelopersFilterTooltipText", "Hiding Other Developers Assets");
	}
}

void SAssetPicker::HandleShowOtherDevelopersCheckStateChanged( ECheckBoxState InCheckboxState )
{
	if (FilterListPtr) // Filter UI enabled?
	{
		// Pin+activate or unpin+deactivate the filter. A widget is pinned on the filter UI. It allows the user to activate/deactive the filter independently of the 'checked' state.
		FilterListPtr->SetFrontendFilterCheckState(HideOtherDevelopersFilter, InCheckboxState);
	}
	else
	{
		if (InCheckboxState == ECheckBoxState::Checked)
		{
			HideOtherDevelopersFilter->SetActiveInCollection(HideOtherDevelopersFilter.ToSharedRef(), /* do show other developers content*/ false, *FrontendFilters);
		}
		else
		{
			HideOtherDevelopersFilter->SetActiveInCollection(HideOtherDevelopersFilter.ToSharedRef(), /* do not show other developers content*/ true, *FrontendFilters);
		}
	}

	OnFilterChanged();
}

ECheckBoxState SAssetPicker::GetShowOtherDevelopersCheckState() const
{
	if (FilterListPtr) // Filter UI enabled?
	{
		return FilterListPtr->GetFrontendFilterCheckState(HideOtherDevelopersFilter); // Tells whether the 'other developer' filter is pinned on the filter UI. (The filter itself may be active or not).
	}
	else
	{
		if (IsShowingOtherDevelopersContent())
		{
			return ECheckBoxState::Checked;
		}
		else
		{
			return ECheckBoxState::Unchecked;
		}
	}
}

void SAssetPicker::OnRenameRequested() const
{
	const TArray<FContentBrowserItem> SelectedItems = AssetViewPtr->GetSelectedItems();
	if (SelectedItems.Num() == 1)
	{
		AssetViewPtr->RenameItem(SelectedItems[0]);
	}
}

bool SAssetPicker::CanExecuteRenameRequested()
{
	if(!bAllowRename)
	{
		return false;
	}
	return ContentBrowserUtils::CanRenameFromAssetView(AssetViewPtr);
}

bool SAssetPicker::CanFilterBySelection()
{
	if (!AssetViewPtr.IsValid() || !FilterListPtr.IsValid())
	{
		return false;
	}

	const TArray<FContentBrowserItem> SelectedItems = AssetViewPtr->GetSelectedItems();
	return !SelectedItems.IsEmpty();
}

void SAssetPicker::OnFilterBySelection()
{
	if (!AssetViewPtr.IsValid() || !FilterListPtr.IsValid())
	{
		return;
	}

	const TArray<FContentBrowserItem> SelectedItems = AssetViewPtr->GetSelectedItems();

	TArray<FTopLevelAssetPath> AssetPaths;
	AssetPaths.Reserve(SelectedItems.Num());
	for (const FContentBrowserItem& SelectedItem : SelectedItems)
	{
		FContentBrowserItemDataAttributeValue AttributeValue = SelectedItem.GetItemAttribute(ContentBrowserItemAttributes::ItemTypeName);
		if (AttributeValue.IsValid())
		{
			AssetPaths.Add(FTopLevelAssetPath(AttributeValue.GetValueString()));
		}
	}

	FilterListPtr->SetActiveFilters(AssetPaths);
}

void SAssetPicker::ExecuteRenameCommand()
{
	if (Commands.IsValid())
	{
		Commands->TryExecuteAction(FGenericCommands::Get().Rename.ToSharedRef());
	}
}

void SAssetPicker::BindCommands()
{
	Commands = MakeShareable(new FUICommandList);
	// bind commands
	Commands->MapAction( FGenericCommands::Get().Rename, FUIAction(
		FExecuteAction::CreateSP( this, &SAssetPicker::OnRenameRequested ),
		FCanExecuteAction::CreateSP( this, &SAssetPicker::CanExecuteRenameRequested )
		));

	Commands->MapAction(
		FContentBrowserCommands::Get().FilterBySelection,
		FUIAction(
			FExecuteAction::CreateSP(this, &SAssetPicker::OnFilterBySelection),
			FCanExecuteAction::CreateSP(this, &SAssetPicker::CanFilterBySelection)
		)
	);
}

void SAssetPicker::LoadSettings()
{
	const FString& SettingsString = SaveSettingsName;

	if ( !SettingsString.IsEmpty() )
	{
		// Load all our data using the settings string as a key in the user settings ini
		if (FilterListPtr.IsValid())
		{
			FilterListPtr->LoadSettings();
		}
		
		AssetViewPtr->LoadSettings(GEditorPerProjectIni, SContentBrowser::SettingsIniSection, SettingsString);
	}
}

void SAssetPicker::SaveSettings() const
{
	const FString& SettingsString = SaveSettingsName;

	if ( !SettingsString.IsEmpty() )
	{
		// Save all our data using the settings string as a key in the user settings ini
		if (FilterListPtr.IsValid())
		{
			FilterListPtr->SaveSettings();
		}

		AssetViewPtr->SaveSettings(GEditorPerProjectIni, SContentBrowser::SettingsIniSection, SettingsString);
	}
}

void SAssetPicker::HandleSearchSettingsChanged()
{
	bool bClassNamesProvided = (FilterListPtr.IsValid() ? FilterListPtr->GetInitialClassFilters().Num() != 1 : false);
	TextFilter->SetIncludeClassName(bClassNamesProvided || AssetViewPtr->IsIncludingClassNames());
	TextFilter->SetIncludeAssetPath(AssetViewPtr->IsIncludingAssetPaths());
	TextFilter->SetIncludeCollectionNames(AssetViewPtr->IsIncludingCollectionNames());
}

TSharedPtr<SWidget> SAssetPicker::GetItemContextMenu(TArrayView<const FContentBrowserItem> SelectedItems)
{
	// We may only open the file or folder context menu (folder takes priority), so see whether we have any folders selected
	TArray<const FContentBrowserItem*> SelectedFolders;
	for (const FContentBrowserItem& SelectedItem : SelectedItems)
	{
		if (SelectedItem.IsFolder())
		{
			SelectedFolders.Add(&SelectedItem);
		}
	}

	if (SelectedFolders.Num() > 0)
	{
		// Folders selected - show the folder menu

		TArray<FString> SelectedPackagePaths;
		SelectedPackagePaths.Reserve(SelectedFolders.Num());
		for (const FContentBrowserItem* SelectedFolder : SelectedFolders)
		{
			const FName PackagePath = SelectedFolder->GetInternalPath();
			if (!PackagePath.IsNone())
			{
				SelectedPackagePaths.Add(PackagePath.ToString());
			}
		}

		if (SelectedPackagePaths.Num() > 0 && OnGetFolderContextMenu.IsBound())
		{
			return OnGetFolderContextMenu.Execute(SelectedPackagePaths, FContentBrowserMenuExtender_SelectedPaths(), FOnCreateNewFolder::CreateSP(AssetViewPtr.Get(), &SAssetView::NewFolderItemRequested));
		}
	}
	else
	{
		// Files selected - show the file menu

		TArray<FAssetData> SelectedAssets;
		for (const FContentBrowserItem& SelectedItem : SelectedItems)
		{
			FAssetData ItemAssetData;
			if (SelectedItem.IsFile() && SelectedItem.Legacy_TryGetAssetData(ItemAssetData))
			{
				SelectedAssets.Add(MoveTemp(ItemAssetData));
			}
		}

		if (OnGetAssetContextMenu.IsBound())
		{
			return OnGetAssetContextMenu.Execute(SelectedAssets);
		}
	}

	return nullptr;
}

TOptional<FAssetCategoryPath> SAssetPicker::ConvertAssetTypeCategoryToAssetCategoryPath(EAssetTypeCategories::Type InDefaultFilterMenuExpansion)
{
	// TODO We should completely replace EAssetTypeCategories with FAssetCategoryPath, but FAssetCategoryPath is contained in the AssetDefinitionsModule.
	// Since the API exposes the DefaultFilterMenuExpansion, we can't easily change this
	switch (InDefaultFilterMenuExpansion)
	{
	case EAssetTypeCategories::Basic:
		return EAssetCategoryPaths::Basic;
	case EAssetTypeCategories::Animation:
		return EAssetCategoryPaths::Animation;
	case EAssetTypeCategories::Materials:
		return EAssetCategoryPaths::Material;
	case EAssetTypeCategories::Sounds:
		return EAssetCategoryPaths::Audio;
	case EAssetTypeCategories::Physics:
		return EAssetCategoryPaths::Physics;
	case EAssetTypeCategories::UI:
		return EAssetCategoryPaths::UI;
	case EAssetTypeCategories::Misc:
		return EAssetCategoryPaths::Misc;
	case EAssetTypeCategories::Gameplay:
		return EAssetCategoryPaths::Gameplay;
	case EAssetTypeCategories::Blueprint:
		return EAssetCategoryPaths::Blueprint;
	case EAssetTypeCategories::Media:
		return EAssetCategoryPaths::Media;
	case EAssetTypeCategories::Textures:
		return EAssetCategoryPaths::Texture;
	case EAssetTypeCategories::World:
		break;
	case EAssetTypeCategories::FX:
		return EAssetCategoryPaths::FX;
	}

	return TOptional<FAssetCategoryPath>();
}

#undef LOCTEXT_NAMESPACE
