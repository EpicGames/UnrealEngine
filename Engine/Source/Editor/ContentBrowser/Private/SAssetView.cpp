// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAssetView.h"

#include "Algo/AnyOf.h"
#include "Algo/Count.h"
#include "Algo/Compare.h"
#include "Algo/RemoveIf.h"
#include "Algo/Transform.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "AssetSelection.h"
#include "AssetTextFilter.h"
#include "AssetToolsModule.h"
#include "AssetView/AssetViewConfig.h"
#include "AssetViewTypes.h"
#include "AssetViewWidgets.h"
#include "Async/ParallelFor.h"
#include "Async/WordMutex.h"
#include "Async/UniqueLock.h"
#include "CollectionManagerModule.h"
#include "ContentBrowserCommands.h"
#include "ContentBrowserConfig.h"
#include "ContentBrowserDataDragDropOp.h"
#include "ContentBrowserDataLegacyBridge.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserLog.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserMenuUtils.h"
#include "ContentBrowserModule.h"
#include "ContentBrowserSingleton.h"
#include "ContentBrowserStyle.h"
#include "ContentBrowserUtils.h"
#include "DesktopPlatformModule.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragDropHandler.h"
#include "Editor.h"
#include "EditorWidgetsModule.h"
#include "Engine/Blueprint.h"
#include "Engine/Level.h"
#include "Experimental/ContentBrowserSearchOverrideInternal.h"
#include "Factories/Factory.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "FrontendFilterBase.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IAssetTools.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "ICollectionSource.h"
#include "IContentBrowserDataModule.h"
#include "Materials/Material.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/NamePermissionList.h"
#include "Misc/TextFilterUtils.h"
#include "ObjectTools.h"
#include "SContentBrowser.h"
#include "SFilterList.h"
#include "SPrimaryButton.h"
#include "Settings/ContentBrowserSettings.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "TelemetryRouter.h"
#include "Textures/SlateIcon.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "ToolMenus.h"
#include "UObject/UnrealType.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "ISourceControlModule.h"
#include "RevisionControlStyle/RevisionControlStyle.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"
#define MAX_THUMBNAIL_SIZE 4096

#define ASSET_VIEW_PARANOIA_LIST_CHECKS (0)
#if ASSET_VIEW_PARANOIA_LIST_CHECKS
	#define checkAssetList(cond) check(cond)
#else
	#define checkAssetList(cond)
#endif

static bool bEnableGridTileSwitch = false;
static FAutoConsoleVariableRef CVarEnableGridTileSwitch(
	TEXT("ContentBrowser.EnableGridTileSwitch"),
	bEnableGridTileSwitch,
	TEXT("If true Grid and List view will switch between each other when reaching certain size.\n"
	"List > Huge -> Grid.\n"
	"Grid < Tiny -> List."));

namespace UE::AssetView
{
	/** Time delay between recently added items being added to the filtered asset items list */
	constexpr double TimeBetweenAddingNewAssets = 4.0;

	/** Time delay between performing the last jump, and the jump term being reset */
	constexpr double JumpDelaySeconds = 2.0;

	/** Number of frames a deferred pending list will wait before clearing out the data.*/
	constexpr int32 DeferredSyncTimeoutFramesCount = 30;

	bool AllowAsync = true;
	FAutoConsoleVariableRef CVarAllowAsync(
		TEXT("AssetView.AllowAsync"),
		AllowAsync,
		TEXT("Whether to allow the asset view to perform work with async tasks (rather than time-sliced)"),
		ECVF_Default
	);
	 
	bool AllowParallelism = true;
	FAutoConsoleVariableRef CVarAllowParallelism(
		TEXT("AssetView.AllowParallelism"),
		AllowParallelism,
		TEXT("Whether to allow the asset view to perform work in parallel (e.g. ParallelFor)"),
		ECVF_Default
	);

	// Return the max size of the batch of items to text filter per task - do fewer if parallelism is disabled
	int32 GetMaxTextFilterItemBatch()
	{
		static const int32 NumWorkers = LowLevelTasks::FScheduler::Get().GetNumWorkers();
		return AllowParallelism ? NumWorkers * 1024 : 1024;
	}

	bool AreBackendFiltersDifferent(const FARFilter& A, const FARFilter& B)
	{
		if (A.PackageNames.Num() != B.PackageNames.Num()
		|| A.PackagePaths.Num() != B.PackagePaths.Num()
		|| A.SoftObjectPaths.Num() != B.SoftObjectPaths.Num()
		|| A.ClassPaths.Num() != B.ClassPaths.Num()
		|| A.TagsAndValues.Num() != B.TagsAndValues.Num()
		|| A.RecursiveClassPathsExclusionSet.Num() != B.RecursiveClassPathsExclusionSet.Num()
		|| A.bRecursivePaths != B.bRecursivePaths
		|| A.bRecursiveClasses != B.bRecursiveClasses
		|| A.bIncludeOnlyOnDiskAssets != B.bIncludeOnlyOnDiskAssets
		|| A.WithoutPackageFlags != B.WithoutPackageFlags 
		|| A.WithPackageFlags != B.WithPackageFlags)
		{
			return true;
		}

		// Expect things to be generated in the same order by the filter bar, so just check linear matching
		if (!Algo::Compare(A.PackageNames, B.PackageNames)
		|| !Algo::Compare(A.PackagePaths, B.PackagePaths)
		|| !Algo::Compare(A.SoftObjectPaths, B.SoftObjectPaths)
		|| !Algo::Compare(A.ClassPaths, B.ClassPaths))
		{
			return true;
		}

		for (const FTopLevelAssetPath& Path : A.RecursiveClassPathsExclusionSet)
		{
			if (!B.RecursiveClassPathsExclusionSet.Contains(Path))
			{
				return true;
			}
		}

		for (const FTopLevelAssetPath& Path : B.RecursiveClassPathsExclusionSet)
		{
			if (!A.RecursiveClassPathsExclusionSet.Contains(Path))
			{
				return true;
			}
		}

		TArray<FName> AKeys;
		A.TagsAndValues.GetKeys(AKeys);
		for (FName Key : AKeys)
		{
			if (!B.TagsAndValues.Contains(Key))
			{
				return true;
			}
			TArray<TOptional<FString>> AValues;
			A.TagsAndValues.MultiFind(Key, AValues);
			Algo::SortBy(AValues, [](const TOptional<FString>& S) { return S.Get(FString()); });
			TArray<TOptional<FString>> BValues;
			B.TagsAndValues.MultiFind(Key, BValues);
			Algo::SortBy(BValues, [](const TOptional<FString>& S) { return S.Get(FString()); });

			if (!Algo::Compare(AValues, BValues))
			{
				return true;
			}
		}

		return false;
	}

	bool AreCustomPermissionListsDifferent(TArray<TSharedRef<const FPathPermissionList>>* InCustomPermissionLists,
		TArray<TSharedRef<const FPathPermissionList>>& ExistingPermissionLists)
	{
		if (InCustomPermissionLists == nullptr)
		{
			return ExistingPermissionLists.IsEmpty();
		}

		// Expect order to be built in the same way so if order is different, trigger a rebuild
		// Also expect that if filters change their permission lists, they create a new object. 
		return !Algo::Compare(*InCustomPermissionLists, ExistingPermissionLists);
	}
}

FAssetViewDragAndDropExtender::FPayload::FPayload(TSharedPtr<FDragDropOperation> InDragDropOp, const TArray<FName>& InPackagePaths, const TArray<FCollectionRef>& InCollectionSources)
	: DragDropOp(MoveTemp(InDragDropOp))
	, TempCollectionSources()
	, TempCollections()
	, PackagePaths(InPackagePaths)
	, CollectionSources(InCollectionSources)
	, Collections(TempCollections)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Fill out deprecated Collections with game project Collections for backwards compatibility.
	Algo::TransformIf(
		CollectionSources,
		TempCollections,
		[](const FCollectionRef& Collection) { return Collection.Container == FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(); },
		[](const FCollectionRef& Collection) { return FCollectionNameType(Collection.Name, Collection.Type); });
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FAssetViewDragAndDropExtender::FPayload::FPayload(TSharedPtr<FDragDropOperation> InDragDropOp, const TArray<FName>& InPackagePaths, const TArray<FCollectionNameType>& InCollections)
	: DragDropOp(MoveTemp(InDragDropOp))
	, TempCollectionSources()
	, TempCollections()
	, PackagePaths(InPackagePaths)
	, CollectionSources(TempCollectionSources)
	, Collections(InCollections)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TempCollectionSources.Reserve(Collections.Num());
	Algo::Transform(
		Collections,
		TempCollectionSources,
		[](const FCollectionNameType& Collection) { return FCollectionRef(FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(), Collection); });
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FText SAssetView::ThumbnailSizeToDisplayName(EThumbnailSize InSize)
{
	switch (InSize)
	{
	case EThumbnailSize::Tiny:
		return LOCTEXT("TinyThumbnailSize", "Tiny");
	case EThumbnailSize::Small:
		return LOCTEXT("SmallThumbnailSize", "Small");
	case EThumbnailSize::Medium:
		return LOCTEXT("MediumThumbnailSize", "Medium");
	case EThumbnailSize::Large:
		return LOCTEXT("LargeThumbnailSize", "Large");
	case EThumbnailSize::XLarge:
		return LOCTEXT("XLargeThumbnailSize", "X Large");
	case EThumbnailSize::Huge:
		return LOCTEXT("HugeThumbnailSize", "Huge");
	default:
		return FText::GetEmpty();
	}
}

class FAssetViewFrontendFilterHelper
{
public:
	explicit FAssetViewFrontendFilterHelper(SAssetView* InAssetView)
		: AssetView(InAssetView)
		, ContentBrowserData(IContentBrowserDataModule::Get().GetSubsystem())
		, FolderFilter()
		, bDisplayEmptyFolders(AssetView->IsShowingEmptyFolders())
	{
		if (bDisplayEmptyFolders)
		{
			FolderFilter.HideFolderIfEmptyFilter = ContentBrowserData->CreateHideFolderIfEmptyFilter();
		}
		else
		{
			FolderFilter.ItemCategoryFilter = InAssetView->DetermineItemCategoryFilter();
		}
	}

	bool NeedsQueryFilter()
	{
		return AssetView->OnShouldFilterItem.IsBound() || AssetView->OnShouldFilterAsset.IsBound();
	}

	bool DoesItemPassQueryFilter(const TSharedPtr<FAssetViewItem>& InItemToFilter)
	{
		// Folders aren't subject to additional filtering
		if (InItemToFilter->IsFolder())
		{
			return true;
		}

		if (AssetView->OnShouldFilterItem.IsBound() && AssetView->OnShouldFilterItem.Execute(InItemToFilter->GetItem()))
		{
			return false;
		}

		// If we have OnShouldFilterAsset then it is assumed that we really only want to see true assets and 
		// nothing else so only include things that have asset data and also pass the query filter
		if (AssetView->OnShouldFilterAsset.IsBound())
		{
			FAssetData ItemAssetData;
			if (!InItemToFilter->GetItem().Legacy_TryGetAssetData(ItemAssetData) || AssetView->OnShouldFilterAsset.Execute(ItemAssetData))
			{
				return false;
			}
		}

		return true;
	}

	bool DoesItemPassFrontendFilter(const TSharedPtr<FAssetViewItem>& InItemToFilter)
	{
		// Folders are only subject to "empty" filtering
		if (InItemToFilter->IsFolder())
		{
			if (!ContentBrowserData->IsFolderVisible(InItemToFilter->GetItem().GetVirtualPath(), ContentBrowserUtils::GetIsFolderVisibleFlags(bDisplayEmptyFolders), FolderFilter))
			{
				return false;
			}
			return true;
		}

		// Run the item through the filters
		if (AssetView->IsFrontendFilterActive() && !AssetView->PassesCurrentFrontendFilter(InItemToFilter->GetItem()))
		{
			return false;
		}

		return true;
	}

private:
	SAssetView* AssetView = nullptr;
	UContentBrowserDataSubsystem* ContentBrowserData = nullptr;
	FContentBrowserFolderContentsFilter FolderFilter;
	const bool bDisplayEmptyFolders = true;
};


struct FAssetViewItemFilterState
{
	uint8 Removed : 1;
	uint8 PassedFrontendFilter : 1;
	uint8 PassedTextFilter : 1;	

	// This item passed filtering and was published to the view
	uint8 Published : 1;
	// Priority filtering was performed because of data updates, do not overwrite results with async filtering results
	uint8 PriorityFiltered : 1;
};

/**
 * Manages items returned from backend query and incrementally/asynchronously filtering them.
 * - Recycling of old objects on new query
 */
class FAssetViewItemCollection
{
public: 
	FAssetViewItemCollection()
	{
		bShowingContentVersePath = IAssetTools::Get().ShowingContentVersePath();

		IPluginManager::Get().OnPluginEdited().AddRaw(this, &FAssetViewItemCollection::OnPluginEdited);
	}

	~FAssetViewItemCollection()
	{
		IPluginManager::Get().OnPluginEdited().RemoveAll(this);
	}

	/** Returns the number of items which were fetched and have not been removed. */
	int32 Num() const { return NumValidItems; }

	/** Returns true if there is any incomplete filtering work. */
	bool HasItemsPendingFilter()
	{
		// No need to check text filtering progress/task here as PublishProgress cannot surpass text filtering
		return ItemsPendingPriorityFilter.Num() != 0 || PublishProgress < Items.Num() || ItemsPendingPriorityPublish.Num();
	}

	/** Return the amount of progress made in filtering for presentation to the user; a number between 0 and Num() */
	int32 GetFilterProgress() const
	{
		return PublishProgress;
	}

	/** 
	 * Fetch all items from the given paths (sources) matching the given filter, recycling old FAssetViewItem objects.
	 * 
	 * @param bAllowItemRecycling Whether or not to allow reuse of items and therefore widgets. 
	 * 	Setting to false can avoid lots of time firing modification delegates for recursive searches.
	 */
	void RefreshItemsFromBackend(const FAssetViewContentSources& ContentSources, const FContentBrowserDataFilter& DataFilter, bool bAllowItemRecycling);
	
	/** 
	 * Find an FAssetViewItem containing the given content browser data if one exists.
	 * Returned item should not be modified as background text processing may be operating on it.
	 */
	TSharedPtr<FAssetViewItem> FindItemForRename(const FContentBrowserItem& InItem);

	/** 
	 * Create the given item from user interaction (e.g. create asset, rename asset).
	 * Makes the item visible immediately. 
	 */
	TSharedPtr<FAssetViewItem> CreateItemFromUser(FContentBrowserItem&& InItem, TArray<TSharedPtr<FAssetViewItem>>& FilteredAssetItems);
	
	/** 
	 * Find an existing item or create one from an incremental data update.
	 * If an item exists, the data in it is replaced and a callback fired to be handled by widgets bound to it.
	 * Safe to call during threaded text filtering.
	 */
	TSharedPtr<FAssetViewItem> UpdateData(FContentBrowserItemData&& InData);

	/**
	 * Remove the given item data from the FAssetViewItem that contains it and if that item no longer contains any data,
	 * remove the FAssetViewItem and return it.
	 * Safe to call during threaded text filtering.
	 */
	TSharedPtr<FAssetViewItem> RemoveItemData(const FContentBrowserItemData& InItemData);
	TSharedPtr<FAssetViewItem> RemoveItemData(const FContentBrowserMinimalItemData& InItemData);

	/**
	 * Remove the given item that was being created/renamed.
	 * Safe to call during threaded text filtering.
	 */
	void RemoveItem(const TSharedPtr<FAssetViewItem>& ToRemove);

	/** 
	 * Cancel any in progress async text filtering operation and wait for tasks to shut down.
	 */
	void AbortTextFiltering();	
	
	/** 
	 * Clear the filtering results of all known non-removed items to be run again with new filters.
	 * Cancels async text filtering and waits for cancellation to complete safely.
	 */
	void ResetFilterState();
	
	/** 
	 * Start filtering all items against the given text filter in the background.
	 * Results will be fetched and merged during UpdateItemFiltering.
	 */
	void StartTextFiltering(TSharedPtr<FAssetTextFilter> TextFilter);

	/** 
	 * Run main-thread filtering on items until the specified end time.
	 * 
	 * @param InTextFilter Text filter if any for launching new async text filtering tasks if necessary
	 * @param OutItems Array to populate with new items that passed the filter if any.
	 */
	void UpdateItemFiltering(FAssetViewFrontendFilterHelper& InHelper, double InEndTime, TArray<TSharedPtr<FAssetViewItem>>& OutItems);

	/** 
	 * Perform filtering on any items which already existed and had been filtered when a data update was received.
	 * Returns true if any items changed filter result.
	 * 
	 * @param FilteredAssetItems Items which have already been determined to be visible, to be updated. 
	 */
	bool PerformPriorityFiltering(FAssetViewFrontendFilterHelper& Helper,
		TArray<TSharedPtr<FAssetViewItem>>& FilteredAssetItems);

	bool GetShowingContentVersePath() const
	{
		return bShowingContentVersePath;
	}

	/**
	 * Check if we need to update the path text of any items.
	 * Returns true if the path text may have changed to indicate items need to be sorted and filtered again.
	 */
	bool UpdateShowingContentVersePath();

private:
	int32 CreateItem_Locked(FContentBrowserItemData&& InItem)
	{
		return CreateItem_Locked(FContentBrowserItem(MoveTemp(InItem)));
	}

	int32 CreateItem_Locked(FContentBrowserItem&& InItem)
	{
		uint32 Hash = HashItem(InItem);
		TSharedPtr<FAssetViewItem> NewItem = MakeShared<FAssetViewItem>(Items.Num(), MoveTemp(InItem));
		NewItem->ConfigureAssetPathText(bShowingContentVersePath);
		if (TextFilterTask.IsValid() && bCompiledTextFilterRequiresVersePaths)
		{
			NewItem->CacheVersePathTextForTextFiltering();
		}
		Items.Add(MoveTemp(NewItem));
		FilterState.AddZeroed(1);
		++NumValidItems;
		if (!RefreshLookup())
		{
			// Resize lookup's index list to match capacity of Items rather than its own growth strategy 
			if (Lookup.GetIndexSize() < (uint32)Items.Num())
			{
				Lookup.Resize(Items.Max());	
			}
			Lookup.Add(Hash, Items.Num() - 1);
		}
		return Items.Num() - 1;
	}

	uint32 HashItem(const FContentBrowserItem& Item)
	{
		check(Item.IsValid());
		return GetTypeHash(Item.GetVirtualPath());
	}
	uint32 HashItem(const FContentBrowserItemData& Item)
	{
		check(Item.IsValid());
		return GetTypeHash(Item.GetVirtualPath());
	}
	uint32 HashItem(const FContentBrowserMinimalItemData& Item)
	{
		check(!Item.GetVirtualPath().IsNone());
		return GetTypeHash(Item.GetVirtualPath());
	}
	 
	inline bool IsItemValid(int32 ItemIndex)
	{
		return Items[ItemIndex].IsValid() && FilterState[ItemIndex].Removed == false;
	}

	struct FTextFilterResult
	{
		int32 StartIndex;
		TBitArray<> Results;
		UE::Tasks::TTask<FTextFilterResult> Next;
	};
	FTextFilterResult AsyncFilterText(int32 StartIndex, int32 MaxItems) const;

	inline bool ItemPassedAllFilters(int32 Index) const
	{
		return !FilterState[Index].Removed && FilterState[Index].PassedFrontendFilter && (bAllItemsPassedTextFilter || FilterState[Index].PassedTextFilter);
	}
	
	inline TSharedPtr<FAssetViewItem> MarkItemRemoved(int32 Index)
	{
		check(Items[Index].IsValid() && !FilterState[Index].Removed);
		--NumValidItems;
		bItemsPendingRemove = true;
		FilterState[Index].Removed = true;
		// Do not null out the item because we want to be able to remove it from published items in PerformPriorityFiltering
		return Items[Index];
	}

	/** 
	 * If the number of stored items has grown beyond the bounds of Lookup, rebuild it with larger hash 
	 * @return true if the lookup was rebuilt 
	 */
	bool RefreshLookup();

	void OnPluginEdited(IPlugin& InPlugin)
	{
		// The plugin's Verse path may have changed.  Recompute all Verse paths.
		bShouldInvalidateVersePaths = true;
	}

private:
	// Lock for access to the size and contents of Items - e.g. when text filtering is operating on a batch of items
	// The size of Items may need to change or the ItemData object within items may need modification as data scanning progresses
	mutable FRWLock Lock;

	// Hash of items by virtual path. There may be multiple items with the same virtual path, deduplication is done manually during 
	// population. Objects with the same path may exist unless they are folders, in which case they are merged.
	// After population, objects are looked up by path & source for updates.
	// This lookup is only used on the main thread so it can be safely rebuilt during async text filtering.
	FHashTable Lookup;
	// Linear list of items indexed by Lookup. May contain null entries. 
	TArray<TSharedPtr<FAssetViewItem>> Items;
	// State of non-text filtering matching items in Items. Only modified on main thread. 
	TArray<FAssetViewItemFilterState> FilterState;

	// How many items in Items are not null. Atomically decreased during batch merge of folder items.
	// Also decreased when items are removed by data update notifications.
	std::atomic<int32> NumValidItems{0};
	// How many elements of Items have been tested against frontend filtering if required.
	int32 FrontendFilterProgress = 0;
	// How many elements of Items have gone through text filtering and had their results merged on the main thread.
	int32 TextFilterProgress = 0;
	// How many elements of Items have been published to the view - smaller of FilterProgress and TextFilterProgress on last update 
	// Items with indices below this may have been added to the list/tile/column view so updates to those items require re-filtering
	int32 PublishProgress = 0;

	// Cached compiled text filter for the current filtering pass
	TSharedPtr<FCompiledAssetTextFilter> CompiledTextFilter;
	// True if the compiled text filter might need Verse path info.
	bool bCompiledTextFilterRequiresVersePaths = false;

	// Handle to ongoing task filtering Items by a text query. 
	// Modifications during filtering are protected by the Lock in this class and otherwise yet-to-be-filtered items should not be
	// provided to external code or modified.
	UE::Tasks::TTask<FTextFilterResult> TextFilterTask;
	// Flag to signal cancellation of text filtering task 
	std::atomic<bool> ShouldCancelTextFiltering{false}; 

	// Items which have been updated while visible, so should be re-filtered immediately
	TSet<int32> ItemsPendingPriorityFilter;
	// Items which were updated and passed filtering when they previously failed, so need to be published again
	TSet<int32> ItemsPendingPriorityPublish;

	// If true all items passed text filtering and TextFilterState may be emptied - e.g. if no text filter was applied at all
	bool bAllItemsPassedTextFilter = false;

	// Some items have been marked for removal but their pointers have not been cleared yet because we need to compare against them
	std::atomic<bool> bItemsPendingRemove = false;

	// True if we should show Verse paths in the path column or include them when text filtering.
	bool bShowingContentVersePath;
	// True if a plugin descriptor might have changed in a way that affects Verse paths.
	bool bShouldInvalidateVersePaths = false;
};

bool FAssetViewItemCollection::RefreshLookup()
{
	uint32 HashSize = FDefaultSparseSetAllocator::GetNumberOfHashBuckets(Items.Num());
	if (Lookup.GetHashSize() < HashSize)
	{
		Lookup.Clear(HashSize, Items.Max());
		ParallelFor(TEXT("FAssetViewItemCollection::RefreshLookup"), Items.Num(), 16 * 1024, [this](int32 ItemIndex){
			if (!IsItemValid(ItemIndex))
			{
				return;
			}

			const TSharedPtr<FAssetViewItem> Item = Items[ItemIndex];
			uint32 Hash = HashItem(Item->GetItem());
			Lookup.Add_Concurrent(Hash, ItemIndex);
		}, UE::AssetView::AllowParallelism ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
		return true;
	}
	return false;
}

TSharedPtr<FAssetViewItem> FAssetViewItemCollection::FindItemForRename(const FContentBrowserItem& InItem)
{
	const uint32 Hash = HashItem(InItem);
	FContentBrowserItemKey ItemKey(InItem);
	for (uint32 It = Lookup.First(Hash); Lookup.IsValid(It); It = Lookup.Next(It))
	{
		if (IsItemValid(It) && ItemKey == FContentBrowserItemKey(Items[It]->GetItem()))
		{
			checkf(FilterState[It].Published, 
				TEXT("Only items which have been made visible in the UI should be available for renaming to maintain thread safety with async text filtering."));
			return Items[It];
		}
	}
	return {};
}

TSharedPtr<FAssetViewItem> FAssetViewItemCollection::CreateItemFromUser(FContentBrowserItem&& InItem, TArray<TSharedPtr<FAssetViewItem>>& FilteredAssetItems)
{
	int32 Index;
	{
		FWriteScopeLock Guard(Lock);
		Index = CreateItem_Locked(MoveTemp(InItem));
	}
	// Make this item visible immediately, forcing it to be so regardless of current filter set until filtering is refreshed.
	FilterState[Index].PassedFrontendFilter = true;
	FilterState[Index].PassedTextFilter = true;
	FilterState[Index].Published = true;
	FilterState[Index].PriorityFiltered = true;
	FilteredAssetItems.Add(Items[Index]);
	return Items[Index];
}

TSharedPtr<FAssetViewItem> FAssetViewItemCollection::UpdateData(FContentBrowserItemData&& InData)
{
	const uint32 Hash = HashItem(InData);
	FContentBrowserItemKey ItemKey(InData);
	int32 ExistingItemIndex = INDEX_NONE;
	for (uint32 It = Lookup.First(Hash); Lookup.IsValid(It); It = Lookup.Next(It))
	{
		if (IsItemValid(It) && ItemKey == FContentBrowserItemKey(Items[It]->GetItem()))
		{
			ExistingItemIndex = It;
			break;
		}
	}

	if (ExistingItemIndex != INDEX_NONE)
	{
		FWriteScopeLock Guard(Lock);
		// Update the item and mark it for re-filtering if it has already been filtered
		Items[ExistingItemIndex]->AppendItemData(MoveTemp(InData));
		Items[ExistingItemIndex]->BroadcastItemDataChanged();
		check(!FilterState[ExistingItemIndex].Removed);
	}
	else
	{
		FWriteScopeLock Guard(Lock);
		ExistingItemIndex = CreateItem_Locked(MoveTemp(InData));
	}

	if (ExistingItemIndex < PublishProgress) 
	{
		// This item was already filtered so we may want to remove it from the view or add it 
		ItemsPendingPriorityFilter.Add(ExistingItemIndex);
	}
	return Items[ExistingItemIndex];
}

TSharedPtr<FAssetViewItem> FAssetViewItemCollection::RemoveItemData(const FContentBrowserItemData& InItemData)
{
	return RemoveItemData(FContentBrowserMinimalItemData(InItemData));
}

TSharedPtr<FAssetViewItem> FAssetViewItemCollection::RemoveItemData(const FContentBrowserMinimalItemData& InItemData)
{
	const uint32 Hash = HashItem(InItemData);
	FContentBrowserItemKey ItemKey(InItemData.GetItemType(), InItemData.GetVirtualPath(), InItemData.GetDataSource());
	for (uint32 It = Lookup.First(Hash); Lookup.IsValid(It); It = Lookup.Next(It))
	{
		if (IsItemValid(It) && ItemKey == FContentBrowserItemKey(Items[It]->GetItem()))
		{
			TSharedRef<FAssetViewItem> ItemToRemove = Items[It].ToSharedRef();

			{
				// We only need to lock around the modification of the data stored in ItemToRemove because the background text search may be reading it.
				FWriteScopeLock Guard(Lock);
				ItemToRemove->RemoveItemData(InItemData);
			}

			// Only fully remove this item if every sub-item is removed (items become invalid when empty)
			if (ItemToRemove->GetItem().IsValid())
			{
				return {};
			}

			// This item was already filtered so we may want to remove it from the view.
			if (It < (uint32)PublishProgress) 
			{
				ItemsPendingPriorityFilter.Add(It);
			}
			Lookup.Remove(Hash, It);
			return MarkItemRemoved(It);
		}
	}
	return {};
}
 

void FAssetViewItemCollection::RemoveItem(const TSharedPtr<FAssetViewItem>& ToRemove)
{
	// There is no need to lock here because we don't modify the item which the background text search may be reading.
	const uint32 Hash = HashItem(ToRemove->GetItem());
	for (uint32 It = Lookup.First(Hash); Lookup.IsValid(It); It = Lookup.Next(It))
	{
		if (Items[It] == ToRemove)
		{
			check(!FilterState[It].Removed);
		 	Lookup.Remove(Hash, It);
			// This item was already filtered so we may want to remove it from the view.
			if (It < (uint32)PublishProgress)
			{
				ItemsPendingPriorityFilter.Add(It);
			}
			MarkItemRemoved(It);	
			return;
		}
	}
}

void FAssetViewItemCollection::ResetFilterState()
{
	check(!TextFilterTask.IsValid());

	if (bItemsPendingRemove)
	{
		for (int32 i=0; i < Items.Num(); ++i)
		{
			if (FilterState[i].Removed)
			{
				Items[i].Reset();
			}
		}
		bItemsPendingRemove = false;
	}

	ShouldCancelTextFiltering = true;

	ItemsPendingPriorityPublish.Reset();
	FilterState.Reset();
	FilterState.AddZeroed(Items.Num());
	FrontendFilterProgress = 0;
	PublishProgress = 0;
	TextFilterProgress = 0;

	// Recreate the Removed flag if necessary after FilterState was wiped so that we know which items are expected to be null
	if (Items.Num() != NumValidItems)
	{
		for (int32 i=0; i < Items.Num(); ++i)
		{
			if (!Items[i].IsValid())
			{
				FilterState[i].Removed = true;
			}
		}
	}		
}

void FAssetViewItemCollection::AbortTextFiltering()
{
	ShouldCancelTextFiltering = true;
	// Wait until the task sees the flag and doesn't spawn a continuation
	while (TextFilterTask.IsValid())
	{
		TextFilterTask.Wait();
		TextFilterTask = TextFilterTask.GetResult().Next;
	}
}

void FAssetViewItemCollection::StartTextFiltering(TSharedPtr<FAssetTextFilter> TextFilter)
{
	// Text filter task reads CompiledTextFilter so must not be running here
	check(!TextFilterTask.IsValid());

	ShouldCancelTextFiltering = false;
	if (!TextFilter.IsValid() || TextFilter->IsEmpty())
	{
		CompiledTextFilter.Reset();
		bCompiledTextFilterRequiresVersePaths = false;
		bAllItemsPassedTextFilter = true;
		return;
	}

	CompiledTextFilter = TextFilter->Compile();
	bCompiledTextFilterRequiresVersePaths = bShowingContentVersePath;
	bAllItemsPassedTextFilter = false;

	if (UE::AssetView::AllowAsync)
	{
		// Verse paths need to be computed on the game thread at the moment.
		if (bCompiledTextFilterRequiresVersePaths)
		{
			for (const TSharedPtr<FAssetViewItem>& Item : Items)
			{
				if (Item.IsValid())
				{
					Item->CacheVersePathTextForTextFiltering();
				}
			}
		}

		const int32 MaxItemsPerTask = UE::AssetView::GetMaxTextFilterItemBatch();
		TextFilterTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, MaxItemsPerTask]() {
			return AsyncFilterText(0, MaxItemsPerTask);
		});
	}
}

FAssetViewItemCollection::FTextFilterResult FAssetViewItemCollection::AsyncFilterText(int32 StartIndex, int32 InMaxItems) const
{
	if (ShouldCancelTextFiltering)
	{
		return FTextFilterResult{ StartIndex, {}, {} };
	}

	FReadScopeLock Guard(Lock);

	// How many items to filter in between checking for interruption and allowing other threads to acquire the lock
	int32 NumItemsToFilter = FMath::Min(InMaxItems, Items.Num() - StartIndex);

	TBitArray<> MergedResult;
	MergedResult.Add(false, NumItemsToFilter);

	constexpr int32 MinThreadWorkSize = 1024;
	TArray<FCompiledAssetTextFilter> Contexts;
	auto CreateContext = [this](int32 ContextIndex, int32 NumContexts) {
		return CompiledTextFilter->CloneForThreading();
	};
	auto DoWork = [&MergedResult, StartIndex, this](FCompiledAssetTextFilter& Filter, int32 TaskIndex) {
		const TSharedPtr<FAssetViewItem>& Item = Items[StartIndex + TaskIndex];
		bool bPasses = Item.IsValid() &&
			(bCompiledTextFilterRequiresVersePaths ?
			Filter.PassesFilter(Item->GetItem(), Item->GetVersePathTextForTextFiltering().ToString()) :
			Filter.PassesFilter(Item->GetItem()));
		if (bPasses)
		{
			MergedResult[TaskIndex].AtomicSet(true);
		}
	};
	ParallelForWithTaskContext(TEXT("AssetViewTextFiltering"), Contexts, NumItemsToFilter, MinThreadWorkSize, CreateContext,
		DoWork, UE::AssetView::AllowParallelism ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

	int32 NextStartIndex = StartIndex + NumItemsToFilter;
	// Need to check termination condition while holding the lock
	bool bContinue = NextStartIndex < Items.Num() && UE::AssetView::AllowAsync && !ShouldCancelTextFiltering; 
	UE::Tasks::TTask<FTextFilterResult> NextTask;
	if (bContinue)
	{
		return FTextFilterResult{ StartIndex, MoveTemp(MergedResult),
			UE::Tasks::Launch(
				UE_SOURCE_LOCATION, [this, NextStartIndex, InMaxItems]() { return AsyncFilterText(NextStartIndex, InMaxItems); }) };
	}
	else
	{
		return FTextFilterResult{ StartIndex, MoveTemp(MergedResult), {} };
	}
}

void FAssetViewItemCollection::UpdateItemFiltering(
	FAssetViewFrontendFilterHelper& InHelper, double InEndTime, TArray<TSharedPtr<FAssetViewItem>>& OutItems)
{
	if (ItemsPendingPriorityFilter.Num())
	{
		PerformPriorityFiltering(InHelper, OutItems);
	}

	const bool bNeedsQueryFilter = InHelper.NeedsQueryFilter();
	do 
	{
		constexpr int FilterBatchSize = 128;
		const int32 End = FMath::Min(FrontendFilterProgress + FilterBatchSize, Items.Num());

		// Query filter
		if (bNeedsQueryFilter)
		{
			for (int32 Index = FrontendFilterProgress; Index < End; ++Index)
			{
				if (FilterState[Index].Removed || FilterState[Index].PriorityFiltered)
				{
					continue;
				}
				if (!InHelper.DoesItemPassQueryFilter(Items[Index]))
				{
					// Failing this filter is equivalent to not being returned from the backend
					MarkItemRemoved(Index);
				}
			}
		}
		for (int32 Index = FrontendFilterProgress; Index < End; ++Index)
		{
			if (FilterState[Index].Removed || FilterState[Index].PriorityFiltered)
			{
				continue;
			}

			if (InHelper.DoesItemPassFrontendFilter(Items[Index]))
			{
				FilterState[Index].PassedFrontendFilter = true;
			}
		}

		if (!bAllItemsPassedTextFilter)
		{
			check(CompiledTextFilter.IsValid());
			// Result must be moved in so TextFilterTask can be overwritten 
			auto MergeTextFilterResult = [this](FTextFilterResult Result) {
				TextFilterTask = MoveTemp(Result.Next);
				TextFilterProgress = Result.StartIndex + Result.Results.Num();
			
				for (TConstSetBitIterator<> It(Result.Results); It; ++It)
				{
					int32 ItemIndex = Result.StartIndex + It.GetIndex();
					if (!FilterState[ItemIndex].PriorityFiltered)
					{
						FilterState[ItemIndex].PassedTextFilter = true;
					}
				}
			};

			if (TextFilterTask.IsValid() && TextFilterTask.IsCompleted())
			{
				MergeTextFilterResult(MoveTemp(TextFilterTask.GetResult()));
			}

			auto CacheVersePathTextForTextFiltering = [this]()
			{
				// Verse paths need to be computed on the game thread at the moment.
				if (bCompiledTextFilterRequiresVersePaths)
				{
					for (int32 Index = TextFilterProgress; Index < Items.Num(); ++Index)
					{
						if (Items[Index].IsValid())
						{
							Items[Index]->CacheVersePathTextForTextFiltering();
						}
					}
				}
			};

			if (!TextFilterTask.IsValid() && TextFilterProgress < Items.Num() && UE::AssetView::AllowAsync)
			{
				CacheVersePathTextForTextFiltering();

				// New elements were added after the text filter attempted to launch a continuation task
				TextFilterTask = UE::Tasks::Launch(UE_SOURCE_LOCATION,
					[this, StartIndex = TextFilterProgress]() {
						return AsyncFilterText(StartIndex, UE::AssetView::GetMaxTextFilterItemBatch());
					});
			}
			
			// In case flag was flipped while filtering was running, wait til task ends before performing text filtering on the game thread.
			if (!UE::AssetView::AllowAsync && !TextFilterTask.IsValid() && TextFilterProgress < Items.Num())
			{
				CacheVersePathTextForTextFiltering();

				FTextFilterResult Result = AsyncFilterText(TextFilterProgress, UE::AssetView::GetMaxTextFilterItemBatch());
				MergeTextFilterResult(Result);
			}
		}

		FrontendFilterProgress = End;
	} while(FPlatformTime::Seconds() < InEndTime && FrontendFilterProgress < Items.Num());

	// Append items which have passed both text and frontend filtering to OutItems
	int32 CanPublish = FMath::Min(FrontendFilterProgress, bAllItemsPassedTextFilter ? Items.Num() : TextFilterProgress);
	for (int32 i = PublishProgress; i < CanPublish; ++i)
	{
		if (!FilterState[i].PriorityFiltered && ensureMsgf(!FilterState[i].Published, TEXT("Standard-publish item %d was already published. PublishProgress: %d CanPublish: %d"), i, PublishProgress, CanPublish))
		{
			const bool bPublish = ItemPassedAllFilters(i);
			FilterState[i].Published = bPublish;
			if (bPublish)
			{
				OutItems.Add(Items[i]);
			}
		}
	}

	// Publish items which originally failed filtering and then were updated to pass it
	for (int32 Index : ItemsPendingPriorityPublish)
	{
		if (ensureMsgf(!FilterState[Index].Published, TEXT("Priority-publish item %d was already published. PublishProgress: %d CanPublish: %d"), Index, PublishProgress, CanPublish))
		{
			// Check we didn't update the state again to failure or removal
			const bool bPublish = ItemPassedAllFilters(Index);
			FilterState[Index].Published = bPublish;
			if (bPublish)
			{
				OutItems.Add(Items[Index]);
			}
		}
	}
	ItemsPendingPriorityPublish.Reset();

	PublishProgress = CanPublish;
}

bool FAssetViewItemCollection::PerformPriorityFiltering(FAssetViewFrontendFilterHelper& Helper,
	TArray<TSharedPtr<FAssetViewItem>>& FilteredAssetItems)
{
	int32 PrevNum = FilteredAssetItems.Num();
	if (ItemsPendingPriorityFilter.Num())
	{
		const bool bRunQueryFilter = Helper.NeedsQueryFilter();
		if (bRunQueryFilter)
		{
			for (int32 Index : ItemsPendingPriorityFilter)
			{
				if (FilterState[Index].Removed)
				{
					continue;
				}
				if (!Helper.DoesItemPassQueryFilter(Items[Index]))
				{
					// Failing this filter is equivalent to not being returned from the backend
					MarkItemRemoved(Index);
				}
			}
		}
		for (int32 Index : ItemsPendingPriorityFilter)
		{
			if (FilterState[Index].Removed)
			{
				continue;
			}

			// This may hide an item which was shown or show an item which was hidden - later we will check if we can remove without a re-sort, or if we need to add and re-sort
			FilterState[Index].PassedFrontendFilter = Helper.DoesItemPassFrontendFilter(Items[Index]);
		}

		if (!bAllItemsPassedTextFilter)
		{
			check(CompiledTextFilter.IsValid());
			// TODO: Is it possible we get a very large update from the backend for items which have already been filtered? So should we launch tasks to do this? 
			for (int32 Index : ItemsPendingPriorityFilter)
			{
				if (FilterState[Index].Removed)
				{
					continue;
				}

				bool bPassed = bCompiledTextFilterRequiresVersePaths ?
					CompiledTextFilter->PassesFilter(Items[Index]->GetItem(), Items[Index]->GetVersePathTextForTextFiltering().ToString()) :
					CompiledTextFilter->PassesFilter(Items[Index]->GetItem());
				FilterState[Index].PassedTextFilter = bPassed;
			}
		}
		
		TSet<TSharedPtr<FAssetViewItem>> ToRemove;
		for (int32 Index : ItemsPendingPriorityFilter)
		{
			if (Index >= PublishProgress)
			{
				// If item has yet to be published in the normal order, just leave the filter results for UpdateItemFiltering
				continue;
			}

			// Only set flag if we're taking control of this item's publish state
			FilterState[Index].PriorityFiltered = true;
			const bool bPublish = !FilterState[Index].Removed && FilterState[Index].PassedFrontendFilter
				&& (bAllItemsPassedTextFilter || FilterState[Index].PassedTextFilter);

			if (FilterState[Index].Published && !bPublish)
			{
				// Remove item while maintaining sorting of remaining items
				ToRemove.Add(Items[Index]);
			}
			else if (!FilterState[Index].Published && bPublish)
			{
				// Newly passing item - this means the view will have to be re-sorted 
				// Defer addition until UpdateItemFiltering
				ItemsPendingPriorityPublish.Add(Index);
			}
		}

		FilteredAssetItems.SetNum(Algo::StableRemoveIf(
			FilteredAssetItems, [&ToRemove](const TSharedPtr<FAssetViewItem> Item) { return ToRemove.Contains(Item); }));

		ItemsPendingPriorityFilter.Reset();
	}
	
	if (bItemsPendingRemove)
	{
		FWriteScopeLock Guard(Lock);
		for (int32 i=0; i < Items.Num(); ++i)
		{
			if (FilterState[i].Removed && Items[i].IsValid())
			{
				Items[i].Reset();
			}
		}
		bItemsPendingRemove = false;
	}
	return PrevNum != FilteredAssetItems.Num();
}

bool FAssetViewItemCollection::UpdateShowingContentVersePath()
{
	bool bInvalidateAssetPathText = false;

	const bool bNewShowingContentVersePath = IAssetTools::Get().ShowingContentVersePath();
	if (bShowingContentVersePath != bNewShowingContentVersePath)
	{
		bShowingContentVersePath = bNewShowingContentVersePath;

		bInvalidateAssetPathText = true;
	}

	if (bShouldInvalidateVersePaths)
	{
		bShouldInvalidateVersePaths = false;

		if (bShowingContentVersePath)
		{
			bInvalidateAssetPathText = true;
		}
	}

	if (bInvalidateAssetPathText)
	{
		// Lock since we may update a Verse path that the background text search is reading.
		FWriteScopeLock Guard(Lock);
		for (const TSharedPtr<FAssetViewItem>& Item : Items)
		{
			if (Item.IsValid())
			{
				Item->ConfigureAssetPathText(bShowingContentVersePath);
				Item->BroadcastItemDataChanged();
			}
		}
		return true;
	}

	return false;
}

/** 
 * SAssetView
 */
SAssetView::SAssetView()
: Items(MakePimpl<FAssetViewItemCollection>())
{

}

SAssetView::~SAssetView()
{
	if (IContentBrowserDataModule* ContentBrowserDataModule = IContentBrowserDataModule::GetPtr())
	{
		if (UContentBrowserDataSubsystem* ContentBrowserData = ContentBrowserDataModule->GetSubsystem())
		{
			ContentBrowserData->OnItemDataUpdated().RemoveAll(this);
			ContentBrowserData->OnItemDataRefreshed().RemoveAll(this);
			ContentBrowserData->OnItemDataDiscoveryComplete().RemoveAll(this);
		}
	}

	// Remove the listener for when view settings are changed
	UContentBrowserSettings::OnSettingChanged().RemoveAll(this);

	TSet<TSharedPtr<ICollectionContainer>> UniqueCollectionContainers;
	for (const FCollectionRef& Collection : ContentSources.GetCollections())
	{
		bool bIsAlreadyInSet = false;
		UniqueCollectionContainers.Add(Collection.Container, &bIsAlreadyInSet);
		if (!bIsAlreadyInSet)
		{
			Collection.Container->OnAssetsAddedToCollection().RemoveAll(this);
			Collection.Container->OnAssetsRemovedFromCollection().RemoveAll(this);
			Collection.Container->OnCollectionRenamed().RemoveAll(this);
			Collection.Container->OnCollectionUpdated().RemoveAll(this);
		}
	}

	if ( FrontendFilters.IsValid() )
	{
		// Clear the frontend filter changed delegate
		FrontendFilters->OnChanged().RemoveAll( this );
	}

	if (TextFilter.IsValid())
	{
		TextFilter->OnChanged().RemoveAll(this);
	}
}


void SAssetView::Construct( const FArguments& InArgs )
{
	ViewCorrelationGuid = FGuid::NewGuid();	

	InitialNumAmortizedTasks = 0;
	TotalAmortizeTime = 0;
	AmortizeStartTime = 0;
	MaxSecondsPerFrame = 0.015f;

	bFillEmptySpaceInTileView = InArgs._FillEmptySpaceInTileView;
	FillScale = 1.0f;

	bShowRedirectors = InArgs._ShowRedirectors;
	bLastShowRedirectors = bShowRedirectors.Get(false);

	ThumbnailHintFadeInSequence.JumpToStart();
	ThumbnailHintFadeInSequence.AddCurve(0, 0.5f, ECurveEaseFunction::Linear);

	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	ContentBrowserData->OnItemDataUpdated().AddSP(this, &SAssetView::HandleItemDataUpdated);
	ContentBrowserData->OnItemDataRefreshed().AddSP(this, &SAssetView::RequestSlowFullListRefresh);
	ContentBrowserData->OnItemDataDiscoveryComplete().AddSP(this, &SAssetView::HandleItemDataDiscoveryComplete);
	FilterCacheID.Initialaze(ContentBrowserData);

	// Listen for when view settings are changed
	UContentBrowserSettings::OnSettingChanged().AddSP(this, &SAssetView::HandleSettingChanged);

	ThumbnailSizes = TMap<EAssetViewType::Type, EThumbnailSize>
	{
		{EAssetViewType::List, InArgs._InitialThumbnailSize},
		{EAssetViewType::Tile, InArgs._InitialThumbnailSize},
		// Force only the column default to be tiny like the older CB
		{EAssetViewType::Column, EThumbnailSize::Tiny},
		{EAssetViewType::Custom, InArgs._InitialThumbnailSize},
		// Set a default for this case even though it should never land here
		{EAssetViewType::MAX, InArgs._InitialThumbnailSize}
	};

	// Get desktop metrics
	FDisplayMetrics DisplayMetrics;
	FSlateApplication::Get().GetCachedDisplayMetrics( DisplayMetrics );

	const FIntPoint DisplaySize(
		DisplayMetrics.PrimaryDisplayWorkAreaRect.Right - DisplayMetrics.PrimaryDisplayWorkAreaRect.Left,
		DisplayMetrics.PrimaryDisplayWorkAreaRect.Bottom - DisplayMetrics.PrimaryDisplayWorkAreaRect.Top );

	ThumbnailScaleRangeScalar = (float)DisplaySize.Y / 2160.f;

	// Use the shared ThumbnailPool for the rendering of thumbnails
	AssetThumbnailPool = UThumbnailManager::Get().GetSharedThumbnailPool();
	NumOffscreenThumbnails = 64;
	ListViewThumbnailResolution = 256;
	ListViewThumbnailPadding = UE::Editor::ContentBrowser::IsNewStyleEnabled() ? 2 : 4;
	TileViewThumbnailResolution = 256;
	TileViewThumbnailPadding = 9;

	// Max Size for the thumbnail
	const int32 MaxTileViewThumbnailSize = UE::Editor::ContentBrowser::IsNewStyleEnabled() ? 160 : 150;

	TileViewThumbnailSize = MaxTileViewThumbnailSize;

	const int32 MaxListViewThumbnailViewSize = UE::Editor::ContentBrowser::IsNewStyleEnabled() ? 160 : 64;

	ListViewThumbnailSize = MaxListViewThumbnailViewSize;

	TileViewNameHeight = 50;

	// Need to assign the ViewType before updating the ThumbnailSizeValue
	if (InArgs._InitialViewType >= 0 && InArgs._InitialViewType < EAssetViewType::MAX)
	{
		CurrentViewType = InArgs._InitialViewType;
	}
	else
	{
		CurrentViewType = EAssetViewType::Tile;
	}

	UpdateThumbnailSizeValue();
	MinThumbnailScale = 0.2f * ThumbnailScaleRangeScalar;
	MaxThumbnailScale = 1.9f * ThumbnailScaleRangeScalar;

	SortManager = MakeShared<FAssetViewSortManager>();

	bCanShowClasses = InArgs._CanShowClasses;

	bCanShowFolders = InArgs._CanShowFolders;
	
	bCanShowReadOnlyFolders = InArgs._CanShowReadOnlyFolders;

	bFilterRecursivelyWithBackendFilter = InArgs._FilterRecursivelyWithBackendFilter;
		
	bCanShowRealTimeThumbnails = InArgs._CanShowRealTimeThumbnails;

	bCanShowDevelopersFolder = InArgs._CanShowDevelopersFolder;

	bCanShowFavorites = InArgs._CanShowFavorites;

	SelectionMode = InArgs._SelectionMode;

	bShowPathInColumnView = InArgs._ShowPathInColumnView;
	bShowTypeInColumnView = InArgs._ShowTypeInColumnView;
	bShowAssetAccessSpecifier = InArgs._ShowAssetAccessSpecifier;
	bSortByPathInColumnView = bShowPathInColumnView && InArgs._SortByPathInColumnView;
	bShowTypeInTileView = InArgs._ShowTypeInTileView;
	bForceShowEngineContent = InArgs._ForceShowEngineContent;
	bForceShowPluginContent = InArgs._ForceShowPluginContent;
	bForceHideScrollbar = InArgs._ForceHideScrollbar;
	bShowDisallowedAssetClassAsUnsupportedItems = InArgs._ShowDisallowedAssetClassAsUnsupportedItems;

	bPendingUpdateThumbnails = false;
	bShouldNotifyNextAssetSync = true;
	CurrentThumbnailSize = TileViewThumbnailSize;

	ContentSources = InArgs._InitialContentSources;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (ContentSources.IsEmpty() && !InArgs._InitialSourcesData.IsEmpty())
	{
		// Fall back to the InitialSourcesData field for backwards compatibility.

		TArray<FCollectionRef> Collections;
		Collections.Reserve(InArgs._InitialSourcesData.Collections.Num());
		Algo::Transform(
			InArgs._InitialSourcesData.Collections,
			Collections,
			[](const FCollectionNameType& Collection) { return FCollectionRef(FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(), Collection); });

		ContentSources = FAssetViewContentSources(InArgs._InitialSourcesData.VirtualPaths, MoveTemp(Collections));

		ContentSources.OnEnumerateCustomSourceItemDatas = InArgs._InitialSourcesData.OnEnumerateCustomSourceItemDatas;
		ContentSources.bIncludeVirtualPaths = InArgs._InitialSourcesData.IsIncludingVirtualPaths();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TSet<TSharedPtr<ICollectionContainer>> UniqueCollectionContainers;
	for (const FCollectionRef& Collection : ContentSources.GetCollections())
	{
		bool bIsAlreadyInSet = false;
		UniqueCollectionContainers.Add(Collection.Container, &bIsAlreadyInSet);
		if (!bIsAlreadyInSet)
		{
			Collection.Container->OnAssetsAddedToCollection().AddSP(this, &SAssetView::OnAssetsAddedToCollection);
			Collection.Container->OnAssetsRemovedFromCollection().AddSP(this, &SAssetView::OnAssetsRemovedFromCollection);
			Collection.Container->OnCollectionRenamed().AddSP(this, &SAssetView::OnCollectionRenamed);
			Collection.Container->OnCollectionUpdated().AddSP(this, &SAssetView::OnCollectionUpdated);
		}
	}
	BackendFilter = InArgs._InitialBackendFilter;

	FrontendFilters = InArgs._FrontendFilters;
	if (FrontendFilters.IsValid())
	{
		FrontendFilters->OnChanged().AddSP(this, &SAssetView::OnFrontendFiltersChanged);
	}
	TextFilter = InArgs._TextFilter;
	if (TextFilter.IsValid())
	{
		TextFilter->OnChanged().AddSP(this, &SAssetView::OnFrontendFiltersChanged);
	}

	OnShouldFilterAsset = InArgs._OnShouldFilterAsset;
	OnShouldFilterItem = InArgs._OnShouldFilterItem;

	OnNewItemRequested = InArgs._OnNewItemRequested;
	OnItemSelectionChanged = InArgs._OnItemSelectionChanged;
	OnItemsActivated = InArgs._OnItemsActivated;
	OnItemsDragged = InArgs._OnItemsDragged;
	OnGetItemContextMenu = InArgs._OnGetItemContextMenu;
	OnItemRenameCommitted = InArgs._OnItemRenameCommitted;
	OnAssetTagWantsToBeDisplayed = InArgs._OnAssetTagWantsToBeDisplayed;
	OnIsAssetValidForCustomToolTip = InArgs._OnIsAssetValidForCustomToolTip;
	OnGetCustomAssetToolTip = InArgs._OnGetCustomAssetToolTip;
	OnVisualizeAssetToolTip = InArgs._OnVisualizeAssetToolTip;
	OnAssetToolTipClosing = InArgs._OnAssetToolTipClosing;
	OnGetCustomSourceAssets = InArgs._OnGetCustomSourceAssets;
	HighlightedText = InArgs._HighlightedText;
	ThumbnailLabel = InArgs._ThumbnailLabel;
	AllowThumbnailHintLabel = InArgs._AllowThumbnailHintLabel;
	InitialCategoryFilter = InArgs._InitialCategoryFilter;
	AssetShowWarningText = InArgs._AssetShowWarningText;
	AssetShowWarningTextJustification = InArgs._AssetShowWarningTextJustification;
	AssetShowWarningTextSelectable = InArgs._AssetShowWarningTextSelectable;
	bAllowDragging = InArgs._AllowDragging;
	bAllowFocusOnSync = InArgs._AllowFocusOnSync;
	HiddenColumnNames = DefaultHiddenColumnNames = InArgs._HiddenColumnNames;
	ListHiddenColumnNames = DefaultListHiddenColumnNames = InArgs._ListHiddenColumnNames;
	CustomColumns = InArgs._CustomColumns;
	OnColumnVisibilityChanged = InArgs._OnColumnVisibilityChanged;
	OnSearchOptionsChanged = InArgs._OnSearchOptionsChanged;
	bShowPathViewFilters = InArgs._bShowPathViewFilters;
	OnExtendAssetViewOptionsMenuContext = InArgs._OnExtendAssetViewOptionsMenuContext;
	AssetViewOptionsProfile = InArgs._AssetViewOptionsProfile;

	bPendingSortFilteredItems = false;
	bQuickFrontendListRefreshRequested = false;
	bSlowFullListRefreshRequested = false;
	LastSortTime = 0;
	SortDelaySeconds = 8;

	bBulkSelecting = false;
	bAllowThumbnailEditMode = InArgs._AllowThumbnailEditMode;
	bThumbnailEditMode = false;
	bUserSearching = false;
	bPendingFocusOnSync = false;
	bWereItemsRecursivelyFiltered = false;

	OwningContentBrowser = InArgs._OwningContentBrowser;

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetClassPermissionList = AssetToolsModule.Get().GetAssetClassPathPermissionList(EAssetClassAction::ViewAsset);
	FolderPermissionList = AssetToolsModule.Get().GetFolderPermissionList();
	WritableFolderPermissionList = AssetToolsModule.Get().GetWritableFolderPermissionList();

	if(InArgs._AllowCustomView)
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		if(ContentBrowserModule.GetContentBrowserViewExtender().IsBound())
		{
			ViewExtender = ContentBrowserModule.GetContentBrowserViewExtender().Execute();

			// Bind the delegates the custom view is responsible for firing
			if(ViewExtender)
			{
				ViewExtender->OnSelectionChanged().BindSP(this, &SAssetView::AssetSelectionChanged);
				ViewExtender->OnContextMenuOpened().BindSP(this, &SAssetView::OnGetContextMenuContent);
				ViewExtender->OnItemScrolledIntoView().BindSP(this, &SAssetView::ItemScrolledIntoView);
				ViewExtender->OnItemDoubleClicked().BindSP(this, &SAssetView::OnListMouseButtonDoubleClick);
			}
		}
	}
	
	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");
	TSharedRef<SWidget> AssetDiscoveryIndicator = EditorWidgetsModule.CreateAssetDiscoveryIndicator(EAssetDiscoveryIndicatorScaleMode::Scale_Vertical);

	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	BindCommands();

	ChildSlot
	.Padding(0.0f)
	[
		SNew(SBorder)
		.Padding(0.f)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		[
			VerticalBox
		]
	];

	// Assets area
	VerticalBox->AddSlot()
	.FillHeight(1.f)
	[
		SNew( SVerticalBox ) 

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew( SBox )
			.Visibility_Lambda([this] { return InitialNumAmortizedTasks > 0 ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed; })
			.HeightOverride( 2.f )
			[
				SNew( SProgressBar )
				.Percent( this, &SAssetView::GetIsWorkingProgressBarState )
				.BorderPadding( FVector2D(0,0) )
			]
		]
		
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SAssignNew(ViewContainer, SBox)
				.Padding(UE::Editor::ContentBrowser::IsNewStyleEnabled() ? FMargin(6.0f, 0.0f) : 6.0f)
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0, 14, 0, 0))
			[
				SNew(SScrollBox)
				.Visibility( this, &SAssetView::IsAssetShowWarningTextVisible )
				+SScrollBox::Slot()
				.Padding(8.f)
				[
					// A warning to display when there are no assets to show, allow text copy
					SNew(SMultiLineEditableText)
					.IsReadOnly(true)
					.Justification(AssetShowWarningTextJustification)
					.Text(this, &SAssetView::GetAssetShowWarningText)
					.AutoWrapText(true)
					.IsEnabled(AssetShowWarningTextSelectable)
				]
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Bottom)
			.Padding(FMargin(24, 0, 24, 0))
			[
				// Asset discovery indicator
				AssetDiscoveryIndicator
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(FMargin(8, 0))
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ErrorReporting.EmptyBox"))
				.BorderBackgroundColor(this, &SAssetView::GetQuickJumpColor)
				.Visibility(this, &SAssetView::IsQuickJumpVisible)
				[
					SNew(STextBlock)
					.Text(this, &SAssetView::GetQuickJumpTerm)
				]
			]
		]
	];

	// Thumbnail edit mode banner
	VerticalBox->AddSlot()
	.AutoHeight()
	.Padding(0.f, 4.f)
	[
		SNew(SBorder)
		.Visibility( this, &SAssetView::GetEditModeLabelVisibility )
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Content()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(4.f, 0.f, 0.f, 0.f)
			.FillWidth(1.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ThumbnailEditModeLabel", "Editing Thumbnails. Drag a thumbnail to rotate it if there is a 3D environment."))
				.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.Primary"))
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("EndThumbnailEditModeButton", "Done Editing"))
				.OnClicked(this, &SAssetView::EndThumbnailEditModeClicked)
			]
		]
	];

	if (InArgs._ShowBottomToolbar)
	{
		const TSharedRef<SHorizontalBox> BottomToolBarBox = SNew(SHorizontalBox);

		if (!UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			// Asset count
			BottomToolBarBox->AddSlot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			.Padding(8, 5)
			[
				SNew(STextBlock)
				.Text(this, &SAssetView::GetAssetCountText)
			];
		}

		// View mode combo button
		BottomToolBarBox->AddSlot()
		.AutoWidth()
		[
			SNew(SComboButton)
			.Visibility(InArgs._ShowViewOptions ? EVisibility::Visible : EVisibility::Collapsed)
			.ContentPadding(0.f)
			.ButtonStyle( FAppStyle::Get(), "ToggleButton" ) // Use the tool bar item style for this button
			.OnGetMenuContent( this, &SAssetView::GetViewButtonContent )
			.ButtonContent()
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image( FAppStyle::GetBrush("GenericViewButton") )
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f, 0.f, 0.f, 0.f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text( LOCTEXT("ViewButton", "View Options") )
				]
			]
		];

		// Bottom panel
		VerticalBox->AddSlot()
		.AutoHeight()
		[
			BottomToolBarBox
		];
	}

	CreateCurrentView();

	if( InArgs._InitialAssetSelection.IsValid() )
	{
		// sync to the initial item without notifying of selection
		bShouldNotifyNextAssetSync = false;
		SyncToLegacy( MakeArrayView(&InArgs._InitialAssetSelection, 1), TArrayView<const FString>() );
	}

	// If currently looking at column, and you could choose to sort by path in column first and then name
	// Generalizing this is a bit difficult because the column ID is not accessible or is not known
	// Currently I assume this won't work, if this view mode is not column. Otherwise, I don't think sorting by path
	// is a good idea. 
	if (CurrentViewType == EAssetViewType::Column && bSortByPathInColumnView)
	{
		SortManager->SetSortColumnId(EColumnSortPriority::Primary, SortManager->PathColumnId);
		SortManager->SetSortColumnId(EColumnSortPriority::Secondary, SortManager->NameColumnId);
		SortManager->SetSortMode(EColumnSortPriority::Primary, EColumnSortMode::Ascending);
		SortManager->SetSortMode(EColumnSortPriority::Secondary, EColumnSortMode::Ascending);
		SortList();
	}
}

TOptional< float > SAssetView::GetIsWorkingProgressBarState() const
{
	if (Items->HasItemsPendingFilter())
	{
		return static_cast<float>(Items->GetFilterProgress()) / static_cast<float>(Items->Num());
	}
	return 0.0f;
}

void SAssetView::SetContentSources(const FAssetViewContentSources& InContentSources)
{
	TSet<TSharedPtr<ICollectionContainer>> OldCollectionContainers;
	for (const FCollectionRef& Collection : ContentSources.GetCollections())
	{
		OldCollectionContainers.Add(Collection.Container);
	}

	// Update the path and collection lists
	ContentSources = InContentSources;

	TSet<TSharedPtr<ICollectionContainer>> NewCollectionContainers;
	for (const FCollectionRef& Collection : ContentSources.GetCollections())
	{
		NewCollectionContainers.Add(Collection.Container);
	}

	for (const TSharedPtr<ICollectionContainer>& CollectionContainer : NewCollectionContainers)
	{
		if (OldCollectionContainers.Remove(CollectionContainer))
		{
			continue;
		}

		CollectionContainer->OnAssetsAddedToCollection().AddSP(this, &SAssetView::OnAssetsAddedToCollection);
		CollectionContainer->OnAssetsRemovedFromCollection().AddSP(this, &SAssetView::OnAssetsRemovedFromCollection);
		CollectionContainer->OnCollectionRenamed().AddSP(this, &SAssetView::OnCollectionRenamed);
		CollectionContainer->OnCollectionUpdated().AddSP(this, &SAssetView::OnCollectionUpdated);
	}

	for (const TSharedPtr<ICollectionContainer>& CollectionContainer : OldCollectionContainers)
	{
		CollectionContainer->OnAssetsAddedToCollection().RemoveAll(this);
		CollectionContainer->OnAssetsRemovedFromCollection().RemoveAll(this);
		CollectionContainer->OnCollectionRenamed().RemoveAll(this);
		CollectionContainer->OnCollectionUpdated().RemoveAll(this);
	}

	RequestSlowFullListRefresh();
	ClearSelection();
}

void SAssetView::SetSourcesData(const FSourcesData& InSourcesData)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	TArray<FCollectionRef> Collections;
	Collections.Reserve(InSourcesData.Collections.Num());
	Algo::Transform(
		InSourcesData.Collections,
		Collections,
		[](const FCollectionNameType& Collection) { return FCollectionRef(FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(), Collection); });

	FAssetViewContentSources NewContentSources(InSourcesData.VirtualPaths, MoveTemp(Collections));

	NewContentSources.OnEnumerateCustomSourceItemDatas = InSourcesData.OnEnumerateCustomSourceItemDatas;
	NewContentSources.bIncludeVirtualPaths = InSourcesData.IsIncludingVirtualPaths();

	SetContentSources(NewContentSources);

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

const FAssetViewContentSources& SAssetView::GetContentSources() const
{
	return ContentSources;
}

FSourcesData SAssetView::GetSourcesData() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	FSourcesData SourcesData;

	SourcesData.VirtualPaths = ContentSources.GetVirtualPaths();
	Algo::TransformIf(
		ContentSources.GetCollections(),
		SourcesData.Collections,
		[](const FCollectionRef& Collection) { return Collection.Container == FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(); },
		[](const FCollectionRef& Collection) { return FCollectionNameType(Collection.Name, Collection.Type); });
	SourcesData.OnEnumerateCustomSourceItemDatas = ContentSources.OnEnumerateCustomSourceItemDatas;
	SourcesData.bIncludeVirtualPaths = ContentSources.IsIncludingVirtualPaths();

	return SourcesData;
	
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool SAssetView::IsAssetPathSelected() const
{
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

	TArray<FName> InternalPaths;
	InternalPaths.Reserve(ContentSources.GetVirtualPaths().Num());
	for (const FName& VirtualPath : ContentSources.GetVirtualPaths())
	{
		FName ConvertedPath;
		if (ContentBrowserData->TryConvertVirtualPath(VirtualPath, ConvertedPath) == EContentBrowserPathType::Internal)
		{
			InternalPaths.Add(ConvertedPath);
		}
	}

	int32 NumAssetPaths, NumClassPaths;
	ContentBrowserUtils::CountPathTypes(InternalPaths, NumAssetPaths, NumClassPaths);

	// Check that only asset paths are selected
	return NumAssetPaths > 0 && NumClassPaths == 0;
}

void SAssetView::SetBackendFilter(const FARFilter& InBackendFilter, TArray<TSharedRef<const FPathPermissionList>>* InCustomPermissionLists)
{
	using namespace UE::AssetView;
	// Sometimes "filter changed" notifications are broadcast for the content browser to rebuild its filtering when nothing actually changed
	// Notably custom text filters will do this. 
	// If we don't need to do a full refresh, don't bother. 
	if (AreBackendFiltersDifferent(BackendFilter, InBackendFilter) 
	|| AreCustomPermissionListsDifferent(InCustomPermissionLists, BackendCustomPathFilters))
	{
		BackendFilter = InBackendFilter;
		if (InCustomPermissionLists)
		{
			BackendCustomPathFilters = *InCustomPermissionLists;
		}
		else
		{
			BackendCustomPathFilters.Reset();
		}
		RequestSlowFullListRefresh();
	}
}

void SAssetView::AppendBackendFilter(FARFilter& FilterToAppendTo) const
{
	FilterToAppendTo.Append(BackendFilter);
}

void SAssetView::NewFolderItemRequested(const FContentBrowserItemTemporaryContext& NewItemContext)
{
	// Don't allow asset creation while renaming
	if (IsRenamingAsset())
	{
		return;
	}

	// we should only be creating one deferred folder at a time
	if (!ensureAlwaysMsgf(!DeferredItemToCreate.IsValid(), TEXT("Deferred new asset folder creation while there is already a deferred item creation: %s"), *NewItemContext.GetItem().GetItemName().ToString()))
	{
		if (DeferredItemToCreate->bWasAddedToView)
		{
			FContentBrowserItemKey ItemToRemoveKey(DeferredItemToCreate->ItemContext.GetItem());
			FilteredAssetItems.RemoveAll([&ItemToRemoveKey](const TSharedPtr<FAssetViewItem>& InAssetViewItem) { return ItemToRemoveKey == FContentBrowserItemKey(InAssetViewItem->GetItem()); });
			RefreshList();
		}

		DeferredItemToCreate.Reset();
	}


	// Folder creation requires focus to give object a name, otherwise object will not be created
	TSharedPtr<SWindow> OwnerWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (OwnerWindow.IsValid() && !OwnerWindow->HasAnyUserFocusOrFocusedDescendants())
	{
		FSlateApplication::Get().SetUserFocus(FSlateApplication::Get().GetUserIndexForKeyboard(), AsShared(), EFocusCause::SetDirectly);
	}

	// Notify that we're about to start creating this item, as we may need to do things like ensure the parent folder is visible
	OnNewItemRequested.ExecuteIfBound(NewItemContext.GetItem());

	// Defer folder creation until next tick, so we get a chance to refresh the view
	DeferredItemToCreate = MakeUnique<FCreateDeferredItemData>();
	DeferredItemToCreate->ItemContext = NewItemContext;

	UE_LOGF(LogContentBrowser, Log, "Deferred new asset folder creation: %ls", *NewItemContext.GetItem().GetItemName().ToString());
}

void SAssetView::NewFileItemRequested(const FContentBrowserItemDataTemporaryContext& NewItemContext)
{
	// Don't allow asset creation while renaming
	if (IsRenamingAsset())
	{
		return;
	}

	// We should only be creating one deferred file at a time
	if (!ensureAlwaysMsgf(!DeferredItemToCreate.IsValid(), TEXT("Deferred new asset file creation while there is already a deferred item creation: %s"), *NewItemContext.GetItemData().GetItemName().ToString()))
	{
		if (DeferredItemToCreate->bWasAddedToView)
		{
			FContentBrowserItemKey ItemToRemoveKey(DeferredItemToCreate->ItemContext.GetItem());
			FilteredAssetItems.RemoveAll([&ItemToRemoveKey](const TSharedPtr<FAssetViewItem>& InAssetViewItem){ return ItemToRemoveKey == FContentBrowserItemKey(InAssetViewItem->GetItem()); });
			RefreshList();
		}

		DeferredItemToCreate.Reset();
	}

	// File creation requires focus to give item a name, otherwise the item will not be created
	TSharedPtr<SWindow> OwnerWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (OwnerWindow.IsValid() && !OwnerWindow->HasAnyUserFocusOrFocusedDescendants())
	{
		FSlateApplication::Get().SetUserFocus(FSlateApplication::Get().GetUserIndexForKeyboard(), AsShared(), EFocusCause::SetDirectly);
	}

	// Notify that we're about to start creating this item, as we may need to do things like ensure the parent folder is visible
	if (OnNewItemRequested.IsBound())
	{
		OnNewItemRequested.Execute(FContentBrowserItem(NewItemContext.GetItemData()));
	}

	// Defer file creation until next tick, so we get a chance to refresh the view
	DeferredItemToCreate = MakeUnique<FCreateDeferredItemData>();
	DeferredItemToCreate->ItemContext.AppendContext(CopyTemp(NewItemContext));

	UE_LOGF(LogContentBrowser, Log, "Deferred new asset file creation: %ls", *NewItemContext.GetItemData().GetItemName().ToString());
}

void SAssetView::BeginCreateDeferredItem()
{
	if (DeferredItemToCreate.IsValid() && !DeferredItemToCreate->bWasAddedToView)
	{
		TSharedPtr<FAssetViewItem> NewItem = MakeShared<FAssetViewItem>(INDEX_NONE, DeferredItemToCreate->ItemContext.GetItem());
		NewItem->ConfigureAssetPathText(Items->GetShowingContentVersePath());
		AwaitingScrollIntoViewForRename = NewItem;
		DeferredItemToCreate->bWasAddedToView = true;

		FilteredAssetItems.Insert(NewItem, 0);
		SortManager->SortList(FilteredAssetItems, MajorityAssetType, CustomColumns);

		SetSelection(NewItem);
		RequestScrollIntoView(NewItem);

		UE_LOGF(LogContentBrowser, Log, "Creating deferred item: %ls", *NewItem->GetItem().GetItemName().ToString());
	}
}

FContentBrowserItem SAssetView::EndCreateDeferredItem(const TSharedPtr<FAssetViewItem>& InItem, const FString& InName, const bool bFinalize, FText& OutErrorText)
{
	FContentBrowserItem FinalizedItem;

	if (DeferredItemToCreate.IsValid() && DeferredItemToCreate->bWasAddedToView)
	{
		checkf(FContentBrowserItemKey(InItem->GetItem()) == FContentBrowserItemKey(DeferredItemToCreate->ItemContext.GetItem()), TEXT("DeferredItemToCreate was still set when attempting to rename a different item!"));

		// Remove the temporary item before we do any work to ensure the new item creation is not prevented
		Items->RemoveItem(InItem);
		FilteredAssetItems.Remove(InItem);
		RequestQuickFrontendListRefresh();
		RefreshList();

		// If not finalizing then we just discard the temporary
		if (bFinalize)
		{
			UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
			FScopedSuppressContentBrowserDataTick TickSuppression(ContentBrowserData);

			if (DeferredItemToCreate->ItemContext.ValidateItem(InName, &OutErrorText))
			{
				FinalizedItem = DeferredItemToCreate->ItemContext.FinalizeItem(InName, &OutErrorText);
			}
		}
	}

	// Always reset the deferred item to avoid having it dangle, which can lead to potential crashes.
	DeferredItemToCreate.Reset();

	UE_LOGF(LogContentBrowser, Log, "End creating deferred item %ls", *InItem->GetItem().GetItemName().ToString());

	return FinalizedItem;
}

void SAssetView::CreateNewAsset(const FString& DefaultAssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory)
{
	ContentBrowserDataLegacyBridge::OnCreateNewAsset().ExecuteIfBound(*DefaultAssetName, *PackagePath, AssetClass, Factory, UContentBrowserDataMenuContext_AddNewMenu::FOnBeginItemCreation::CreateSP(this, &SAssetView::NewFileItemRequested));
}

void SAssetView::RenameItem(const FContentBrowserItem& ItemToRename)
{
	if (const TSharedPtr<FAssetViewItem> Item = Items->FindItemForRename(ItemToRename))
	{
		CancelPendingRename();

		AwaitingScrollIntoViewForRename = Item;
		
		SetSelection(Item);
		RequestScrollIntoView(Item);
	}
}

void SAssetView::SyncToItems(TArrayView<const FContentBrowserItem> ItemsToSync, const bool bFocusOnSync)
{
	PendingSyncItems.Reset();

	for (const FContentBrowserItem& Item : ItemsToSync)
	{
		PendingSyncItems.SelectedVirtualPaths.Add(Item.GetVirtualPath());
	}
	InitDeferredPendingSyncItems();	
	bPendingFocusOnSync = bFocusOnSync;
}

void SAssetView::SyncToVirtualPaths(TArrayView<const FName> VirtualPathsToSync, const bool bFocusOnSync)
{
	PendingSyncItems.Reset();
	for (const FName& VirtualPathToSync : VirtualPathsToSync)
	{
		PendingSyncItems.SelectedVirtualPaths.Add(VirtualPathToSync);
	}
	InitDeferredPendingSyncItems();
	bPendingFocusOnSync = bFocusOnSync;
}

void SAssetView::SyncToLegacy(TArrayView<const FAssetData> AssetDataList, TArrayView<const FString> FolderList, const bool bFocusOnSync)
{
	SyncToItems(ContentBrowserUtils::ConvertLegacySelectionToItems(AssetDataList, FolderList), bFocusOnSync);
}

void SAssetView::InitDeferredPendingSyncItems()
{
	DeferredPendingSyncItems.AddMissingVirtualPaths(PendingSyncItems);
	DeferredSyncTimeoutFrames = UE::AssetView::DeferredSyncTimeoutFramesCount;
}

void SAssetView::SyncToSelection( const bool bFocusOnSync )
{
	PendingSyncItems.Reset();

	TArray<TSharedPtr<FAssetViewItem>> SelectedItems = GetSelectedViewItems();
	for (const TSharedPtr<FAssetViewItem>& Item : SelectedItems)
	{
		if (Item.IsValid())
		{
			PendingSyncItems.SelectedVirtualPaths.Add(Item->GetItem().GetVirtualPath());
		}
	}
	bPendingFocusOnSync = bFocusOnSync;
}

void SAssetView::ApplyHistoryData( const FHistoryData& History )
{
	SetContentSources(History.ContentSources);
	PendingSyncItems = History.SelectionData;
	bPendingFocusOnSync = true;
}

TArray<TSharedPtr<FAssetViewItem>> SAssetView::GetSelectedViewItems() const
{
	switch (GetCurrentViewType())
	{
		case EAssetViewType::List: return ListView->GetSelectedItems();
		case EAssetViewType::Tile: return TileView->GetSelectedItems();
		case EAssetViewType::Column: return ColumnView->GetSelectedItems();
		case EAssetViewType::Custom: return ViewExtender->GetSelectedItems();
		default:
		ensure(0); // Unknown list type
		return TArray<TSharedPtr<FAssetViewItem>>();
	}
}

TArray<FContentBrowserItem> SAssetView::GetSelectedItems() const
{
	TArray<TSharedPtr<FAssetViewItem>> SelectedViewItems = GetSelectedViewItems();

	TArray<FContentBrowserItem> SelectedItems;
	for (const TSharedPtr<FAssetViewItem>& SelectedViewItem : SelectedViewItems)
	{
		if (!SelectedViewItem->IsTemporary())
		{
			SelectedItems.Emplace(SelectedViewItem->GetItem());
		}
	}
	return SelectedItems;
}

TArray<FContentBrowserItem> SAssetView::GetSelectedFolderItems() const
{
	TArray<TSharedPtr<FAssetViewItem>> SelectedViewItems = GetSelectedViewItems();

	TArray<FContentBrowserItem> SelectedFolders;
	for (const TSharedPtr<FAssetViewItem>& SelectedViewItem : SelectedViewItems)
	{
		if (SelectedViewItem->IsFolder() && !SelectedViewItem->IsTemporary())
		{
			SelectedFolders.Emplace(SelectedViewItem->GetItem());
		}
	}
	return SelectedFolders;
}

TArray<FContentBrowserItem> SAssetView::GetSelectedFileItems() const
{
	TArray<TSharedPtr<FAssetViewItem>> SelectedViewItems = GetSelectedViewItems();

	TArray<FContentBrowserItem> SelectedFiles;
	for (const TSharedPtr<FAssetViewItem>& SelectedViewItem : SelectedViewItems)
	{
		if (SelectedViewItem->IsFile() && !SelectedViewItem->IsTemporary())
		{
			SelectedFiles.Emplace(SelectedViewItem->GetItem());
		}
	}
	return SelectedFiles;
}

TArray<FAssetData> SAssetView::GetSelectedAssets() const
{
	TArray<TSharedPtr<FAssetViewItem>> SelectedViewItems = GetSelectedViewItems();

	// TODO: Abstract away?
	TArray<FAssetData> SelectedAssets;
	for (const TSharedPtr<FAssetViewItem>& SelectedViewItem : SelectedViewItems)
	{
		// Only report non-temporary & non-folder items
		FAssetData ItemAssetData;
		if (!SelectedViewItem->IsTemporary() && SelectedViewItem->IsFile() && SelectedViewItem->GetItem().Legacy_TryGetAssetData(ItemAssetData))
		{
			SelectedAssets.Add(MoveTemp(ItemAssetData));
		}
	}
	return SelectedAssets;
}

TArray<FString> SAssetView::GetSelectedFolders() const
{
	TArray<TSharedPtr<FAssetViewItem>> SelectedViewItems = GetSelectedViewItems();

	// TODO: Abstract away?
	TArray<FString> SelectedFolders;
	for (const TSharedPtr<FAssetViewItem>& SelectedViewItem : SelectedViewItems)
	{
		if (SelectedViewItem->IsFolder())
		{
			SelectedFolders.Emplace(SelectedViewItem->GetItem().GetVirtualPath().ToString());
		}
	}
	return SelectedFolders;
}

void SAssetView::RequestSlowFullListRefresh()
{
	bSlowFullListRefreshRequested = true;
}

void SAssetView::RequestQuickFrontendListRefresh()
{
	bQuickFrontendListRefreshRequested = true;
}

FString SAssetView::GetThumbnailScaleSettingPath(const FString& SettingsString, const FString& InViewTypeString) const
{
	return SettingsString + TEXT(".ThumbnailSize") + InViewTypeString;
}

void SAssetView::LoadScaleSetting(const FString& IniFilename, const FString& IniSection, const FString& SettingsString, const FString& InViewTypeString, EThumbnailSize& OutThumbnailSize)
{
	int32 ThumbnailSizeConfig = (int32)EThumbnailSize::Medium;
	if (GConfig->GetInt(*IniSection, *GetThumbnailScaleSettingPath(SettingsString, InViewTypeString), ThumbnailSizeConfig, IniFilename))
	{
		// Clamp value to normal range and update state
		ThumbnailSizeConfig = FMath::Clamp<int32>(ThumbnailSizeConfig, 0, (int32)EThumbnailSize::MAX-1);

		// TODO: Remove this afterwards, current CB should hide new size
		if (!UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			if ((EThumbnailSize)ThumbnailSizeConfig == EThumbnailSize::XLarge)
			{
				ThumbnailSizeConfig -= 1;
			}
		}

		OutThumbnailSize = (EThumbnailSize)ThumbnailSizeConfig;
	}
}

FString SAssetView::GetCurrentViewTypeSettingPath(const FString& SettingsString) const
{
	return SettingsString + TEXT(".CurrentViewType");
}

void SAssetView::SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const
{
	// ThumbnailSize saves
	GConfig->SetInt(*IniSection, *GetThumbnailScaleSettingPath(SettingsString, GridViewSpecifier), (int32)ThumbnailSizes[EAssetViewType::Tile], IniFilename);
	GConfig->SetInt(*IniSection, *GetThumbnailScaleSettingPath(SettingsString, ListViewSpecifier), (int32)ThumbnailSizes[EAssetViewType::List], IniFilename);
	GConfig->SetInt(*IniSection, *GetThumbnailScaleSettingPath(SettingsString, CustomViewSpecifier), (int32)ThumbnailSizes[EAssetViewType::Custom], IniFilename);

	// Save the ThumbnailSize config for the ColumnView only in the new CB
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		GConfig->SetInt(*IniSection, *GetThumbnailScaleSettingPath(SettingsString, ColumnViewSpecifier), (int32)ThumbnailSizes[EAssetViewType::Column], IniFilename);
	}

	GConfig->SetInt(*IniSection, *GetCurrentViewTypeSettingPath(SettingsString), CurrentViewType, IniFilename);
	GConfig->SetFloat(*IniSection, *(SettingsString + TEXT(".ZoomScale")), ZoomScale, IniFilename);

	GConfig->SetArray(*IniSection, *(SettingsString + TEXT(".HiddenColumns")), HiddenColumnNames, IniFilename);
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		// Used to discern if at some point the ColumnVisibility changed, if true the LoadedColumns will always be used instead
		GConfig->SetBool(*IniSection, *(SettingsString + TEXT(".ListViewColumnsManuallyChangedOnce")), bListViewColumnsManuallyChangedOnce, IniFilename);
		GConfig->SetBool(*IniSection, *(SettingsString + TEXT(".ColumnViewColumnsManuallyChangedOnce")), bColumnViewColumnsManuallyChangedOnce, IniFilename);

		// ListView HiddenColumns
		GConfig->SetArray(*IniSection, *(SettingsString + TEXT(".ListHiddenColumns")), ListHiddenColumnNames, IniFilename);
	}
}

void SAssetView::LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString)
{
	// Set the LoadSetting flag to true
	TGuardValue<bool> ScopeGuard(bLoadingSettings, true);

	// ThumbnailSize loadings
	LoadScaleSetting(IniFilename, *IniSection, SettingsString, GridViewSpecifier, ThumbnailSizes[EAssetViewType::Tile]);
	LoadScaleSetting(IniFilename, *IniSection, SettingsString, ListViewSpecifier, ThumbnailSizes[EAssetViewType::List]);
	LoadScaleSetting(IniFilename, *IniSection, SettingsString, CustomViewSpecifier, ThumbnailSizes[EAssetViewType::Custom]);

	// Load the ThumbnailSize config for the ColumnView only in the new CB
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		LoadScaleSetting(IniFilename, *IniSection, SettingsString, ColumnViewSpecifier, ThumbnailSizes[EAssetViewType::Column]);
	}

	int32 ViewType = EAssetViewType::Tile;
	if (GConfig->GetInt(*IniSection, *GetCurrentViewTypeSettingPath(SettingsString), ViewType, IniFilename))
	{
		// Clamp value to normal range and update state
		if (ViewType < 0 || ViewType >= EAssetViewType::MAX)
		{
			ViewType = EAssetViewType::Tile;
		}

		SetCurrentViewType((EAssetViewType::Type)ViewType);
	}

	// Update the size value after loading the config of the CurrentViewType and the sizes
	// Since if the View was the same as before it won't get called during SetCurrentViewType
	UpdateThumbnailSizeValue();

	float Zoom = 0;
	if ( GConfig->GetFloat(*IniSection, *(SettingsString + TEXT(".ZoomScale")), Zoom, IniFilename) )
	{
		// Clamp value to normal range and update state
		ZoomScale = FMath::Clamp<float>(Zoom, 0.f, 1.f);
	}

	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		bool bColumnChangedManually = false;
		if (GConfig->GetBool(*IniSection, *(SettingsString + TEXT(".ColumnViewColumnsManuallyChangedOnce")), bColumnChangedManually, IniFilename) )
		{
			// Whether the column where changed by the user even once for this Config, if yes always use the LoadedColumns
			bColumnViewColumnsManuallyChangedOnce = bColumnChangedManually;
		}
	}

	TArray<FString> LoadedHiddenColumnNames;
	GConfig->GetArray(*IniSection, *(SettingsString + TEXT(".HiddenColumns")), LoadedHiddenColumnNames, IniFilename);
	if (LoadedHiddenColumnNames.Num() > 0 || bColumnViewColumnsManuallyChangedOnce)
	{
		HiddenColumnNames = LoadedHiddenColumnNames;

		// Also update the visibility of the columns we just loaded in (unless this is called before creation and ColumnView doesn't exist)
		if (ColumnView)
		{
			for (const SHeaderRow::FColumn& Column : ColumnView->GetHeaderRow()->GetColumns())
			{
				ColumnView->GetHeaderRow()->SetShowGeneratedColumn(Column.ColumnId, !HiddenColumnNames.Contains(Column.ColumnId.ToString()));
			}
		}
	}

	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		bool bColumnChangedManually = false;
		if ( GConfig->GetBool(*IniSection, *(SettingsString + TEXT(".ListViewColumnsManuallyChangedOnce")), bColumnChangedManually, IniFilename) )
		{
			// Whether the column where changed by the user even once for this Config, if yes always use the LoadedColumns
			bListViewColumnsManuallyChangedOnce = bColumnChangedManually;
		}

		TArray<FString> LoadedListHiddenColumnNames;
		GConfig->GetArray(*IniSection, *(SettingsString + TEXT(".ListHiddenColumns")), LoadedListHiddenColumnNames, IniFilename);
		if (LoadedListHiddenColumnNames.Num() > 0 || bListViewColumnsManuallyChangedOnce)
		{
			ListHiddenColumnNames = LoadedListHiddenColumnNames;

			// Also update the visibility of the columns we just loaded in (unless this is called before creation and ColumnView doesn't exist)
			if (ListView)
			{
				for (const SHeaderRow::FColumn& ListColumn : ListView->GetHeaderRow()->GetColumns())
				{
					ListView->GetHeaderRow()->SetShowGeneratedColumn(ListColumn.ColumnId, !ListHiddenColumnNames.Contains(ListColumn.ColumnId.ToString()));
				}
			}
		}
	}
}

// Adjusts the selected asset by the selection delta, which should be +1 or -1)
void SAssetView::AdjustActiveSelection(int32 SelectionDelta)
{
	// Find the index of the first selected item
	TArray<TSharedPtr<FAssetViewItem>> SelectionSet = GetSelectedViewItems();
	
	int32 SelectedSuggestion = INDEX_NONE;

	if (SelectionSet.Num() > 0)
	{
		if (!FilteredAssetItems.Find(SelectionSet[0], /*out*/ SelectedSuggestion))
		{
			// Should never happen
			ensureMsgf(false, TEXT("SAssetView has a selected item that wasn't in the filtered list"));
			return;
		}
	}
	else
	{
		SelectedSuggestion = 0;
		SelectionDelta = 0;
	}

	if (FilteredAssetItems.Num() > 0)
	{
		// Move up or down one, wrapping around
		SelectedSuggestion = (SelectedSuggestion + SelectionDelta + FilteredAssetItems.Num()) % FilteredAssetItems.Num();

		// Pick the new asset
		const TSharedPtr<FAssetViewItem>& NewSelection = FilteredAssetItems[SelectedSuggestion];

		RequestScrollIntoView(NewSelection);
		SetSelection(NewSelection);
	}
	else
	{
		ClearSelection();
	}
}

void SAssetView::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	using namespace UE::AssetView;

	if (Items->UpdateShowingContentVersePath())
	{
		// A path may have changed, if sorting on the path column we need to resort.
		if (bShowPathInColumnView)
		{
			for (int32 PriorityIdx = 0; PriorityIdx < EColumnSortPriority::Max; PriorityIdx++)
			{
				if (SortManager->GetSortColumnId(static_cast<EColumnSortPriority::Type>(PriorityIdx)) == FAssetViewSortManager::PathColumnId)
				{
					// Don't sync to selection if we are just going to do it below
					SortList(!PendingSyncItems.Num());
					break;
				}
			}
		}

		// A path may have changed, if filtering on a path we need to refilter.
		bQuickFrontendListRefreshRequested = true;
	}

	// Adjust min and max thumbnail scale based on dpi
	MinThumbnailScale = (0.2f * ThumbnailScaleRangeScalar)/AllottedGeometry.Scale;
	MaxThumbnailScale = (1.9f * ThumbnailScaleRangeScalar)/AllottedGeometry.Scale;

	CalculateFillScale( AllottedGeometry );

	CurrentTime = InCurrentTime;

	if (FSlateApplication::Get().GetActiveModalWindow().IsValid())
	{
		// If we're in a model window then we need to tick the thumbnail pool in order for thumbnails to render correctly.
		AssetThumbnailPool->Tick(InDeltaTime);
	}

	const bool bNewShowRedirectors = bShowRedirectors.Get(false);
	if (bNewShowRedirectors != bLastShowRedirectors)
	{
		bLastShowRedirectors = bNewShowRedirectors;
		OnFrontendFiltersChanged(); // refresh the same as if filters changed
	}

	CalculateThumbnailHintColorAndOpacity();

	// Wait for any pending refresh to complete, otherwise VisibleItems will be out of date.
	// If we sorted a long list that can cause us to update thousands of thumbnails.
	if (bPendingUpdateThumbnails && !IsPendingRefresh())
	{
		UpdateThumbnails();
		bPendingUpdateThumbnails = false;
	}

	if (bSlowFullListRefreshRequested)
	{
		RefreshSourceItems();
		bSlowFullListRefreshRequested = false;
		bQuickFrontendListRefreshRequested = true;
	}

	bool bForceViewUpdate = false;
	if (bQuickFrontendListRefreshRequested)
	{
		ResetQuickJump();

		RefreshFilteredItems();

		bQuickFrontendListRefreshRequested = false;
		bForceViewUpdate = true; // If HasItemsPendingFilter is empty we still need to update the view
	}

	if (HasItemsPendingFilter() || bForceViewUpdate)
	{
		if (bForceViewUpdate)
		{
			RefreshList();
		}

		bForceViewUpdate = false;

		const double TickStartTime = FPlatformTime::Seconds();
		const bool bWasWorking = InitialNumAmortizedTasks > 0;

		// Mark the first amortize time
		if (AmortizeStartTime == 0)
		{
			AmortizeStartTime = FPlatformTime::Seconds();
			InitialNumAmortizedTasks = Items->Num();
			
			CurrentFrontendFilterTelemetry = { ViewCorrelationGuid, FilterSessionCorrelationGuid };
			CurrentFrontendFilterTelemetry.FrontendFilters = FrontendFilters;
			CurrentFrontendFilterTelemetry.TotalItemsToFilter = Items->Num();
			CurrentFrontendFilterTelemetry.PriorityItemsToFilter = 0;
		}

		int32 PreviousFilteredAssetItems = FilteredAssetItems.Num();
		ProcessItemsPendingFilter(TickStartTime);
		if (PreviousFilteredAssetItems == 0 && FilteredAssetItems.Num() != 0)
		{
			CurrentFrontendFilterTelemetry.ResultLatency = FPlatformTime::Seconds() - AmortizeStartTime;
		}
		CurrentFrontendFilterTelemetry.TotalResults = FilteredAssetItems.Num(); // Provide number of results even if filtering is interrupted

		if (HasItemsPendingFilter())
		{
			if (bPendingSortFilteredItems && InCurrentTime > LastSortTime + SortDelaySeconds)
			{
				// Don't sync to selection if we are just going to do it below
				SortList(!PendingSyncItems.Num());
			}
			
			CurrentFrontendFilterTelemetry.WorkDuration += FPlatformTime::Seconds() - TickStartTime;

			// Need to finish processing queried items before rest of function is safe
			return;
		}
		else
		{
			// Update the columns in the column view now that we know the majority type
			if (CurrentViewType == EAssetViewType::Column)
			{
				int32 HighestCount = 0;
				FName HighestType;
				for (auto TypeIt = FilteredAssetItemTypeCounts.CreateConstIterator(); TypeIt; ++TypeIt)
				{
					if (TypeIt.Value() > HighestCount)
					{
						HighestType = TypeIt.Key();
						HighestCount = TypeIt.Value();
					}
				}

				SetMajorityAssetType(HighestType);
			}

			if (bPendingSortFilteredItems && (bWasWorking || (InCurrentTime > LastSortTime + SortDelaySeconds)))
			{
				// Don't sync to selection if we are just going to do it below
				SortList(!PendingSyncItems.Num());
			}

			CurrentFrontendFilterTelemetry.WorkDuration += FPlatformTime::Seconds() - TickStartTime;

			double AmortizeDuration = FPlatformTime::Seconds() - AmortizeStartTime;
			TotalAmortizeTime += AmortizeDuration;
			AmortizeStartTime = 0;
			InitialNumAmortizedTasks = 0;
			
			OnCompleteFiltering(AmortizeDuration);
		}
	}

	if ( PendingSyncItems.Num() > 0 )
	{
		if (bPendingSortFilteredItems)
		{
			// Don't sync to selection because we are just going to do it below
			SortList(/*bSyncToSelection=*/false);
		}
		
		bBulkSelecting = true;
		ClearSelection();
		bool bFoundScrollIntoViewTarget = false;

		for ( auto ItemIt = FilteredAssetItems.CreateConstIterator(); ItemIt; ++ItemIt )
		{
			const auto& Item = *ItemIt;
			if(Item.IsValid())
			{
				const FName ItemVirtualPath = Item->GetItem().GetVirtualPath();
				if (PendingSyncItems.SelectedVirtualPaths.Contains(ItemVirtualPath))
				{
					DeferredPendingSyncItems.SelectedVirtualPaths.Remove(ItemVirtualPath);

					SetItemSelection(Item, true, ESelectInfo::OnNavigation);
					
					// Scroll the first item in the list that can be shown into view
					if ( !bFoundScrollIntoViewTarget )
					{
						RequestScrollIntoView(Item);
						bFoundScrollIntoViewTarget = true;
					}
				}
			}
		}
	
		bBulkSelecting = false;

		if (DeferredSyncTimeoutFrames > 0)
		{
			DeferredSyncTimeoutFrames--;

			if (DeferredSyncTimeoutFrames == 0)
			{
				DeferredPendingSyncItems.Reset();
			}
		}

		if (DeferredPendingSyncItems.Num() == 0)
		{
			if (bShouldNotifyNextAssetSync && !bUserSearching)
			{
				AssetSelectionChanged(TSharedPtr<FAssetViewItem>(), ESelectInfo::Direct);
			}

			// Default to always notifying
			bShouldNotifyNextAssetSync = true;
			
			PendingSyncItems.Reset();

			if (bAllowFocusOnSync && bPendingFocusOnSync)
			{
				FocusList();
			}
		}
	}

	if ( IsHovered() )
	{
		// This prevents us from sorting the view immediately after the cursor leaves it
		LastSortTime = CurrentTime;
	}
	else if ( bPendingSortFilteredItems && InCurrentTime > LastSortTime + SortDelaySeconds )
	{
		SortList();
	}

	// create any pending items now
	BeginCreateDeferredItem();

	// Do quick-jump last as the Tick function might have canceled it
	if(QuickJumpData.bHasChangedSinceLastTick)
	{
		QuickJumpData.bHasChangedSinceLastTick = false;

		const bool bWasJumping = QuickJumpData.bIsJumping;
		QuickJumpData.bIsJumping = true;

		QuickJumpData.LastJumpTime = InCurrentTime;
		QuickJumpData.bHasValidMatch = PerformQuickJump(bWasJumping);
	}
	else if(QuickJumpData.bIsJumping && InCurrentTime > QuickJumpData.LastJumpTime + JumpDelaySeconds)
	{
		ResetQuickJump();
	}

	TSharedPtr<FAssetViewItem> AssetAwaitingRename = AwaitingRename.Pin();
	if (AssetAwaitingRename.IsValid())
	{
		TSharedPtr<SWindow> OwnerWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (!OwnerWindow.IsValid())
		{
			AwaitingRename = nullptr;
		}
		else if (OwnerWindow->HasAnyUserFocusOrFocusedDescendants())
		{
			CancelPendingRename();
			AssetAwaitingRename->OnRenameRequested().ExecuteIfBound();
			AwaitingRename = nullptr;
		}
	}
}

void SAssetView::CalculateFillScale( const FGeometry& AllottedGeometry )
{
	if ( bFillEmptySpaceInTileView && CurrentViewType == EAssetViewType::Tile )
	{
		float ItemWidth = GetTileViewItemBaseWidth();

		// Scrollbars are 16, but we add 1 to deal with half pixels.
		const float ScrollbarWidth = 16 + 1;
		float TotalWidth = AllottedGeometry.GetLocalSize().X -(ScrollbarWidth);
		float Coverage = TotalWidth / ItemWidth;
		int32 NumItems = (int)( TotalWidth / ItemWidth );

		// If there isn't enough room to support even a single item, don't apply a fill scale.
		if ( NumItems > 0 )
		{
			float GapSpace = ItemWidth * ( Coverage - (float)NumItems );
			float ExpandAmount = GapSpace / (float)NumItems;
			FillScale = ( ItemWidth + ExpandAmount ) / ItemWidth;
			FillScale = FMath::Max( 1.0f, FillScale );
		}
		else
		{
			FillScale = 1.0f;
		}
	}
	else
	{
		FillScale = 1.0f;
	}
}

void SAssetView::CalculateThumbnailHintColorAndOpacity()
{
	if ( HighlightedText.Get().IsEmpty() )
	{
		if ( ThumbnailHintFadeInSequence.IsPlaying() )
		{
			if ( ThumbnailHintFadeInSequence.IsForward() )
			{
				ThumbnailHintFadeInSequence.Reverse();
			}
		}
		else if ( ThumbnailHintFadeInSequence.IsAtEnd() ) 
		{
			ThumbnailHintFadeInSequence.PlayReverse(this->AsShared());
		}
	}
	else 
	{
		if ( ThumbnailHintFadeInSequence.IsPlaying() )
		{
			if ( ThumbnailHintFadeInSequence.IsInReverse() )
			{
				ThumbnailHintFadeInSequence.Reverse();
			}
		}
		else if ( ThumbnailHintFadeInSequence.IsAtStart() ) 
		{
			ThumbnailHintFadeInSequence.Play(this->AsShared());
		}
	}

	const float Opacity = ThumbnailHintFadeInSequence.GetLerp();
	ThumbnailHintColorAndOpacity = FLinearColor( 1.0, 1.0, 1.0, Opacity );
}

bool SAssetView::HasItemsPendingFilter() const
{
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	if (AssetSearchOverride.IsValid() && AssetSearchOverride->IsOverrideActive() && (AssetSearchOverride->HasPendingResults() || !AssetSearchOverride->GetInternalApi().PublishedResults.IsEmpty()))
	{
		return true;
	}
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	return Items->HasItemsPendingFilter();
}

bool SAssetView::HasThumbnailsPendingUpdate() const
{
	return bPendingUpdateThumbnails;
}

bool SAssetView::HasDeferredItemToCreate() const
{
	return DeferredItemToCreate.IsValid();
}

void SAssetView::ProcessItemsPendingFilter(const double TickStartTime)
{
	const double ProcessItemsPendingFilterStartTime = FPlatformTime::Seconds();

	FAssetViewFrontendFilterHelper FrontendFilterHelper(this);
	const bool bFlushAllPendingItems = TickStartTime < 0;
	int32 OldCount = FilteredAssetItems.Num();
	TArray<TSharedPtr<FAssetViewItem>> FilterResults;

	Items->UpdateItemFiltering(FrontendFilterHelper, bFlushAllPendingItems ? MAX_dbl : TickStartTime + MaxSecondsPerFrame, FilterResults);

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		if (AssetSearchOverride.IsValid() && AssetSearchOverride->IsOverrideActive())
		{
			AssetSearchOverride->OnItemsAvailable(MakeArrayView(FilterResults));
			FilterResults.Reset();

			FilterResults.Append(MoveTemp(AssetSearchOverride->GetInternalApi().PublishedResults));
			AssetSearchOverride->GetInternalApi().PublishedResults.Reset();

			if (!Items->HasItemsPendingFilter())
			{
				AssetSearchOverride->GetInternalApi().OnAllKnownItemsAvailable();
			}
		}
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS


	// exclude duplicates
	for (TSharedPtr<FAssetViewItem>& Result : FilterResults)
	{
		bool bAlreadyAdded = false;
		const FContentBrowserItem& Item = Result->GetItem();
		
		if (FSoftObjectPath ItemPath; Item.IsValid() && Item.GetItemAliasedPath(ItemPath))
		{
			AlreadyAddedOriginalAssets.Add(ItemPath, &bAlreadyAdded);
		}
		if (!bAlreadyAdded)
		{
			FilteredAssetItems.Add(MoveTemp(Result));
		}
	}

	if (CurrentViewType == EAssetViewType::Column)
	{
		for (int32 i = OldCount; i < FilteredAssetItems.Num(); ++i)
		{
			const TSharedPtr<FAssetViewItem>& Item = FilteredAssetItems[i];
			const FContentBrowserItemDataAttributeValue TypeNameValue = Item->GetItem().GetItemAttribute(ContentBrowserItemAttributes::ItemTypeName);
			if (TypeNameValue.IsValid())
			{
				FilteredAssetItemTypeCounts.FindOrAdd(TypeNameValue.GetValue<FName>())++;
			}
		}
	}

	if (FilteredAssetItems.Num() > OldCount)
	{
		bPendingSortFilteredItems = true;
		RefreshList();
	}

	UE_LOGF(LogContentBrowser, VeryVerbose, "AssetView - ProcessItemsPendingFilter completed in %0.4f seconds", FPlatformTime::Seconds() - ProcessItemsPendingFilterStartTime);
}

void SAssetView::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	TSharedPtr< FAssetDragDropOp > AssetDragDropOp = DragDropEvent.GetOperationAs< FAssetDragDropOp >();
	if( AssetDragDropOp.IsValid() )
	{
		AssetDragDropOp->ResetToDefaultToolTip();
	}

	TSharedPtr<FDragDropOperation> DragDropOp = DragDropEvent.GetOperation();
	if (DragDropOp.IsValid())
	{
		// Do we have a custom handler for this drag event?
		FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>("ContentBrowser");
		const TArray<FAssetViewDragAndDropExtender>& AssetViewDragAndDropExtenders = ContentBrowserModule.GetAssetViewDragAndDropExtenders();
		for (const auto& AssetViewDragAndDropExtender : AssetViewDragAndDropExtenders)
		{
			if (AssetViewDragAndDropExtender.OnDragLeaveDelegate.IsBound() && AssetViewDragAndDropExtender.OnDragLeaveDelegate.Execute(FAssetViewDragAndDropExtender::FPayload(DragDropOp, ContentSources.GetVirtualPaths(), ContentSources.GetCollections())))
			{
				return;
			}
		}
	}
}

FReply SAssetView::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDragDropOperation> DragDropOp = DragDropEvent.GetOperation();
	if (DragDropOp.IsValid())
	{
		// Do we have a custom handler for this drag event?
		FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>("ContentBrowser");
		const TArray<FAssetViewDragAndDropExtender>& AssetViewDragAndDropExtenders = ContentBrowserModule.GetAssetViewDragAndDropExtenders();
		for (const auto& AssetViewDragAndDropExtender : AssetViewDragAndDropExtenders)
		{
			if (AssetViewDragAndDropExtender.OnDragOverDelegate.IsBound() && AssetViewDragAndDropExtender.OnDragOverDelegate.Execute(FAssetViewDragAndDropExtender::FPayload(DragDropOp, ContentSources.GetVirtualPaths(), ContentSources.GetCollections())))
			{
				return FReply::Handled();
			}
		}
	}

	if (ContentSources.HasVirtualPaths())
	{
		UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

		const FContentBrowserItem DropFolderItem = ContentBrowserData->GetItemAtPath(ContentSources.GetVirtualPaths()[0], EContentBrowserItemTypeFilter::IncludeFolders);
		if (DropFolderItem.IsValid() && DragDropHandler::HandleDragOverItem(DropFolderItem, DragDropEvent))
		{
			return FReply::Handled();
		}
	}
	else if (HasSingleCollectionSource())
	{
		TArray<FSoftObjectPath> NewCollectionItems;

		if (TSharedPtr<FContentBrowserDataDragDropOp> ContentDragDropOp = DragDropEvent.GetOperationAs<FContentBrowserDataDragDropOp>())
		{
			for (const FContentBrowserItem& DraggedItem : ContentDragDropOp->GetDraggedFiles())
			{
				FSoftObjectPath CollectionItemId;
				if (DraggedItem.TryGetCollectionId(CollectionItemId))
				{
					NewCollectionItems.Add(CollectionItemId);
				}
			}
		}
		else
		{
			const TArray<FAssetData> AssetDatas = AssetUtil::ExtractAssetDataFromDrag(DragDropEvent);
			Algo::Transform(AssetDatas, NewCollectionItems, &FAssetData::GetSoftObjectPath);
		}

		if (NewCollectionItems.Num() > 0)
		{
			if (TSharedPtr<FAssetDragDropOp> AssetDragDropOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>())
			{
				TArray<FSoftObjectPath> ObjectPaths;
				const FCollectionRef& Collection = ContentSources.GetCollections()[0];
				Collection.Container->GetObjectsInCollection(Collection.Name, Collection.Type, ObjectPaths);

				bool IsValidDrop = false;
				for (const FSoftObjectPath& NewCollectionItem : NewCollectionItems)
				{
					if (!ObjectPaths.Contains(NewCollectionItem))
					{
						IsValidDrop = true;
						break;
					}
				}

				if (IsValidDrop)
				{
					AssetDragDropOp->SetToolTip(NSLOCTEXT("AssetView", "OnDragOverCollection", "Add to Collection"), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")));
				}
			}

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply SAssetView::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDragDropOperation> DragDropOp = DragDropEvent.GetOperation();
	if (DragDropOp.IsValid())
	{
		// Do we have a custom handler for this drag event?
		FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>("ContentBrowser");
		const TArray<FAssetViewDragAndDropExtender>& AssetViewDragAndDropExtenders = ContentBrowserModule.GetAssetViewDragAndDropExtenders();
		for (const auto& AssetViewDragAndDropExtender : AssetViewDragAndDropExtenders)
		{
			if (AssetViewDragAndDropExtender.OnDropDelegate.IsBound() && AssetViewDragAndDropExtender.OnDropDelegate.Execute(FAssetViewDragAndDropExtender::FPayload(DragDropOp, ContentSources.GetVirtualPaths(), ContentSources.GetCollections())))
			{
				return FReply::Handled();
			}
		}
	}

	if (ContentSources.HasVirtualPaths())
	{
		UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

		const FContentBrowserItem DropFolderItem = ContentBrowserData->GetItemAtPath(ContentSources.GetVirtualPaths()[0], EContentBrowserItemTypeFilter::IncludeFolders);
		if (DropFolderItem.IsValid() && DragDropHandler::HandleDragDropOnItem(DropFolderItem, DragDropEvent, AsShared()))
		{
			return FReply::Handled();
		}
	}
	else if (HasSingleCollectionSource())
	{
		TArray<FSoftObjectPath> NewCollectionItems;

		if (TSharedPtr<FContentBrowserDataDragDropOp> ContentDragDropOp = DragDropEvent.GetOperationAs<FContentBrowserDataDragDropOp>())
		{
			for (const FContentBrowserItem& DraggedItem : ContentDragDropOp->GetDraggedFiles())
			{
				FSoftObjectPath CollectionItemId;
				if (DraggedItem.TryGetCollectionId(CollectionItemId))
				{
					NewCollectionItems.Add(CollectionItemId);
				}
			}
		}
		else
		{
			const TArray<FAssetData> AssetDatas = AssetUtil::ExtractAssetDataFromDrag(DragDropEvent);
			Algo::Transform(AssetDatas, NewCollectionItems, &FAssetData::GetSoftObjectPath);
		}

		if (NewCollectionItems.Num() > 0)
		{
			const FCollectionRef& Collection = ContentSources.GetCollections()[0];
			Collection.Container->AddToCollection(Collection.Name, Collection.Type, NewCollectionItems);

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply SAssetView::OnKeyChar( const FGeometry& MyGeometry,const FCharacterEvent& InCharacterEvent )
{
	const bool bIsControlOrCommandDown = InCharacterEvent.IsControlDown() || InCharacterEvent.IsCommandDown();
	
	const bool bTestOnly = false;
	if(HandleQuickJumpKeyDown(InCharacterEvent.GetCharacter(), bIsControlOrCommandDown, InCharacterEvent.IsAltDown(), bTestOnly).IsEventHandled())
	{
		return FReply::Handled();
	}

	// If the user pressed a key we couldn't handle, reset the quick-jump search
	ResetQuickJump();

	return FReply::Unhandled();
}

FReply SAssetView::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	const bool bIsControlOrCommandDown = InKeyEvent.IsControlDown() || InKeyEvent.IsCommandDown();

	if (Commands->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	// Swallow the key-presses used by the quick-jump in OnKeyChar to avoid other things (such as the viewport commands) getting them instead
	// eg) Pressing "W" without this would set the viewport to "translate" mode
	else if(HandleQuickJumpKeyDown((TCHAR)InKeyEvent.GetCharacter(), bIsControlOrCommandDown, InKeyEvent.IsAltDown(), /*bTestOnly*/true).IsEventHandled())
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SAssetView::OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	// Make sure to not change the thumbnail scaling when we're in Columns view since thumbnail scaling isn't applicable there.
	if (MouseEvent.IsControlDown() && IsThumbnailScalingAllowed())
	{
		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			const int32 Delta = MouseEvent.GetWheelDelta() > 0 ? 1 : -1;
			bool IsLessThanMinSize = (int32)ThumbnailSizes[CurrentViewType] + Delta < 0;
			bool IsMoreThanMaxSize = (int32)ThumbnailSizes[CurrentViewType] + Delta == (int32)EThumbnailSize::MAX;
			bool bWasSizeSupposedToChange = false;
			// If shift is pressed or if we are in the List view the CTRL + Wheel should jump ThumbnailSize by design
			if (MouseEvent.IsShiftDown() || CurrentViewType != EAssetViewType::Tile)
			{
				bWasSizeSupposedToChange = true;
				EThumbnailSize DesiredThumbnailSize = (EThumbnailSize)FMath::Clamp<int32>((int32)ThumbnailSizes[CurrentViewType] + Delta, 0, (int32)EThumbnailSize::MAX - 1);

				if (DesiredThumbnailSize != ThumbnailSizes[CurrentViewType])
				{
					OnThumbnailSizeChanged(DesiredThumbnailSize);
				}
			}
			else
			{
				const float NewDelta = (float)Delta * 0.4f;
				if ((ZoomScale == 1.f && NewDelta > 0) || (ZoomScale == 0.f && NewDelta < 0))
				{
					bWasSizeSupposedToChange = true;
					const int32 Step = (int32)FMath::Sign(NewDelta);
					EThumbnailSize OldSize = ThumbnailSizes[CurrentViewType];
					ThumbnailSizes[CurrentViewType] = (EThumbnailSize)FMath::Clamp<int32>(((int32)ThumbnailSizes[CurrentViewType] + Step), 0, (int32)EThumbnailSize::MAX - 1);
					if (OldSize != ThumbnailSizes[CurrentViewType])
					{
						OnThumbnailSizeChanged(ThumbnailSizes[CurrentViewType]);
						ZoomScale = NewDelta > 0.f ? 0.f : 1.f;
					}
				}
				else
				{
					ZoomScale = FMath::Clamp(ZoomScale + NewDelta, 0.f, 1.f);
					// Always refresh the view when changing size otherwise some items may be missing sometimes
					RefreshList();
				}
			}

			// Switch the view automatically
			if (bWasSizeSupposedToChange && bEnableGridTileSwitch)
			{
				if (CurrentViewType == EAssetViewType::List && IsMoreThanMaxSize)
				{
					ZoomScale = 0.f;
					SetCurrentViewType(EAssetViewType::Tile);
					OnThumbnailSizeChanged(EThumbnailSize::Tiny);
				}
				else if (CurrentViewType == EAssetViewType::Tile && IsLessThanMinSize)
				{
					SetCurrentViewType(EAssetViewType::List);
                    OnThumbnailSizeChanged(EThumbnailSize::Huge);
				}
			}
		}
		else
		{
			// Step up/down a level depending on the scroll wheel direction.
			// Clamp value to enum min/max before updating.
			const int32 Delta = MouseEvent.GetWheelDelta() > 0 ? 1 : -1;
			EThumbnailSize DesiredThumbnailSize = (EThumbnailSize)FMath::Clamp<int32>((int32)ThumbnailSizes[CurrentViewType] + Delta, 0, (int32)EThumbnailSize::MAX - 1);

			// TODO: Remove this afterwards, current CB should hide new size
			if (DesiredThumbnailSize == EThumbnailSize::XLarge)
			{
				if (Delta > 0)
				{
					DesiredThumbnailSize = EThumbnailSize::Huge;
				}
				else
				{
					DesiredThumbnailSize = EThumbnailSize::Large;
				}
			}

			if ( DesiredThumbnailSize != ThumbnailSizes[CurrentViewType] )
			{
				OnThumbnailSizeChanged(DesiredThumbnailSize);
			}
		}
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SAssetView::OnFocusChanging( const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent)
{
	ResetQuickJump();
}

TSharedRef<SAssetTileView> SAssetView::CreateTileView()
{
	return SNew(SAssetTileView)
		.SelectionMode( SelectionMode )
		.ListItemsSource(&FilteredAssetItems)
		.ItemAlignment(EListItemAlignment::LeftAligned)
		.OnGenerateTile(this, &SAssetView::MakeTileViewWidget)
		.OnItemToString_Debug_Static(&FAssetViewItem::ItemToString_Debug)
		.OnItemScrolledIntoView(this, &SAssetView::ItemScrolledIntoView)
		.OnContextMenuOpening(this, &SAssetView::OnGetContextMenuContent)
		.OnMouseButtonDoubleClick(this, &SAssetView::OnListMouseButtonDoubleClick)
		.OnSelectionChanged(this, &SAssetView::AssetSelectionChanged)
		.ItemHeight(this, &SAssetView::GetTileViewItemHeight)
		.ItemWidth(this, &SAssetView::GetTileViewItemWidth)
		.ScrollbarVisibility(bForceHideScrollbar ? EVisibility::Collapsed : EVisibility::Visible);
}

TSharedRef<SAssetListView> SAssetView::CreateListView()
{
	TSharedRef<SLayeredImage> RevisionControlColumnIcon = SNew(SLayeredImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FRevisionControlStyleManager::Get().GetBrush("RevisionControl.Icon"));

	RevisionControlColumnIcon->AddLayer(TAttribute<const FSlateBrush*>::CreateSP(this, &SAssetView::GetRevisionControlColumnIconBadge));

	TSharedPtr<SAssetListView> NewListView = SNew(SAssetListView)
		.SelectionMode(SelectionMode)
		.ListItemsSource(&FilteredAssetItems)
		.OnGenerateRow(this, &SAssetView::MakeListViewWidget)
		.OnItemToString_Debug_Static(&FAssetViewItem::ItemToString_Debug)
		.OnItemScrolledIntoView(this, &SAssetView::ItemScrolledIntoView)
		.OnContextMenuOpening(this, &SAssetView::OnGetContextMenuContent)
		.OnMouseButtonDoubleClick(this, &SAssetView::OnListMouseButtonDoubleClick)
		.OnSelectionChanged(this, &SAssetView::AssetSelectionChanged)
		.ScrollbarVisibility(bForceHideScrollbar ? EVisibility::Collapsed : EVisibility::Visible)
		.HeaderRow
		(
			SNew(SHeaderRow)
			.ResizeMode(UE::Editor::ContentBrowser::IsNewStyleEnabled() ? ESplitterResizeMode::Fill : ESplitterResizeMode::FixedSize)
			.CanSelectGeneratedColumn(UE::Editor::ContentBrowser::IsNewStyleEnabled())
			.OnHiddenColumnsListChanged(this, &SAssetView::OnHiddenColumnsChanged)

			// Revision Control column, currently doesn't support sorting
			+ SHeaderRow::Column(SortManager->RevisionControlColumnId)
			.FixedWidth(30.f)
			.HAlignHeader(HAlign_Center)
			.VAlignHeader(VAlign_Center)
			.HAlignCell(HAlign_Center)
			.VAlignCell(VAlign_Center)
			.DefaultLabel( LOCTEXT("Column_RC", "Revision Control") )
			.DefaultTooltip(LOCTEXT("Column_RC_Tooltip", "Revision Control Status"))
			[
				RevisionControlColumnIcon
			]
			
			+ SHeaderRow::Column(SortManager->NameColumnId)
			.FillWidth(300)
			.SortMode(TAttribute<EColumnSortMode::Type>::Create(TAttribute<EColumnSortMode::Type>::FGetter::CreateSP(this, &SAssetView::GetColumnSortMode, SortManager->NameColumnId)))
			.SortPriority(TAttribute<EColumnSortPriority::Type>::Create(TAttribute<EColumnSortPriority::Type>::FGetter::CreateSP(this, &SAssetView::GetColumnSortPriority, SortManager->NameColumnId)))
			.OnSort( FOnSortModeChanged::CreateSP( this, &SAssetView::OnSortColumnHeader ) )
			.DefaultLabel( LOCTEXT("Column_Name", "Name") )
			.ShouldGenerateWidget(true) // Can't hide name column, so at least one column is visible
		);

	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		const TArray<FString>& HiddenColumnsToUse = CurrentViewType == EAssetViewType::List ? ListHiddenColumnNames : HiddenColumnNames;

		{
			const bool bIsColumnVisible = !HiddenColumnsToUse.Contains(SortManager->RevisionControlColumnId.ToString());
			NewListView->GetHeaderRow()->SetShowGeneratedColumn(SortManager->RevisionControlColumnId, bIsColumnVisible);
		}

		NewListView->GetHeaderRow()->SetOnGetMaxRowSizeForColumn(FOnGetMaxRowSizeForColumn::CreateRaw(NewListView.Get(), &SAssetColumnView::GetMaxRowSizeForColumn));

		if (bShowTypeInColumnView || CurrentViewType == EAssetViewType::List)
		{
			NewListView->GetHeaderRow()->AddColumn(
					SHeaderRow::Column(SortManager->ClassColumnId)
					.FillWidth(160)
					.SortMode(TAttribute<EColumnSortMode::Type>::Create(TAttribute< EColumnSortMode::Type >::FGetter::CreateSP(this, &SAssetView::GetColumnSortMode, SortManager->ClassColumnId)))
					.SortPriority(TAttribute<EColumnSortPriority::Type>::Create(TAttribute<EColumnSortPriority::Type>::FGetter::CreateSP(this, &SAssetView::GetColumnSortPriority, SortManager->ClassColumnId)))
					.OnSort(FOnSortModeChanged::CreateSP(this, &SAssetView::OnSortColumnHeader))
					.DefaultLabel(LOCTEXT("Column_Class", "Type"))
				);

			const bool bIsColumnVisible = !HiddenColumnsToUse.Contains(SortManager->ClassColumnId.ToString());
		
			NewListView->GetHeaderRow()->SetShowGeneratedColumn(SortManager->ClassColumnId, bIsColumnVisible);
		}

		if (bShowPathInColumnView && CurrentViewType == EAssetViewType::Column)
		{
			NewListView->GetHeaderRow()->AddColumn(
					SHeaderRow::Column(SortManager->PathColumnId)
					.FillWidth(160)
					.SortMode(TAttribute<EColumnSortMode::Type>::Create(TAttribute< EColumnSortMode::Type >::FGetter::CreateSP(this, &SAssetView::GetColumnSortMode, SortManager->PathColumnId)))
					.SortPriority(TAttribute<EColumnSortPriority::Type>::Create(TAttribute<EColumnSortPriority::Type>::FGetter::CreateSP(this, &SAssetView::GetColumnSortPriority, SortManager->PathColumnId)))
					.OnSort(FOnSortModeChanged::CreateSP(this, &SAssetView::OnSortColumnHeader))
					.DefaultLabel(LOCTEXT("Column_Path", "Path"))
				);

			const bool bIsColumnVisible = !HiddenColumnsToUse.Contains(SortManager->PathColumnId.ToString());
		
			NewListView->GetHeaderRow()->SetShowGeneratedColumn(SortManager->PathColumnId, bIsColumnVisible);
		}

		if (bShowAssetAccessSpecifier)
		{
			NewListView->GetHeaderRow()->AddColumn(
				SHeaderRow::Column(SortManager->AssetAccessSpecifierColumnId)
				.FillWidth(160)
				.SortMode(TAttribute<EColumnSortMode::Type>::Create(TAttribute< EColumnSortMode::Type >::FGetter::CreateSP(this, &SAssetView::GetColumnSortMode, SortManager->AssetAccessSpecifierColumnId)))
				.SortPriority(TAttribute<EColumnSortPriority::Type>::Create(TAttribute<EColumnSortPriority::Type>::FGetter::CreateSP(this, &SAssetView::GetColumnSortPriority, SortManager->AssetAccessSpecifierColumnId)))
				.OnSort(FOnSortModeChanged::CreateSP(this, &SAssetView::OnSortColumnHeader))
				.DefaultLabel(LOCTEXT("Column_AssetAccessSpecifier", "Asset Access Specifier"))
			);

			const bool bIsColumnVisible = !HiddenColumnNames.Contains(SortManager->AssetAccessSpecifierColumnId.ToString());

			NewListView->GetHeaderRow()->SetShowGeneratedColumn(SortManager->AssetAccessSpecifierColumnId, bIsColumnVisible);
		}
	}
	return NewListView.ToSharedRef();
}

TSharedRef<SAssetColumnView> SAssetView::CreateColumnView()
{
	TSharedRef<SLayeredImage> RevisionControlColumnIcon = SNew(SLayeredImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FRevisionControlStyleManager::Get().GetBrush("RevisionControl.Icon"));

	RevisionControlColumnIcon->AddLayer(TAttribute<const FSlateBrush*>::CreateSP(this, &SAssetView::GetRevisionControlColumnIconBadge));
	
	TSharedPtr<SAssetColumnView> NewColumnView = SNew(SAssetColumnView)
		.SelectionMode( SelectionMode )
		.ListItemsSource(&FilteredAssetItems)
		.OnGenerateRow(this, &SAssetView::MakeColumnViewWidget)
		.OnItemToString_Debug_Static(&FAssetViewItem::ItemToString_Debug)
		.OnItemScrolledIntoView(this, &SAssetView::ItemScrolledIntoView)
		.OnContextMenuOpening(this, &SAssetView::OnGetContextMenuContent)
		.OnMouseButtonDoubleClick(this, &SAssetView::OnListMouseButtonDoubleClick)
		.OnSelectionChanged(this, &SAssetView::AssetSelectionChanged)
		.Visibility(this, &SAssetView::GetColumnViewVisibility)
		.ScrollbarVisibility(bForceHideScrollbar ? EVisibility::Collapsed : EVisibility::Visible)
		.HeaderRow
		(
			SNew(SHeaderRow)
			.ResizeMode(ESplitterResizeMode::Fill)
			.CanSelectGeneratedColumn(true)
			.OnHiddenColumnsListChanged(this, &SAssetView::OnHiddenColumnsChanged)

			// Revision Control column, currently doesn't support sorting
			+ SHeaderRow::Column(SortManager->RevisionControlColumnId)
			.FixedWidth(30.f)
			.HAlignHeader(HAlign_Center)
			.VAlignHeader(VAlign_Center)
			.HAlignCell(HAlign_Center)
			.VAlignCell(VAlign_Center)
			.DefaultLabel( LOCTEXT("Column_RC", "Revision Control") )
			[
				RevisionControlColumnIcon
			]
			
			+ SHeaderRow::Column(SortManager->NameColumnId)
			.FillWidth(300)
			.SortMode(TAttribute<EColumnSortMode::Type>::Create(TAttribute<EColumnSortMode::Type>::FGetter::CreateSP(this, &SAssetView::GetColumnSortMode, SortManager->NameColumnId)))
			.SortPriority(TAttribute<EColumnSortPriority::Type>::Create(TAttribute<EColumnSortPriority::Type>::FGetter::CreateSP(this, &SAssetView::GetColumnSortPriority, SortManager->NameColumnId)))
			.OnSort( FOnSortModeChanged::CreateSP( this, &SAssetView::OnSortColumnHeader ) )
			.DefaultLabel( LOCTEXT("Column_Name", "Name") )
			.ShouldGenerateWidget(true) // Can't hide name column, so at least one column is visible
		);

	{
		const bool bIsColumnVisible = !HiddenColumnNames.Contains(SortManager->RevisionControlColumnId.ToString());
        NewColumnView->GetHeaderRow()->SetShowGeneratedColumn(SortManager->RevisionControlColumnId, bIsColumnVisible);
	}

	NewColumnView->GetHeaderRow()->SetOnGetMaxRowSizeForColumn(FOnGetMaxRowSizeForColumn::CreateRaw(NewColumnView.Get(), &SAssetColumnView::GetMaxRowSizeForColumn));

	if(bShowTypeInColumnView)
	{
		NewColumnView->GetHeaderRow()->AddColumn(
				SHeaderRow::Column(SortManager->ClassColumnId)
				.FillWidth(160)
				.SortMode(TAttribute< EColumnSortMode::Type >::Create(TAttribute< EColumnSortMode::Type >::FGetter::CreateSP(this, &SAssetView::GetColumnSortMode, SortManager->ClassColumnId)))
				.SortPriority(TAttribute< EColumnSortPriority::Type >::Create(TAttribute< EColumnSortPriority::Type >::FGetter::CreateSP(this, &SAssetView::GetColumnSortPriority, SortManager->ClassColumnId)))
				.OnSort(FOnSortModeChanged::CreateSP(this, &SAssetView::OnSortColumnHeader))
				.DefaultLabel(LOCTEXT("Column_Class", "Type"))
			);

		const bool bIsColumnVisible = !HiddenColumnNames.Contains(SortManager->ClassColumnId.ToString());
		
		NewColumnView->GetHeaderRow()->SetShowGeneratedColumn(SortManager->ClassColumnId, bIsColumnVisible);
	}

	if (bShowPathInColumnView)
	{
		NewColumnView->GetHeaderRow()->AddColumn(
				SHeaderRow::Column(SortManager->PathColumnId)
				.FillWidth(160)
				.SortMode(TAttribute< EColumnSortMode::Type >::Create(TAttribute< EColumnSortMode::Type >::FGetter::CreateSP(this, &SAssetView::GetColumnSortMode, SortManager->PathColumnId)))
				.SortPriority(TAttribute< EColumnSortPriority::Type >::Create(TAttribute< EColumnSortPriority::Type >::FGetter::CreateSP(this, &SAssetView::GetColumnSortPriority, SortManager->PathColumnId)))
				.OnSort(FOnSortModeChanged::CreateSP(this, &SAssetView::OnSortColumnHeader))
				.DefaultLabel(LOCTEXT("Column_Path", "Path"))
			);

		const bool bIsColumnVisible = !HiddenColumnNames.Contains(SortManager->PathColumnId.ToString());
		
		NewColumnView->GetHeaderRow()->SetShowGeneratedColumn(SortManager->PathColumnId, bIsColumnVisible);
	}

	if (bShowAssetAccessSpecifier)
	{
		NewColumnView->GetHeaderRow()->AddColumn(
			SHeaderRow::Column(SortManager->AssetAccessSpecifierColumnId)
			.FillWidth(160)
			.SortMode(TAttribute<EColumnSortMode::Type>::Create(TAttribute< EColumnSortMode::Type >::FGetter::CreateSP(this, &SAssetView::GetColumnSortMode, SortManager->AssetAccessSpecifierColumnId)))
			.SortPriority(TAttribute<EColumnSortPriority::Type>::Create(TAttribute<EColumnSortPriority::Type>::FGetter::CreateSP(this, &SAssetView::GetColumnSortPriority, SortManager->AssetAccessSpecifierColumnId)))
			.OnSort(FOnSortModeChanged::CreateSP(this, &SAssetView::OnSortColumnHeader))
			.DefaultLabel(LOCTEXT("Column_AssetAccessSpecifier", "Asset Access Specifier"))
		);

		const bool bIsColumnVisible = !HiddenColumnNames.Contains(SortManager->AssetAccessSpecifierColumnId.ToString());

		NewColumnView->GetHeaderRow()->SetShowGeneratedColumn(SortManager->AssetAccessSpecifierColumnId, bIsColumnVisible);
	}

	return NewColumnView.ToSharedRef();
}

const FSlateBrush* SAssetView::GetRevisionControlColumnIconBadge() const
{
	if (ISourceControlModule::Get().IsEnabled())
	{
		return FRevisionControlStyleManager::Get().GetBrush("RevisionControl.Icon.ConnectedBadge");
	}
	else
	{
		return nullptr;
	}
}


bool SAssetView::IsValidSearchToken(const FString& Token) const
{
	if ( Token.Len() == 0 )
	{
		return false;
	}

	// A token may not be only apostrophe only, or it will match every asset because the text filter compares against the pattern Class'ObjectPath'
	if ( Token.Len() == 1 && Token[0] == '\'' )
	{
		return false;
	}

	return true;
}

EContentBrowserItemCategoryFilter SAssetView::DetermineItemCategoryFilter() const
{
	// Check whether any legacy delegates are bound (the Content Browser doesn't use these, only pickers do)
	// These limit the view to things that might use FAssetData
	const bool bHasLegacyDelegateBindings = OnIsAssetValidForCustomToolTip.IsBound()
										 || OnGetCustomAssetToolTip.IsBound()
										 || OnVisualizeAssetToolTip.IsBound()
										 || OnAssetToolTipClosing.IsBound()
										 || OnShouldFilterAsset.IsBound();

	EContentBrowserItemCategoryFilter ItemCategoryFilter = bHasLegacyDelegateBindings ? EContentBrowserItemCategoryFilter::IncludeAssets : InitialCategoryFilter;
	if (IsShowingCppContent())
	{
		ItemCategoryFilter |= EContentBrowserItemCategoryFilter::IncludeClasses;
	}
	else
	{
		ItemCategoryFilter &= ~EContentBrowserItemCategoryFilter::IncludeClasses;
	}
	ItemCategoryFilter |= EContentBrowserItemCategoryFilter::IncludeCollections;
	if (IsShowingRedirectors())
	{
		ItemCategoryFilter |= EContentBrowserItemCategoryFilter::IncludeRedirectors;
	}
	else
	{
		ItemCategoryFilter &= ~EContentBrowserItemCategoryFilter::IncludeRedirectors;
	}
	return ItemCategoryFilter;
}

FContentBrowserDataFilter SAssetView::CreateBackendDataFilter(bool bInvalidateCache) const
{
	// Assemble the filter using the current sources
	// Force recursion when the user is searching
	const bool bHasCollections = ContentSources.HasCollections();
	const bool bRecurse = ShouldFilterRecursively();
	const bool bUsingFolders = IsShowingFolders() && !bRecurse;

	FContentBrowserDataFilter DataFilter;
	DataFilter.bRecursivePaths = bRecurse || !bUsingFolders || bHasCollections;

	DataFilter.ItemTypeFilter = EContentBrowserItemTypeFilter::IncludeFiles
		| ((bUsingFolders && !bHasCollections) ? EContentBrowserItemTypeFilter::IncludeFolders : EContentBrowserItemTypeFilter::IncludeNone);

	DataFilter.ItemCategoryFilter = DetermineItemCategoryFilter();

	DataFilter.ItemAttributeFilter = EContentBrowserItemAttributeFilter::IncludeProject
		| (IsShowingEngineContent() ? EContentBrowserItemAttributeFilter::IncludeEngine : EContentBrowserItemAttributeFilter::IncludeNone)
		| (IsShowingPluginContent() ? EContentBrowserItemAttributeFilter::IncludePlugins : EContentBrowserItemAttributeFilter::IncludeNone)
		| (IsShowingDevelopersContent() ? EContentBrowserItemAttributeFilter::IncludeDeveloper : EContentBrowserItemAttributeFilter::IncludeNone)
		| (IsShowingLocalizedContent() ? EContentBrowserItemAttributeFilter::IncludeLocalized : EContentBrowserItemAttributeFilter::IncludeNone);

	TSharedPtr<FPathPermissionList> CombinedFolderPermissionList = ContentBrowserUtils::GetCombinedFolderPermissionList(FolderPermissionList, IsShowingReadOnlyFolders() ? nullptr : WritableFolderPermissionList);
	
	UContentBrowserDataSubsystem* CBData = IContentBrowserDataModule::Get().GetSubsystem();
	if (BackendCustomPathFilters.Num())
	{
		if (!CombinedFolderPermissionList.IsValid())
		{
			CombinedFolderPermissionList = MakeShared<FPathPermissionList>();
		}

		if (!CombinedFolderPermissionList->HasAllowListEntries() 
		&& Algo::AnyOf(BackendCustomPathFilters, UE_PROJECTION_MEMBER(FPathPermissionList, HasAllowListEntries)))
		{
			// Need to add an explicit allow-root to the combined list before combining so that the allow list entries don't take everything away
			CombinedFolderPermissionList->AddAllowListItem("AssetView", TEXTVIEW("/"));
		}

		TArray<FName> SelectedPaths;
		SelectedPaths.Reserve(ContentSources.GetVirtualPaths().Num());
		// Convert paths to internal if possible
		for (FName VirtualPath : ContentSources.GetVirtualPaths())
		{
			FName ConvertedPath;
			CBData->TryConvertVirtualPath(VirtualPath, ConvertedPath);
			SelectedPaths.Add(ConvertedPath);
		}
		// If a filter list explicitly denies a folder we have selected, ignore that filter.
		for (const TSharedRef<const FPathPermissionList>& CustomList : BackendCustomPathFilters)
		{
			const bool bFiltersExplicitSelection = !bRecurse && Algo::AnyOf(SelectedPaths, [&CustomList](FName SelectedPath) {
				return !CustomList->PassesStartsWithFilter(WriteToString<256>(SelectedPath));
			});
			if (!bFiltersExplicitSelection)
			{
				CombinedFolderPermissionList = MakeShared<FPathPermissionList>(CombinedFolderPermissionList->CombinePathFilters(*CustomList));
			}
		}
	}
	
	if (bShowDisallowedAssetClassAsUnsupportedItems && AssetClassPermissionList && AssetClassPermissionList->HasFiltering())
	{
		// The unsupported item will created as an unsupported asset item instead of normal asset item for the writable folders
		FContentBrowserDataUnsupportedClassFilter& UnsupportedClassFilter = DataFilter.ExtraFilters.FindOrAddFilter<FContentBrowserDataUnsupportedClassFilter>();
		UnsupportedClassFilter.ClassPermissionList = AssetClassPermissionList;
		UnsupportedClassFilter.FolderPermissionList = WritableFolderPermissionList;
	}

	ContentBrowserUtils::AppendAssetFilterToContentBrowserFilter(BackendFilter, AssetClassPermissionList, CombinedFolderPermissionList, DataFilter);

	if (bHasCollections && !ContentSources.IsDynamicCollection())
	{
		FContentBrowserDataCollectionFilter& CollectionFilter = DataFilter.ExtraFilters.FindOrAddFilter<FContentBrowserDataCollectionFilter>();
		CollectionFilter.Collections = ContentSources.GetCollections();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Fill out deprecated SelectedCollections with game project collections for backwards compatibility.
		Algo::TransformIf(
			CollectionFilter.Collections,
			CollectionFilter.SelectedCollections,
			[](const FCollectionRef& Collection) { return Collection.Container == FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(); },
			[](const FCollectionRef& Collection) { return FCollectionNameType(Collection.Name, Collection.Type); });
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		CollectionFilter.bIncludeChildCollections = !bUsingFolders;
	}

	if (OnGetCustomSourceAssets.IsBound())
	{
		FContentBrowserDataLegacyFilter& LegacyFilter = DataFilter.ExtraFilters.FindOrAddFilter<FContentBrowserDataLegacyFilter>();
		LegacyFilter.OnGetCustomSourceAssets = OnGetCustomSourceAssets;
	}

	DataFilter.CacheID = FilterCacheID;

	if (bInvalidateCache)
	{
		if (ContentSources.IsIncludingVirtualPaths())
		{
			static const FName RootPath = "/";
			const TArrayView<const FName> DataSourcePaths = ContentSources.HasVirtualPaths() ? MakeArrayView(ContentSources.GetVirtualPaths()) : MakeArrayView(&RootPath, 1);
			FilterCacheID.RemoveUnusedCachedData(DataSourcePaths, DataFilter);
		}
		else
		{
			// Not sure what is the right thing to do here so clear the cache
			FilterCacheID.ClearCachedData();
		}
	}

	return DataFilter;
}

void SAssetView::RefreshSourceItems()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SAssetView::RefreshSourceItems);
	const double RefreshSourceItemsStartTime = FPlatformTime::Seconds();
	
	OnInterruptFiltering();

	FilterSessionCorrelationGuid = FGuid::NewGuid();
	UE::Telemetry::ContentBrowser::FBackendFilterTelemetry Telemetry(ViewCorrelationGuid, FilterSessionCorrelationGuid);
	VisibleItems.Reset();
	RelevantThumbnails.Reset();

	if (ContentSources.OnEnumerateCustomSourceItemDatas.IsBound())
	{
		Telemetry.bHasCustomItemSources = true;
	}

	const bool bInvalidateFilterCache = true;
	FContentBrowserDataFilter DataFilter = CreateBackendDataFilter(bInvalidateFilterCache);
	Telemetry.DataFilter = &DataFilter;
	bool bChangedRecursiveness = bWereItemsRecursivelyFiltered != DataFilter.bRecursivePaths;
	bWereItemsRecursivelyFiltered = DataFilter.bRecursivePaths;

	Items->RefreshItemsFromBackend(ContentSources, DataFilter, !bChangedRecursiveness);

	Telemetry.NumBackendItems = Items->Num();
	Telemetry.RefreshSourceItemsDurationSeconds = FPlatformTime::Seconds() - RefreshSourceItemsStartTime;
	FTelemetryRouter::Get().ProvideTelemetry(Telemetry);
	UE_LOGF(LogContentBrowser, VeryVerbose, "AssetView - RefreshSourceItems completed in %0.4f seconds", 
		FPlatformTime::Seconds() - RefreshSourceItemsStartTime);
}

void FAssetViewItemCollection::RefreshItemsFromBackend(const FAssetViewContentSources& ContentSources, const FContentBrowserDataFilter& DataFilter, bool bAllowItemRecycling)
{
	AbortTextFiltering();

	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	TArray<FContentBrowserItemData> NewItemDatas;
	if (DataFilter.bRecursivePaths)
	{
		NewItemDatas.Reserve(1024 * 1024); // Assume many recursive searches will return a lot of items and start with a lot of space 
	}

	if (ContentSources.OnEnumerateCustomSourceItemDatas.IsBound())
	{
		ContentSources.OnEnumerateCustomSourceItemDatas.Execute([&NewItemDatas](FContentBrowserItemData&& InItemData) { NewItemDatas.Add(MoveTemp(InItemData)); return true; });
	}

	if (ContentSources.IsIncludingVirtualPaths() || ContentSources.HasCollections())
	{
		if (ContentSources.HasCollections() && EnumHasAnyFlags(DataFilter.ItemCategoryFilter, EContentBrowserItemCategoryFilter::IncludeCollections))
		{
			// If we are showing collections then we may need to add dummy folder items for the child collections
			// Note: We don't check the IncludeFolders flag here, as that is forced to false when collections are selected,
			// instead we check the state of bIncludeChildCollections which will be false when we want to show collection folders
			const FContentBrowserDataCollectionFilter* CollectionFilter = DataFilter.ExtraFilters.FindFilter<FContentBrowserDataCollectionFilter>();
			if (CollectionFilter && !CollectionFilter->bIncludeChildCollections)
			{
				TArray<FCollectionNameType> ChildCollections;
				for(const FCollectionRef& Collection : ContentSources.GetCollections())
				{
					ChildCollections.Reset();
					Collection.Container->GetChildCollections(Collection.Name, Collection.Type, ChildCollections);

					for (const FCollectionNameType& ChildCollection : ChildCollections)
					{
						// Use "Collections" as the root of the path to avoid this being confused with other view folders - see ContentBrowserUtils::IsCollectionPath
						FContentBrowserItemData FolderItemData(
							nullptr, 
							EContentBrowserItemFlags::Type_Folder | EContentBrowserItemFlags::Category_Collection, 
							*(TEXT("/Collections/") / Collection.Container->MakeCollectionPath(ChildCollection.Name, ChildCollection.Type)),
							ChildCollection.Name, 
							FText::FromName(ChildCollection.Name), 
							nullptr,
							FName()
							);

						NewItemDatas.Add(MoveTemp(FolderItemData));
					}
				}
			}
		}

		if (ContentSources.IsIncludingVirtualPaths())
		{
			SCOPED_NAMED_EVENT(FetchCBItems, FColor::White);
			static const FName RootPath = "/";
			const TArrayView<const FName> DataSourcePaths = ContentSources.HasVirtualPaths() ? MakeArrayView(ContentSources.GetVirtualPaths()) : MakeArrayView(&RootPath, 1);
			for (const FName& DataSourcePath : DataSourcePaths)
			{
				// Ensure paths do not contain trailing slash
				ensure(DataSourcePath == RootPath || !FStringView(FNameBuilder(DataSourcePath)).EndsWith(TEXT('/')));
				ContentBrowserData->EnumerateItemsUnderPath(DataSourcePath, DataFilter, [&NewItemDatas](FContentBrowserItemData&& Item) { NewItemDatas.Add(MoveTemp(Item)); return true; });
			}
		}
	}

	TArray<TSharedPtr<FAssetViewItem>> OldItems = MoveTemp(Items);
	FHashTable OldLookup = MoveTemp(Lookup); 
	TArray<FContentBrowserItemKey> OldItemKeys;
	OldItemKeys.AddZeroed(OldItems.Num());
	ParallelFor(TEXT("ExtractOldItemKeys"), OldItems.Num(), 16 * 1024, 
		[&OldItems, &OldItemKeys](int32 Index) {
			if (OldItems[Index].IsValid())
			{
				OldItemKeys[Index] = FContentBrowserItemKey(OldItems[Index]->GetItem());
			}
		});

	Items.Reset(NewItemDatas.Num());
	Items.AddZeroed(NewItemDatas.Num());
	bItemsPendingRemove = false;

	// Create or recyle FAssetViewItem for each FContentBrowserItemData. Build new hashtable concurrently at the same time
	uint32 HashSize = FDefaultSparseSetAllocator::GetNumberOfHashBuckets(Items.Num());
	std::atomic<bool> bAnyFolders(false);
	std::atomic<bool> bAnyRecycled(false);
	Lookup.Clear(HashSize, Items.Num());
	{
		// Used to handle multiple item data (folder) mapping to the same old item.
		UE::FMutex OldItemMutex;
		SCOPED_NAMED_EVENT(CreateItems, FColor::White);
		ParallelFor(TEXT("CreateFAssetViewItem"), NewItemDatas.Num(), 16 * 1024,
			[&NewItemDatas, &OldItems, &OldLookup, &OldItemKeys, bAllowItemRecycling, &bAnyRecycled, &bAnyFolders, &OldItemMutex, this](int32 Index) {
				FContentBrowserItemData& ItemData = NewItemDatas[Index];
				FName VirtualPath = ItemData.GetVirtualPath();
				int32 OldItemIndex = INDEX_NONE;
				
				FContentBrowserItemKey ItemKey(ItemData);
				uint32 Hash = HashItem(ItemData);
				if (bAllowItemRecycling)
				{
					for (OldItemIndex = OldLookup.First(Hash); OldLookup.IsValid(OldItemIndex); OldItemIndex = OldLookup.Next(OldItemIndex))
					{
						if (ItemKey == OldItemKeys[OldItemIndex])
						{
							bAnyRecycled.store(true, std::memory_order_relaxed);
							break;
						}
					}
				}

				if (ItemData.IsFolder())
				{
					bAnyFolders.store(true, std::memory_order_relaxed);
				}

				if (OldLookup.IsValid(OldItemIndex))
				{
					TSharedPtr<FAssetViewItem> OldItem;
					// Try and acquire old item if another thread doesn't get there first (folder items share keys)
					{
						UE::TUniqueLock OldItemLock(OldItemMutex);
						OldItem = MoveTemp(OldItems[OldItemIndex]);
						OldItems[OldItemIndex].Reset();
					}

					if (OldItem.IsValid())
					{
						OldItem->ResetItemData(OldItemIndex, Index, MoveTemp(ItemData));
						Items[Index] = MoveTemp(OldItem);
						Lookup.Add_Concurrent(Hash, Index);
						return;
					}
				}

				// Was not able to recycle an old item
				TSharedPtr<FAssetViewItem> NewItem = MakeShared<FAssetViewItem>(Index, MoveTemp(ItemData));
				NewItem->ConfigureAssetPathText(bShowingContentVersePath);
				Items[Index] = MoveTemp(NewItem);
				Lookup.Add_Concurrent(Hash, Index);
			}, UE::AssetView::AllowParallelism ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
	}

	// Reset this before merging to avoid duplicate work around nulled entries
	ResetFilterState();	

	NumValidItems = Items.Num();
	if (bAnyFolders.load())
	{
		SCOPED_NAMED_EVENT(MergeDuplicates, FColor::White);
		// Merge items with the same path
		// Loop over each bucket lookup for duplicate names in that bucket and merging the items
		// This is done in parallel because each worker will only touch items in its bucket
		ParallelFor(TEXT("MergeDuplicates"), HashSize, 8, [this](int32 JobIndex) {
			uint32 Bucket = *reinterpret_cast<uint32*>(&JobIndex);
			for (uint32 StartIndex = Lookup.First(Bucket); Lookup.IsValid(StartIndex); StartIndex = Lookup.Next(StartIndex))
			{
				if (!Items[StartIndex]->IsFolder())
				{
					continue;
				}

				TArray<uint32, TInlineAllocator<4>> ToMerge;

				FContentBrowserItemKey MergeWithKey(Items[StartIndex]->GetItem());
				uint32 Other = Lookup.Next(StartIndex);
				while (Lookup.IsValid(Other))
				{
					check(Items[Other].IsValid());
					FContentBrowserItemKey OtherKey(Items[Other]->GetItem());
					if (MergeWithKey == OtherKey)
					{
						uint32 ToRemove = Other;
						Other = Lookup.Next(Other);
						Lookup.Remove(Bucket, ToRemove);
						ToMerge.Add(ToRemove);
					}
					else
					{
						Other = Lookup.Next(Other);
					}
				}

				// For determinism, make sure we merge items preserving their original order.
				// Different items may have different display data, and Lookup.Add_Concurrent
				// could cause us to select different items to display each time we refresh.
				if (!ToMerge.IsEmpty())
				{
					ToMerge.Add(StartIndex);
					ToMerge.Sort();

					const uint32 MergeToIndex = ToMerge[0];

					if (StartIndex != MergeToIndex)
					{
						checkf(Items[MergeToIndex]->GetItem().GetInternalItems().Num() == 1, TEXT("New items should only have a single internal item before merging."));

						// Update the Item's index to StartIndex.
						Items[MergeToIndex]->ResetItemData(MergeToIndex, StartIndex, *Items[MergeToIndex]->GetItem().GetPrimaryInternalItem());
					}

					for (uint32 ToRemove : MakeArrayView(ToMerge).RightChop(1))
					{
						TSharedPtr<FAssetViewItem> RemovedItem = MarkItemRemoved(ToRemove);
						Items[MergeToIndex]->AppendItemData(RemovedItem->GetItem());
						Items[ToRemove].Reset();
					}

					// Make sure StartIndex still points to the merged item.
					if (StartIndex != MergeToIndex)
					{
						Swap(Items[StartIndex], Items[MergeToIndex]);
						Swap(FilterState[StartIndex], FilterState[MergeToIndex]);
					}
				}
			}

		}, EParallelForFlags::Unbalanced | (UE::AssetView::AllowParallelism ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread));
	}

	// We already nulled out the items we removed above 
	bItemsPendingRemove = false;

	// If we recycled any items (e.g. we changed item visibility settings but not path) notify their widgets that we changed the item data
	if (bAnyRecycled.load())
	{
		for (const TSharedPtr<FAssetViewItem>& Item : Items)
		{
			if (Item.IsValid())
			{
				Item->BroadcastItemDataChanged();
			}
		}
	}

	// Until we start filtering and get a compiled filter, initialize to no filtering
	bAllItemsPassedTextFilter = true;
	CompiledTextFilter.Reset();
	bCompiledTextFilterRequiresVersePaths = false;
}

bool SAssetView::IsFilteringRecursively() const
{
	if (!bFilterRecursivelyWithBackendFilter)
	{
		return false;
	}

	// New CB Style has setting among asset filter options
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		if (TSharedPtr<SFilterList> FilterBarPinned = FilterBar.Pin())
		{
			return FilterBarPinned->IsFilteringPathsRecursively();
		}
	}

	// In some cases we want to not filter recursively even if we have a backend filter (e.g. the open level window)
	// Most of the time, bFilterRecursivelyWithBackendFilter is true
	if (const FContentBrowserInstanceConfig* EditorConfig = GetContentBrowserConfig())
	{
		return EditorConfig->bFilterRecursively;
	}

	return GetDefault<UContentBrowserSettings>()->FilterRecursively;
}

bool SAssetView::IsToggleFilteringRecursivelyAllowed() const
{
	return bFilterRecursivelyWithBackendFilter;
}

void SAssetView::ToggleFilteringRecursively()
{
	check(IsToggleFilteringRecursivelyAllowed());

	bool bNewState = !GetDefault<UContentBrowserSettings>()->FilterRecursively;
	
	if (FContentBrowserInstanceConfig* EditorConfig = GetContentBrowserConfig())
	{
		bNewState = !EditorConfig->bFilterRecursively;

		EditorConfig->bFilterRecursively = bNewState;
		UContentBrowserConfig::Get()->SaveEditorConfig();
	}

	GetMutableDefault<UContentBrowserSettings>()->FilterRecursively = bNewState;
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
}

bool SAssetView::ShouldFilterRecursively() const
{
	// Quick check for conditions that activate the recursive filtering
	if (bUserSearching)
	{
		return true;
	}

	if (IsFilteringRecursively())
	{
		if (!BackendFilter.IsEmpty() )
		{
			return true;
		}

		// Otherwise, check if there are any non-inverse frontend filters selected
		if (FrontendFilters.IsValid())
		{
			for (const TSharedPtr<IFilter<FAssetFilterType>>& Filter : *FrontendFilters.Get())
			{
				if (const auto* FrontendFilter = static_cast<FFrontendFilter*>(Filter.Get()))
				{
					if (!FrontendFilter->IsInverseFilter())
					{
						return true;
					}
				}
			}
		}
	}

	// No sources - view will show everything
	if (ContentSources.IsEmpty())
	{
		return true;
	}

	// No filters, do not override folder view with recursive filtering
	return false;
}

void SAssetView::RefreshFilteredItems()
{
	const double RefreshFilteredItemsStartTime = FPlatformTime::Seconds();

	OnInterruptFiltering();

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	if (AssetSearchOverride.IsValid() && AssetSearchOverride->IsOverrideActive())
	{
		UE::Editor::ContentBrowser::Extension::FInternalApiWrapper& SearchOverrideApi = AssetSearchOverride->GetInternalApi();
		SearchOverrideApi.OnFilteringReset();
	}
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	FilteredAssetItems.Reset();
	AlreadyAddedOriginalAssets.Reset();
	FilteredAssetItemTypeCounts.Reset();
	RelevantThumbnails.Reset();

	AmortizeStartTime = 0;
	InitialNumAmortizedTasks = 0;

	LastSortTime = 0;
	bPendingSortFilteredItems = true;

	Items->AbortTextFiltering();
	Items->ResetFilterState();
	if (TextFilter.IsValid())
	{
		Items->StartTextFiltering(TextFilter);
	}

	// Let the frontend filters know the currently used asset filter in case it is necessary to conditionally filter based on path or class filters
	if (IsFrontendFilterActive() && FrontendFilters.IsValid())
	{
		static const FName RootPath = "/";
		const TArrayView<const FName> DataSourcePaths = ContentSources.HasVirtualPaths() ? MakeArrayView(ContentSources.GetVirtualPaths()) : MakeArrayView(&RootPath, 1);

		const bool bInvalidateFilterCache = false;
		const FContentBrowserDataFilter DataFilter = CreateBackendDataFilter(bInvalidateFilterCache);

		for (const TSharedPtr<IFilter<FAssetFilterType>>& Filter : *FrontendFilters.Get())
		{
			// There are only FFrontendFilters in this collection
			const TSharedPtr<FFrontendFilter>& FrontendFilter = StaticCastSharedPtr<FFrontendFilter>(Filter);
			if (FrontendFilter.IsValid())
			{
				FrontendFilter->SetCurrentFilter(DataSourcePaths, DataFilter);
			}
		}
	}

	UE_LOGF(LogContentBrowser, VeryVerbose, "AssetView - RefreshFilteredItems completed in %0.4f seconds", FPlatformTime::Seconds() - RefreshFilteredItemsStartTime);
}

FContentBrowserInstanceConfig* SAssetView::GetContentBrowserConfig() const
{
	if (TSharedPtr<SContentBrowser> ContentBrowser = OwningContentBrowser.Pin())
	{
		if (UContentBrowserConfig* EditorConfig = UContentBrowserConfig::Get())
		{
			return UContentBrowserConfig::Get()->Instances.Find(ContentBrowser->GetInstanceName());
		}
	}
	return nullptr;
}

FAssetViewInstanceConfig* SAssetView::GetAssetViewConfig() const
{
	if (TSharedPtr<SContentBrowser> ContentBrowser = OwningContentBrowser.Pin())
	{
		const FName InstanceName = ContentBrowser->GetInstanceName();
		if (!InstanceName.IsNone())
		{
			if (UAssetViewConfig* Config = UAssetViewConfig::Get())
			{
				return &Config->GetInstanceConfig(InstanceName);
			}
		}
	}

	return nullptr;
}

void SAssetView::BindCommands()
{
	Commands = TSharedPtr<FUICommandList>(new FUICommandList);

	Commands->MapAction(FGenericCommands::Get().Copy, FUIAction(
		FExecuteAction::CreateSP(this, &SAssetView::ExecuteCopy, EAssetViewCopyType::ExportTextPath)
	));

	Commands->MapAction(FContentBrowserCommands::Get().AssetViewCopyObjectPath, FUIAction(
		FExecuteAction::CreateSP(this, &SAssetView::ExecuteCopy, EAssetViewCopyType::ObjectPath)
	));

	Commands->MapAction(FContentBrowserCommands::Get().AssetViewCopyPackageName, FUIAction(
		FExecuteAction::CreateSP(this, &SAssetView::ExecuteCopy, EAssetViewCopyType::PackageName)
	));

	Commands->MapAction(FGenericCommands::Get().Paste, FUIAction(
		FExecuteAction::CreateSP(this, &SAssetView::ExecutePaste),
		FCanExecuteAction::CreateSP(this, &SAssetView::IsAssetPathSelected)
	));

	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		Commands->MapAction(FContentBrowserCommands::Get().GridViewShortcut, FUIAction(
			FExecuteAction::CreateSP(this, &SAssetView::SetCurrentViewTypeFromMenu, EAssetViewType::Tile),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SAssetView::IsCurrentViewType, EAssetViewType::Tile)
		));

		Commands->MapAction(FContentBrowserCommands::Get().ListViewShortcut, FUIAction(
			FExecuteAction::CreateSP(this, &SAssetView::SetCurrentViewTypeFromMenu, EAssetViewType::List),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SAssetView::IsCurrentViewType, EAssetViewType::List)
		));

		Commands->MapAction(FContentBrowserCommands::Get().ColumnViewShortcut, FUIAction(
		FExecuteAction::CreateSP(this, &SAssetView::SetCurrentViewTypeFromMenu, EAssetViewType::Column),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SAssetView::IsCurrentViewType, EAssetViewType::Column)
		));
	}

	FInputBindingManager::Get().RegisterCommandList(FContentBrowserCommands::Get().GetContextName(), Commands.ToSharedRef());
}

void SAssetView::PopulateSelectedFilesAndFolders(TArray<FContentBrowserItem>& OutSelectedFolders, TArray<FContentBrowserItem>& OutSelectedFiles) const
{
	for (const FContentBrowserItem& SelectedItem : GetSelectedItems())
	{
		if (SelectedItem.IsFile())
		{
			OutSelectedFiles.Add(SelectedItem);
		}
		else if (SelectedItem.IsFolder())
		{
			OutSelectedFolders.Add(SelectedItem);
		}
	}
}

void SAssetView::ExecuteCopy(EAssetViewCopyType InCopyType) const
{
	TArray<FContentBrowserItem> SelectedFiles;
	TArray<FContentBrowserItem> SelectedFolders;

	PopulateSelectedFilesAndFolders(SelectedFolders, SelectedFiles);

	FString ClipboardText;
	if (SelectedFiles.Num() > 0)
	{
		switch (InCopyType)
		{
			case EAssetViewCopyType::ExportTextPath:
				ClipboardText += ContentBrowserUtils::GetItemReferencesText(SelectedFiles);
				break;

			case EAssetViewCopyType::ObjectPath:
				ClipboardText += ContentBrowserUtils::GetItemObjectPathText(SelectedFiles);
				break;

			case EAssetViewCopyType::PackageName:
				ClipboardText += ContentBrowserUtils::GetItemPackageNameText(SelectedFiles);
				break;
		}
	}

	ExecuteCopyFolders(SelectedFolders, ClipboardText);

	if (!ClipboardText.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*ClipboardText);
	}
}

void SAssetView::ExecuteCopyFolders(const TArray<FContentBrowserItem>& InSelectedFolders, FString& OutClipboardText) const
{
	const FString FolderReferencesText = ContentBrowserUtils::GetFolderReferencesText(InSelectedFolders);
	if (!FolderReferencesText.IsEmpty())
	{
		if (!OutClipboardText.IsEmpty())
		{
			OutClipboardText += LINE_TERMINATOR;
		}
		OutClipboardText += FolderReferencesText;
	}
}

static bool IsValidObjectPath(const FString& Path, FString& OutObjectClassName, FString& OutObjectPath, FString& OutPackageName)
{
	if (FPackageName::ParseExportTextPath(Path, &OutObjectClassName, &OutObjectPath))
	{
		if (UClass* ObjectClass = UClass::TryFindTypeSlow<UClass>(OutObjectClassName, EFindFirstObjectOptions::ExactClass))
		{
			OutPackageName = FPackageName::ObjectPathToPackageName(OutObjectPath);
			if (FPackageName::IsValidLongPackageName(OutPackageName))
			{
				return true;
			}
		}
	}

	return false;
}

static bool ContainsT3D(const FString& ClipboardText)
{
	return (ClipboardText.StartsWith(TEXT("Begin Object")) && ClipboardText.EndsWith(TEXT("End Object")))
		|| (ClipboardText.StartsWith(TEXT("Begin Map")) && ClipboardText.EndsWith(TEXT("End Map")));
}

void SAssetView::ExecutePaste()
{
	FString AssetPaths;
	TArray<FString> AssetPathsSplit;

	// Get the copied asset paths
	FPlatformApplicationMisc::ClipboardPaste(AssetPaths);

	// Make sure the clipboard does not contain T3D
	AssetPaths.TrimEndInline();
	if (!ContainsT3D(AssetPaths))
	{
		AssetPaths.ParseIntoArrayLines(AssetPathsSplit);

		// Get assets and copy them
		TArray<UObject*> AssetsToCopy;
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		for (const FString& AssetPath : AssetPathsSplit)
		{
			// Validate string
			FString ObjectClassName;
			FString ObjectPath;
			FString PackageName;
			if (IsValidObjectPath(AssetPath, ObjectClassName, ObjectPath, PackageName))
			{
				// Only duplicate the objects of the supported classes.
				if (AssetToolsModule.Get().GetAssetClassPathPermissionList(EAssetClassAction::ViewAsset)->PassesStartsWithFilter(ObjectClassName))
				{
					FLinkerInstancingContext InstancingContext({ ULevel::LoadAllExternalObjectsTag });
					UObject* ObjectToCopy = LoadObject<UObject>(nullptr, *ObjectPath, nullptr, LOAD_None, nullptr, &InstancingContext);
					if (ObjectToCopy && !ObjectToCopy->IsA(UClass::StaticClass()))
					{
						AssetsToCopy.Add(ObjectToCopy);
					}
				}
			}
		}

		if (AssetsToCopy.Num())
		{
			UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
			if (ensure(ContentBrowserData))
			{
				for (const FName& SelectedVirtualPath : ContentSources.GetVirtualPaths())
				{
					const FContentBrowserItem SelectedItem = ContentBrowserData->GetItemAtPath(SelectedVirtualPath, EContentBrowserItemTypeFilter::IncludeFolders);
					if (SelectedItem.IsValid())
					{
						FName PackagePath;
						if (SelectedItem.Legacy_TryGetPackagePath(PackagePath))
						{
							ContentBrowserUtils::CopyAssets(AssetsToCopy, PackagePath.ToString());
							break;
						}
					}
				}
			}
		}
	}
}

bool SAssetView::IsCustomViewSet() const
{
	return ViewExtender.IsValid();
}

TSharedRef<SWidget> SAssetView::CreateCustomView()
{
	return IsCustomViewSet() ? ViewExtender->CreateView(&FilteredAssetItems) : SNullWidget::NullWidget;
}

void SAssetView::ToggleShowAllFolder()
{
	const bool bNewValue = !IsShowingAllFolder();
	GetMutableDefault<UContentBrowserSettings>()->bShowAllFolder = bNewValue;
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
}

bool SAssetView::IsShowingAllFolder() const
{
	return GetDefault<UContentBrowserSettings>()->bShowAllFolder;
}

void SAssetView::ToggleOrganizeFolders()
{
	const bool bNewValue = !IsOrganizingFolders();
	GetMutableDefault<UContentBrowserSettings>()->bOrganizeFolders = bNewValue;
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
}

bool SAssetView::IsOrganizingFolders() const
{
	return GetDefault<UContentBrowserSettings>()->bOrganizeFolders;
}

void SAssetView::SetMajorityAssetType(FName NewMajorityAssetType)
{
	if (CurrentViewType != EAssetViewType::Column)
	{
		return;
	}

	auto IsFixedColumn = [this](FName InColumnId)
	{
		const bool bIsFixedNameColumn = InColumnId == SortManager->NameColumnId;
		const bool bIsFixedRevisionControlColumn = InColumnId == SortManager->RevisionControlColumnId;
		const bool bIsFixedClassColumn = bShowTypeInColumnView && InColumnId == SortManager->ClassColumnId;
		const bool bIsFixedPathColumn = bShowPathInColumnView && InColumnId == SortManager->PathColumnId;
		const bool bIsFixedAssetAccessSpecifierColumn = bShowAssetAccessSpecifier && InColumnId == SortManager->AssetAccessSpecifierColumnId;
		return bIsFixedNameColumn || bIsFixedRevisionControlColumn || bIsFixedClassColumn || bIsFixedPathColumn || bIsFixedAssetAccessSpecifierColumn;
	};


	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	bool bHasDynamicColumns = ContentBrowserModule.IsDynamicTagAssetClass(FTopLevelAssetPath(NewMajorityAssetType.ToString()));

	if ( NewMajorityAssetType != MajorityAssetType || bHasDynamicColumns)
	{
		UE_LOGF(LogContentBrowser, Verbose, "The majority of assets in the view are of type: %ls", *NewMajorityAssetType.ToString());

		MajorityAssetType = NewMajorityAssetType;
		MajorityAssetTypeText = FText::GetEmpty();

		TArray<FName> AddedColumns;
		TSharedPtr<SListView<TSharedPtr<FAssetViewItem>>> ViewToUse = ColumnView;

		// Since the asset type has changed, remove all columns except name and class
		const TIndirectArray<SHeaderRow::FColumn>& Columns = ViewToUse->GetHeaderRow()->GetColumns();

		for ( int32 ColumnIdx = Columns.Num() - 1; ColumnIdx >= 0; --ColumnIdx )
		{
			const FName ColumnId = Columns[ColumnIdx].ColumnId;

			if ( ColumnId != NAME_None && !IsFixedColumn(ColumnId) )
			{
				ViewToUse->GetHeaderRow()->RemoveColumn(ColumnId);
			}
		}

		// Keep track of the current column name to see if we need to change it now that columns are being removed
		// Name, Class, and Path are always relevant
		struct FSortOrder
		{
			bool bSortRelevant;
			FName SortColumn;
			FSortOrder(bool bInSortRelevant, const FName& InSortColumn) : bSortRelevant(bInSortRelevant), SortColumn(InSortColumn) {}
		};
		TArray<FSortOrder> CurrentSortOrder;
		for (int32 PriorityIdx = 0; PriorityIdx < EColumnSortPriority::Max; PriorityIdx++)
		{
			const FName SortColumn = SortManager->GetSortColumnId(static_cast<EColumnSortPriority::Type>(PriorityIdx));
			if (SortColumn != NAME_None)
			{
				const bool bSortRelevant = SortColumn == FAssetViewSortManager::NameColumnId
					|| SortColumn == FAssetViewSortManager::ClassColumnId
					|| SortColumn == FAssetViewSortManager::PathColumnId
					|| SortColumn == FAssetViewSortManager::AssetAccessSpecifierColumnId;
				CurrentSortOrder.Add(FSortOrder(bSortRelevant, SortColumn));
			}
		}

		// Add custom columns
		for (const FAssetViewCustomColumn& Column : CustomColumns)
		{
			FName TagName = Column.ColumnName;

			if (AddedColumns.Contains(TagName))
			{
				continue;
			}
			AddedColumns.Add(TagName);

			ViewToUse->GetHeaderRow()->AddColumn(
				SHeaderRow::Column(TagName)
				.SortMode(TAttribute< EColumnSortMode::Type >::Create(TAttribute< EColumnSortMode::Type >::FGetter::CreateSP(this, &SAssetView::GetColumnSortMode, TagName)))
				.SortPriority(TAttribute< EColumnSortPriority::Type >::Create(TAttribute< EColumnSortPriority::Type >::FGetter::CreateSP(this, &SAssetView::GetColumnSortPriority, TagName)))
				.OnSort(FOnSortModeChanged::CreateSP(this, &SAssetView::OnSortColumnHeader))
				.DefaultLabel(Column.DisplayName)
				.DefaultTooltip(Column.TooltipText)
				.FillWidth(180));

			const bool bIsColumnVisible = !HiddenColumnNames.Contains(TagName.ToString());
		
			ViewToUse->GetHeaderRow()->SetShowGeneratedColumn(TagName, bIsColumnVisible);

			// If we found a tag the matches the column we are currently sorting on, there will be no need to change the column
			for (int32 SortIdx = 0; SortIdx < CurrentSortOrder.Num(); SortIdx++)
			{
				if (TagName == CurrentSortOrder[SortIdx].SortColumn)
				{
					CurrentSortOrder[SortIdx].bSortRelevant = true;
				}
			}
		}

		// If we have a new majority type, add the new type's columns
		if (NewMajorityAssetType != NAME_None)
		{
			FContentBrowserItemDataAttributeValues UnionedItemAttributes;

			// Find an item of this type so we can extract the relevant attribute data from it
			TSharedPtr<FAssetViewItem> MajorityAssetItem;
			for (const TSharedPtr<FAssetViewItem>& FilteredAssetItem : FilteredAssetItems)
			{
				const FContentBrowserItemDataAttributeValue ClassValue = FilteredAssetItem->GetItem().GetItemAttribute(ContentBrowserItemAttributes::ItemTypeName);
				if (ClassValue.IsValid() && ClassValue.GetValue<FName>() == NewMajorityAssetType)
				{
					if (bHasDynamicColumns)
					{
						const FContentBrowserItemDataAttributeValues ItemAttributes = FilteredAssetItem->GetItem().GetItemAttributes(/*bIncludeMetaData*/true);
						UnionedItemAttributes.Append(ItemAttributes); 
						MajorityAssetItem = FilteredAssetItem;
					}
					else
					{
						MajorityAssetItem = FilteredAssetItem;
						break;
					}
				}
			}

			MajorityAssetTypeText = MajorityAssetItem ? MajorityAssetItem->GetAssetClassText() : FText::GetEmpty();

			// Determine the columns by querying the reference item
			if (MajorityAssetItem)
			{
				FContentBrowserItemDataAttributeValues ItemAttributes = bHasDynamicColumns ? UnionedItemAttributes : MajorityAssetItem->GetItem().GetItemAttributes(/*bIncludeMetaData*/true);

				// Add a column for every tag that isn't hidden or using a reserved name
				for (const auto& TagPair : ItemAttributes)
				{
					if (IsFixedColumn(TagPair.Key))
					{
						// Reserved name
						continue;
					}

					if (TagPair.Value.GetMetaData().AttributeType == UObject::FAssetRegistryTag::TT_Hidden)
					{
						// Hidden attribute
						continue;
					}

					if (!OnAssetTagWantsToBeDisplayed.IsBound() || OnAssetTagWantsToBeDisplayed.Execute(NewMajorityAssetType, TagPair.Key))
					{
						if (AddedColumns.Contains(TagPair.Key))
						{
							continue;
						}
						AddedColumns.Add(TagPair.Key);

						ViewToUse->GetHeaderRow()->AddColumn(
							SHeaderRow::Column(TagPair.Key)
							.SortMode(TAttribute< EColumnSortMode::Type >::Create(TAttribute< EColumnSortMode::Type >::FGetter::CreateSP(this, &SAssetView::GetColumnSortMode, TagPair.Key)))
							.SortPriority(TAttribute< EColumnSortPriority::Type >::Create(TAttribute< EColumnSortPriority::Type >::FGetter::CreateSP(this, &SAssetView::GetColumnSortPriority, TagPair.Key)))
							.OnSort(FOnSortModeChanged::CreateSP(this, &SAssetView::OnSortColumnHeader))
							.DefaultLabel(TagPair.Value.GetMetaData().DisplayName)
							.DefaultTooltip(TagPair.Value.GetMetaData().TooltipText)
							.FillWidth(180));

						const bool bIsColumnVisible = !HiddenColumnNames.Contains(TagPair.Key.ToString());
		
						ViewToUse->GetHeaderRow()->SetShowGeneratedColumn(TagPair.Key, bIsColumnVisible);
						
						// If we found a tag the matches the column we are currently sorting on, there will be no need to change the column
						for (int32 SortIdx = 0; SortIdx < CurrentSortOrder.Num(); SortIdx++)
						{
							if (TagPair.Key == CurrentSortOrder[SortIdx].SortColumn)
							{
								CurrentSortOrder[SortIdx].bSortRelevant = true;
							}
						}
					}
				}
			}
		}

		// Are any of the sort columns irrelevant now, if so remove them from the list
		bool CurrentSortChanged = false;
		for (int32 SortIdx = CurrentSortOrder.Num() - 1; SortIdx >= 0; SortIdx--)
		{
			if (!CurrentSortOrder[SortIdx].bSortRelevant)
			{
				CurrentSortOrder.RemoveAt(SortIdx);
				CurrentSortChanged = true;
			}
		}
		if (CurrentSortOrder.Num() > 0 && CurrentSortChanged)
		{
			// Sort order has changed, update the columns keeping those that are relevant
			int32 PriorityNum = EColumnSortPriority::Primary;
			for (int32 SortIdx = 0; SortIdx < CurrentSortOrder.Num(); SortIdx++)
			{
				check(CurrentSortOrder[SortIdx].bSortRelevant);
				if (!SortManager->SetOrToggleSortColumn(static_cast<EColumnSortPriority::Type>(PriorityNum), CurrentSortOrder[SortIdx].SortColumn))
				{
					// Toggle twice so mode is preserved if this isn't a new column assignation
					SortManager->SetOrToggleSortColumn(static_cast<EColumnSortPriority::Type>(PriorityNum), CurrentSortOrder[SortIdx].SortColumn);
				}				
				bPendingSortFilteredItems = true;
				PriorityNum++;
			}
		}
		else if (CurrentSortOrder.Num() == 0)
		{
			// If the current sort column is no longer relevant, revert to "Name" and resort when convenient
			SortManager->ResetSort();
			bPendingSortFilteredItems = true;
		}
	}
}

void SAssetView::OnAssetsAddedToCollection( ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> ObjectPaths )
{
	if (!ContentSources.GetCollections().ContainsByPredicate([&CollectionContainer, &Collection](const FCollectionRef& CollectionRef)
		{
			return &CollectionContainer == CollectionRef.Container.Get() &&
				Collection.Name == CollectionRef.Name &&
				Collection.Type == CollectionRef.Type;
		}))
	{
		return;
	}

	RequestSlowFullListRefresh();
}

void SAssetView::OnAssetsRemovedFromCollection( ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> ObjectPaths )
{
	if (!ContentSources.GetCollections().ContainsByPredicate([&CollectionContainer, &Collection](const FCollectionRef& CollectionRef)
		{
			return &CollectionContainer == CollectionRef.Container.Get() &&
				Collection.Name == CollectionRef.Name &&
				Collection.Type == CollectionRef.Type;
		}))
	{
		return;
	}

	RequestSlowFullListRefresh();
}

void SAssetView::OnCollectionRenamed( ICollectionContainer& CollectionContainer, const FCollectionNameType& OriginalCollection, const FCollectionNameType& NewCollection )
{
	const int32 FoundIndex = ContentSources.GetCollections().IndexOfByPredicate([&CollectionContainer, &OriginalCollection](const FCollectionRef& Collection)
		{
			return &CollectionContainer == Collection.Container.Get() &&
				OriginalCollection.Name == Collection.Name &&
				OriginalCollection.Type == Collection.Type;
		});
	if (FoundIndex != INDEX_NONE)
	{
		TArray<FCollectionRef> Collections = ContentSources.GetCollections();
		Collections[ FoundIndex ] = FCollectionRef(CollectionContainer.AsShared(), NewCollection);
		ContentSources.SetCollections(Collections);
	}
}

void SAssetView::OnCollectionUpdated( ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection )
{
	// A collection has changed in some way, so we need to refresh our backend list
	RequestSlowFullListRefresh();
}

void SAssetView::OnFrontendFiltersChanged()
{
	// We're refreshing so update the redirector visibility state in case it's not also bound to a frontend filter.
	// This potentially avoids a double refresh on the next tick.
	bLastShowRedirectors = bShowRedirectors.Get(false);

	RequestQuickFrontendListRefresh();

	// Combine any currently active custom text filters with the asset text filtering task
	if (TextFilter.IsValid() && FrontendFilters.IsValid())
	{ 
		TArray<FText> CustomTextFilters;
		for (const TSharedPtr<IFilter<FAssetFilterType>>& Filter : *FrontendFilters.Get())
		{
			const TSharedPtr<FFrontendFilter> FrontendFilter = StaticCastSharedPtr<FFrontendFilter>(Filter);
			if (Filter.IsValid())
			{
				TOptional<FText> Text = FrontendFilter->GetAsCustomTextFilter();
				if (Text.IsSet())
				{
					CustomTextFilters.Add(Text.GetValue());
				}
			}
		}
		TextFilter->SetCustomTextFilters(MoveTemp(CustomTextFilters));
	}
	

	// If we're changing between recursive and non-recursive data, we need to fully refresh the source items
	if (ShouldFilterRecursively() != bWereItemsRecursivelyFiltered)
	{
		RequestSlowFullListRefresh();
	}
}

bool SAssetView::IsFrontendFilterActive() const
{
	return ( FrontendFilters.IsValid() && FrontendFilters->Num() > 0 );
}

bool SAssetView::PassesCurrentFrontendFilter(const FContentBrowserItem& Item) const
{
	return !FrontendFilters.IsValid() || FrontendFilters->PassesAllFilters(Item);
}

void SAssetView::SortList(bool bSyncToSelection)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SAssetView::SortList);

	if ( !IsRenamingAsset() )
	{
		PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		if (AssetSearchOverride.IsValid() && AssetSearchOverride->IsOverrideActive() && AssetSearchOverride->IsSortOverridden())
		{
			AssetSearchOverride->SortItemList(FilteredAssetItems);
		}
		PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
		else
		{
			SortManager->SortList(FilteredAssetItems, MajorityAssetType, CustomColumns);
		}

		// Update the thumbnails we were using since the order has changed
		bPendingUpdateThumbnails = true;

		if ( bSyncToSelection )
		{
			// Make sure the selection is in view
			const bool bFocusOnSync = false;
			SyncToSelection(bFocusOnSync);
		}

		RefreshList();
		bPendingSortFilteredItems = false;
		LastSortTime = CurrentTime;
	}
	else
	{
		bPendingSortFilteredItems = true;
	}
}

FLinearColor SAssetView::GetThumbnailHintColorAndOpacity() const
{
	//We update this color in tick instead of here as an optimization
	return ThumbnailHintColorAndOpacity;
}

TSharedRef<SWidget> SAssetView::GetViewButtonContent()
{
	// Get all menu extenders for this context menu from the content browser module
	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>( TEXT("ContentBrowser") );
	const TArray<FContentBrowserMenuExtender>& MenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewViewMenuExtenders();

	TArray<TSharedPtr<FExtender>> Extenders;
	for (const FContentBrowserMenuExtender& Extender : MenuExtenderDelegates)
	{
		if (Extender.IsBound())
		{
			Extenders.Add(Extender.Execute());
		}
	}

	UContentBrowserAssetViewContextMenuContext* Context = NewObject<UContentBrowserAssetViewContextMenuContext>();
	Context->AssetView = SharedThis(this);
	Context->OwningContentBrowser = OwningContentBrowser;

	TSharedPtr<FExtender> MenuExtender = FExtender::Combine(Extenders);
	FToolMenuContext MenuContext(nullptr, MenuExtender, Context);

	if(AssetViewOptionsProfile.IsSet())
	{
		UToolMenuProfileContext* ProfileContext = NewObject<UToolMenuProfileContext>();
		ProfileContext->ActiveProfiles.Add(AssetViewOptionsProfile.GetValue());
		MenuContext.AddObject(ProfileContext);
	}

	if (OnExtendAssetViewOptionsMenuContext.IsBound())
	{
		OnExtendAssetViewOptionsMenuContext.Execute(MenuContext);
	}

	return UToolMenus::Get()->GenerateWidget("ContentBrowser.AssetViewOptions", MenuContext);
}

void SAssetView::PopulateFilterAdditionalParams(FFiltersAdditionalParams& OutParams)
{
	OutParams.CanShowCPPClasses = FCanExecuteAction::CreateSP(this, &SAssetView::IsToggleShowCppContentAllowed);
	OutParams.CanShowDevelopersContent = FCanExecuteAction::CreateSP(this, &SAssetView::IsToggleShowDevelopersContentAllowed);
	OutParams.CanShowEngineFolder = FCanExecuteAction::CreateSP(this, &SAssetView::IsToggleShowEngineContentAllowed);
	OutParams.CanShowPluginFolder = FCanExecuteAction::CreateSP(this, &SAssetView::IsToggleShowPluginContentAllowed);
	OutParams.CanShowLocalizedContent = FCanExecuteAction::CreateSP(this, &SAssetView::IsToggleShowLocalizedContentAllowed);
}

void SAssetView::OnSetSortParameters(
	const FToolMenuContext& InMenuContext,
	const TOptional<const EColumnSortPriority::Type> InSortPriority,
	const TOptional<const FName> InColumnId,
	const TOptional<const EColumnSortMode::Type> InNewSortMode)
{
	TOptional<const EColumnSortMode::Type> NewSortMode = InNewSortMode;

	// Set sort mode to the currently active one, if none specified
	if (!InNewSortMode.IsSet())
	{
		NewSortMode.Emplace(GetSortManager().Pin()->GetSortMode(EColumnSortPriority::Primary));
	}

	SetSortParameters(InSortPriority, InColumnId, NewSortMode);
}

void SAssetView::PopulateSortingButtonMenu(UToolMenu* InToolMenu)
{
	UContentBrowserAssetSortingContextMenuContext* SortingContext = NewObject<UContentBrowserAssetSortingContextMenuContext>();
	SortingContext->OwningContentBrowser = OwningContentBrowser;
	SortingContext->AssetView = SharedThis(this);
	SortingContext->AssetViewSortManager = GetSortManager();

	InToolMenu->Context.AddObject(SortingContext);

	FToolMenuSection& SortBySection = InToolMenu->AddSection("SortBy", LOCTEXT("SortByHeading", "Sort By"));
	{
		static const TOptional<const EColumnSortPriority::Type> UnsetSortPriority;
		static const TOptional<const EColumnSortMode::Type> UnsetSortMode;

		static const TPair<FName, FText> SortableColumnIds[] =
		{
			{ FAssetViewSortManager::NameColumnId, LOCTEXT("SortByOptionLabel_Name", "Name") }, { FAssetViewSortManager::DiskSizeColumnId, LOCTEXT("SortByOptionLabel_Size", "Size") }
		};

		for (const TPair<FName, FText>& SortableColumnId : SortableColumnIds)
		{
			const TOptional<const FName> ColumnId = SortableColumnId.Key;

			FToolUIAction SortByAction;
			SortByAction.ExecuteAction =
				FToolMenuExecuteAction::CreateSP(
					this,
					&SAssetView::OnSetSortParameters,
					UnsetSortPriority,
					ColumnId,
					UnsetSortMode);

			SortByAction.GetActionCheckState =
				FToolMenuGetActionCheckState::CreateSPLambda(
					this,
					[](const FToolMenuContext& InMenuContext, const FName InId)
					{
						if (const UContentBrowserAssetSortingContextMenuContext* SortingContext = InMenuContext.FindContext<UContentBrowserAssetSortingContextMenuContext>())
						{
							if (const TSharedPtr<FAssetViewSortManager> StrongSortManager = SortingContext->AssetViewSortManager.Pin())
							{
								return StrongSortManager->GetSortColumnId(EColumnSortPriority::Primary) == InId
									? ECheckBoxState::Checked
									: ECheckBoxState::Unchecked;
							}
						}

						return ECheckBoxState::Unchecked;
					},
					*ColumnId);

			const FText& SortableColumnLabel = SortableColumnId.Value;

			SortBySection.AddMenuEntry(
				*ColumnId,
				SortableColumnLabel,
				FText::Format(LOCTEXT("SortByOptionToolTip", "Sorts the items by {0}"), SortableColumnLabel),
				FSlateIcon(),
				SortByAction,
				EUserInterfaceActionType::RadioButton);
		}
	}

	FToolMenuSection& SortTypeSection = InToolMenu->AddSection("SortType", LOCTEXT("SortTypeHeading", "Sort Type"));
	{
		auto SetSortMode = [](const FToolMenuContext& InMenuContext, const EColumnSortMode::Type InMode)
		{
			if (UContentBrowserAssetSortingContextMenuContext* SortingContext = InMenuContext.FindContext<UContentBrowserAssetSortingContextMenuContext>())
			{
				if (TSharedPtr<SAssetView> StrongAssetView = SortingContext->AssetView.Pin())
				{
					StrongAssetView->SetSortParameters(EColumnSortPriority::Primary, { }, InMode);
				}
			}
		};

		auto IsSortMode = [](const FToolMenuContext& InMenuContext, const EColumnSortMode::Type InMode)
		{
			if (const UContentBrowserAssetSortingContextMenuContext* SortingContext = InMenuContext.FindContext<UContentBrowserAssetSortingContextMenuContext>())
			{
				if (const TSharedPtr<FAssetViewSortManager> StrongSortManager = SortingContext->AssetViewSortManager.Pin())
				{
					return StrongSortManager->GetSortMode(EColumnSortPriority::Primary) == InMode
						? ECheckBoxState::Checked
						: ECheckBoxState::Unchecked;
				}
			}

			return ECheckBoxState::Unchecked;
		};

		FToolUIAction SortAscendingAction;
		SortAscendingAction.ExecuteAction =
			FToolMenuExecuteAction::CreateSPLambda(
				this,
				SetSortMode,
				EColumnSortMode::Ascending);

		SortAscendingAction.GetActionCheckState =
			FToolMenuGetActionCheckState::CreateSPLambda(
				this,
				IsSortMode,
				EColumnSortMode::Ascending);

		SortTypeSection.AddMenuEntry(
			"Ascending",
			LOCTEXT("AscendingOrder", "Ascending"),
			LOCTEXT("AscendingOrderToolTip", "Sort the items in Ascending order"),
			FSlateIcon(),
			SortAscendingAction,
			EUserInterfaceActionType::RadioButton);

		FToolUIAction SortDescendingAction;
		SortDescendingAction.ExecuteAction =
			FToolMenuExecuteAction::CreateSPLambda(
				this,
				SetSortMode,
				EColumnSortMode::Descending);

		SortDescendingAction.GetActionCheckState =
			FToolMenuGetActionCheckState::CreateSPLambda(
				this,
				IsSortMode,
				EColumnSortMode::Descending);

		SortTypeSection.AddMenuEntry(
			"Descending",
			LOCTEXT("DescendingOrder", "Descending"),
			LOCTEXT("DescendingOrderToolTip", "Sort the items in Descending order"),
			FSlateIcon(),
			SortDescendingAction,
			EUserInterfaceActionType::RadioButton);
	}
}

void SAssetView::ToggleShowFolders()
{
	check(IsToggleShowFoldersAllowed());

	bool bNewState = !GetDefault<UContentBrowserSettings>()->DisplayFolders;

	if (FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		bNewState = !Config->bShowFolders;
		Config->bShowFolders = bNewState;
		UContentBrowserConfig::Get()->SaveEditorConfig();
	}

	GetMutableDefault<UContentBrowserSettings>()->DisplayFolders = bNewState;
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
}

bool SAssetView::IsToggleShowFoldersAllowed() const
{
	return bCanShowFolders;
}

bool SAssetView::IsShowingFolders() const
{
	if (!IsToggleShowFoldersAllowed())
	{
		return false;
	}
	
	if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		return Config->bShowFolders;
	}

	return GetDefault<UContentBrowserSettings>()->DisplayFolders;
}

bool SAssetView::IsShowingReadOnlyFolders() const
{
	return bCanShowReadOnlyFolders;
}

void SAssetView::ToggleShowEmptyFolders()
{
	check(IsToggleShowEmptyFoldersAllowed());

	bool bNewState = !GetDefault<UContentBrowserSettings>()->DisplayEmptyFolders;

	if (FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		bNewState = !Config->bShowEmptyFolders;
		Config->bShowEmptyFolders = bNewState;
		UContentBrowserConfig::Get()->SaveEditorConfig();
	}
	
	GetMutableDefault<UContentBrowserSettings>()->DisplayEmptyFolders = !GetDefault<UContentBrowserSettings>()->DisplayEmptyFolders;
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
}

bool SAssetView::IsToggleShowEmptyFoldersAllowed() const
{
	return bCanShowFolders;
}

bool SAssetView::IsShowingEmptyFolders() const
{
	if (!IsToggleShowEmptyFoldersAllowed())
	{
		return false;
	}

	if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		return Config->bShowEmptyFolders;
	}

	return GetDefault<UContentBrowserSettings>()->DisplayEmptyFolders;
}

bool SAssetView::IsShowingRedirectors() const
{
	return bShowRedirectors.Get(false);
}

void SAssetView::ToggleRealTimeThumbnails()
{
	check(CanShowRealTimeThumbnails());

	bool bNewState = !IsShowingRealTimeThumbnails();

	GetMutableDefault<UContentBrowserSettings>()->RealTimeThumbnails = bNewState;
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
}

bool SAssetView::CanShowRealTimeThumbnails() const
{
	return bCanShowRealTimeThumbnails;
}

bool SAssetView::IsShowingRealTimeThumbnails() const
{
	if (!CanShowRealTimeThumbnails())
	{
		return false;
	}

	return GetDefault<UContentBrowserSettings>()->RealTimeThumbnails;
}

void SAssetView::ToggleShowPluginContent()
{
	check(IsToggleShowPluginContentAllowed());

	bool bNewState = !GetDefault<UContentBrowserSettings>()->GetDisplayPluginFolders();

	if (FContentBrowserInstanceConfig* EditorConfig = GetContentBrowserConfig())
	{
		bNewState = !EditorConfig->bShowPluginContent;
		EditorConfig->bShowPluginContent = bNewState;
		UContentBrowserConfig::Get()->SaveEditorConfig();
	}

	GetMutableDefault<UContentBrowserSettings>()->SetDisplayPluginFolders(bNewState);
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
}

bool SAssetView::IsShowingPluginContent() const
{
	if (bForceShowPluginContent)
	{
		return true;
	}

	if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		return Config->bShowPluginContent;
	}

	return GetDefault<UContentBrowserSettings>()->GetDisplayPluginFolders();
}

void SAssetView::ToggleShowEngineContent()
{
	check(IsToggleShowEngineContentAllowed());

	bool bNewState = !GetDefault<UContentBrowserSettings>()->GetDisplayEngineFolder();

	if (FContentBrowserInstanceConfig* EditorConfig = GetContentBrowserConfig())
	{
		bNewState = !EditorConfig->bShowEngineContent;
		EditorConfig->bShowEngineContent = bNewState;
		UContentBrowserConfig::Get()->SaveEditorConfig();
	}

	GetMutableDefault<UContentBrowserSettings>()->SetDisplayEngineFolder(bNewState);
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
}

bool SAssetView::IsShowingEngineContent() const
{
	if (bForceShowEngineContent)
	{
		return true;
	}

	if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		return Config->bShowEngineContent;
	}

	return GetDefault<UContentBrowserSettings>()->GetDisplayEngineFolder();
}

void SAssetView::ToggleShowDevelopersContent()
{
	check(IsToggleShowDevelopersContentAllowed());

	bool bNewState = !GetDefault<UContentBrowserSettings>()->GetDisplayDevelopersFolder();

	if (FContentBrowserInstanceConfig* EditorConfig = GetContentBrowserConfig())
	{
		bNewState = !EditorConfig->bShowDeveloperContent;
		EditorConfig->bShowDeveloperContent = bNewState;
		UContentBrowserConfig::Get()->SaveEditorConfig();
	}

	GetMutableDefault<UContentBrowserSettings>()->SetDisplayDevelopersFolder(bNewState);
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
}

bool SAssetView::IsToggleShowDevelopersContentAllowed() const
{
	return bCanShowDevelopersFolder;
}

bool SAssetView::IsToggleShowEngineContentAllowed() const
{
	return !bForceShowEngineContent;
}

bool SAssetView::IsToggleShowPluginContentAllowed() const
{
	return !bForceShowPluginContent;
}

bool SAssetView::IsShowingDevelopersContent() const
{
	if (!IsToggleShowDevelopersContentAllowed())
	{
		return false;
	}

	if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		return Config->bShowDeveloperContent;
	}

	return GetDefault<UContentBrowserSettings>()->GetDisplayDevelopersFolder();
}

void SAssetView::ToggleShowLocalizedContent()
{
	check(IsToggleShowLocalizedContentAllowed());

	bool bNewState = !GetDefault<UContentBrowserSettings>()->GetDisplayL10NFolder();

	if (FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		bNewState = !Config->bShowLocalizedContent;
		Config->bShowLocalizedContent = bNewState;
		UContentBrowserConfig::Get()->SaveEditorConfig();
	}

	GetMutableDefault<UContentBrowserSettings>()->SetDisplayL10NFolder(bNewState);
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
}

bool SAssetView::IsToggleShowLocalizedContentAllowed() const
{
	return true;
}

bool SAssetView::IsShowingLocalizedContent() const
{
	if (!IsToggleShowLocalizedContentAllowed())
	{
		return false;
	}

	if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		return Config->bShowLocalizedContent;
	}

	return GetDefault<UContentBrowserSettings>()->GetDisplayL10NFolder();
}

void SAssetView::ToggleShowFavorites()
{
	check(IsToggleShowFavoritesAllowed());

	bool bNewState = !GetDefault<UContentBrowserSettings>()->GetDisplayFavorites();

	if (FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		bNewState = !Config->bShowFavorites;
		Config->bShowFavorites = bNewState;
		UContentBrowserConfig::Get()->SaveEditorConfig();
	}

	GetMutableDefault<UContentBrowserSettings>()->SetDisplayFavorites(bNewState);
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
}

bool SAssetView::IsToggleShowFavoritesAllowed() const
{
	return bCanShowFavorites;
}

bool SAssetView::IsShowingFavorites() const
{
	if (!IsToggleShowFavoritesAllowed())
	{
		return false;
	}

	if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		return Config->bShowFavorites;
	}

	return GetDefault<UContentBrowserSettings>()->GetDisplayFavorites();
}

bool SAssetView::IsToggleShowCppContentAllowed() const
{
	return bCanShowClasses;
}

bool SAssetView::IsShowingCppContent() const
{
	if (!IsToggleShowCppContentAllowed())
	{
		return false;
	}

	if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		return Config->bShowCppFolders;
	}
	
	return GetDefault<UContentBrowserSettings>()->GetDisplayCppFolders();
}

void SAssetView::ToggleIncludeClassNames()
{
	check(IsToggleIncludeClassNamesAllowed());

	bool bNewState = !GetDefault<UContentBrowserSettings>()->GetIncludeClassNames();

	if (FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		bNewState = !Config->bSearchClasses;
		Config->bSearchClasses = bNewState;
		UContentBrowserConfig::Get()->SaveEditorConfig();
	}

	GetMutableDefault<UContentBrowserSettings>()->SetIncludeClassNames(bNewState);
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();

	OnSearchOptionsChanged.ExecuteIfBound();
}

bool SAssetView::IsToggleIncludeClassNamesAllowed() const
{
	return true;
}

bool SAssetView::IsIncludingClassNames() const
{
	if (!IsToggleIncludeClassNamesAllowed())
	{
		return false;
	}

	if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		return Config->bSearchClasses;
	}
	
	return GetDefault<UContentBrowserSettings>()->GetIncludeClassNames();
}

void SAssetView::ToggleIncludeAssetPaths()
{
	check(IsToggleIncludeAssetPathsAllowed());

	bool bNewState = !GetDefault<UContentBrowserSettings>()->GetIncludeAssetPaths();

	if (FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		bNewState = !Config->bSearchAssetPaths;
		Config->bSearchAssetPaths = bNewState;
		UContentBrowserConfig::Get()->SaveEditorConfig();
	}

	GetMutableDefault<UContentBrowserSettings>()->SetIncludeAssetPaths(bNewState);
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();

	OnSearchOptionsChanged.ExecuteIfBound();
}

bool SAssetView::IsToggleIncludeAssetPathsAllowed() const
{
	return true;
}

bool SAssetView::IsIncludingAssetPaths() const
{
	if (!IsToggleIncludeAssetPathsAllowed())
	{
		return false;
	}

	if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		return Config->bSearchAssetPaths;
	}
	
	return GetDefault<UContentBrowserSettings>()->GetIncludeAssetPaths();
}

void SAssetView::ToggleIncludeCollectionNames()
{
	check(IsToggleIncludeCollectionNamesAllowed());

	bool bNewState = !GetDefault<UContentBrowserSettings>()->GetIncludeCollectionNames();

	if (FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		bNewState = !Config->bSearchCollections;
		Config->bSearchCollections = bNewState;
		UContentBrowserConfig::Get()->SaveEditorConfig();
	}

	GetMutableDefault<UContentBrowserSettings>()->SetIncludeCollectionNames(bNewState);
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();

	OnSearchOptionsChanged.ExecuteIfBound();
}

bool SAssetView::IsToggleIncludeCollectionNamesAllowed() const
{
	return true;
}

bool SAssetView::IsIncludingCollectionNames() const
{
	if (!IsToggleIncludeCollectionNamesAllowed())
	{
		return false;
	}
	
	if (const FContentBrowserInstanceConfig* Config = GetContentBrowserConfig())
	{
		return Config->bSearchCollections;
	}
	
	return GetDefault<UContentBrowserSettings>()->GetIncludeCollectionNames();
}

void SAssetView::SetCurrentViewType(EAssetViewType::Type NewType)
{
	if ( ensure(NewType != EAssetViewType::MAX) && NewType != CurrentViewType )
	{
		// If we are setting to the custom type, but the view extender does not exist for some reason - default back to tile
		if(NewType == EAssetViewType::Custom)
		{
			if(!IsCustomViewSet())
			{
				NewType = EAssetViewType::Tile;
			}
		}
		
		ResetQuickJump();

		CurrentViewType = NewType;
		CreateCurrentView();

		SyncToSelection();

		// Clear relevant thumbnails to render fresh ones in the new view if needed
		RelevantThumbnails.Reset();
		VisibleItems.Reset();

		if ( NewType == EAssetViewType::Tile )
		{
			CurrentThumbnailSize = TileViewThumbnailSize;
			bPendingUpdateThumbnails = true;
		}
		else if ( NewType == EAssetViewType::List )
		{
			if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
			{
				if (ThumbnailSizes[CurrentViewType] >= EThumbnailSize::Small)
				{
					CurrentThumbnailSize = ListViewThumbnailSize;
					bPendingUpdateThumbnails = true;
				}
			}
			else
			{
				CurrentThumbnailSize = ListViewThumbnailSize;
				bPendingUpdateThumbnails = true;
			}
		}
		else if ( NewType == EAssetViewType::Column )
		{
			if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
			{
				if (ThumbnailSizes[CurrentViewType] >= EThumbnailSize::Small)
				{
					CurrentThumbnailSize = ListViewThumbnailSize;
					bPendingUpdateThumbnails = true;
				}
			}
			// No thumbnails, but we do need to refresh filtered items to determine a majority asset type
			MajorityAssetType = NAME_None;
			RefreshFilteredItems();
			SortList();
		}

		// Update the size value when switching view to match the current view size
		UpdateThumbnailSizeValue();

		if (FAssetViewInstanceConfig* Config = GetAssetViewConfig())
		{
			Config->ViewType = (uint8) NewType;
			UAssetViewConfig::Get()->SaveEditorConfig();
		}
	}
}

void SAssetView::SetCurrentThumbnailSize(EThumbnailSize NewThumbnailSize)
{
	if (ThumbnailSizes[CurrentViewType] != NewThumbnailSize)
	{
		OnThumbnailSizeChanged(NewThumbnailSize);
	}
}

void SAssetView::SetCurrentViewTypeFromMenu(EAssetViewType::Type NewType)
{
	if (NewType != CurrentViewType)
	{
		SetCurrentViewType(NewType);
	}
}

void SAssetView::CreateCurrentView()
{
	TileView.Reset();
	ListView.Reset();
	ColumnView.Reset();

	TSharedRef<SWidget> NewView = SNullWidget::NullWidget;
	switch (CurrentViewType)
	{
		case EAssetViewType::Tile:
			TileView = CreateTileView();
			NewView = CreateShadowOverlay(TileView.ToSharedRef());
			break;
		case EAssetViewType::List:
			ListView = CreateListView();
			NewView = CreateShadowOverlay(ListView.ToSharedRef());
			break;
		case EAssetViewType::Column:
			if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
			{
				ColumnView = CreateListView();
			}
			else
			{
				ColumnView = CreateColumnView();
			}
			NewView = CreateShadowOverlay(ColumnView.ToSharedRef());
			break;
		case EAssetViewType::Custom:
			// The custom view does not necessarily have an accessible list, so we create a generic scroll border
			CustomView = CreateCustomView();
			NewView = CustomView.ToSharedRef();
			break;
	}
	
	ViewContainer->SetContent( NewView );
}

TSharedRef<SWidget> SAssetView::CreateShadowOverlay( TSharedRef<STableViewBase> Table )
{
	if (bForceHideScrollbar)
	{
		return Table;
	}

	return SNew(SScrollBorder, Table)
		[
			Table
		];
}

EAssetViewType::Type SAssetView::GetCurrentViewType() const
{
	return CurrentViewType;
}

bool SAssetView::IsCurrentViewType(EAssetViewType::Type ViewType) const
{
	return GetCurrentViewType() == ViewType;
}

void SAssetView::FocusList() const
{
	switch ( GetCurrentViewType() )
	{
		case EAssetViewType::List: FSlateApplication::Get().SetKeyboardFocus(ListView, EFocusCause::SetDirectly); break;
		case EAssetViewType::Tile: FSlateApplication::Get().SetKeyboardFocus(TileView, EFocusCause::SetDirectly); break;
		case EAssetViewType::Column: FSlateApplication::Get().SetKeyboardFocus(ColumnView, EFocusCause::SetDirectly); break;
	}
}

bool SAssetView::IsPendingRefresh()
{
	switch (GetCurrentViewType())
	{
	case EAssetViewType::List:
		return ListView->IsPendingRefresh();
	case EAssetViewType::Tile:
		return TileView->IsPendingRefresh();
	case EAssetViewType::Column:
		return ColumnView->IsPendingRefresh();
	default:
		return false;
	}
}

void SAssetView::RefreshList()
{
	switch ( GetCurrentViewType() )
	{
		case EAssetViewType::List: ListView->RequestListRefresh(); break;
		case EAssetViewType::Tile: TileView->RequestListRefresh(); break;
		case EAssetViewType::Column: ColumnView->RequestListRefresh(); break;
		case EAssetViewType::Custom: ViewExtender->OnItemListChanged(&FilteredAssetItems); break;
	}
}

void SAssetView::SetSelection(const TSharedPtr<FAssetViewItem>& Item)
{
	switch ( GetCurrentViewType() )
	{
		case EAssetViewType::List: ListView->SetSelection(Item); break;
		case EAssetViewType::Tile: TileView->SetSelection(Item); break;
		case EAssetViewType::Column: ColumnView->SetSelection(Item); break;
		case EAssetViewType::Custom: ViewExtender->SetSelection(Item, true, ESelectInfo::Direct); break;
	}
}

void SAssetView::SetItemSelection(const TSharedPtr<FAssetViewItem>& Item, bool bSelected, const ESelectInfo::Type SelectInfo)
{
	switch ( GetCurrentViewType() )
	{
		case EAssetViewType::List: ListView->SetItemSelection(Item, bSelected, SelectInfo); break;
		case EAssetViewType::Tile: TileView->SetItemSelection(Item, bSelected, SelectInfo); break;
		case EAssetViewType::Column: ColumnView->SetItemSelection(Item, bSelected, SelectInfo); break;
		case EAssetViewType::Custom: ViewExtender->SetSelection(Item, bSelected, SelectInfo); break;
	}
}

void SAssetView::RequestScrollIntoView(const TSharedPtr<FAssetViewItem>& Item)
{
	switch ( GetCurrentViewType() )
	{
		case EAssetViewType::List: ListView->RequestScrollIntoView(Item); break;
		case EAssetViewType::Tile: TileView->RequestScrollIntoView(Item); break;
		case EAssetViewType::Column: ColumnView->RequestScrollIntoView(Item); break;
		case EAssetViewType::Custom: ViewExtender->RequestScrollIntoView(Item); break;
	}
}

void SAssetView::OnOpenAssetsOrFolders()
{
	if (OnItemsActivated.IsBound())
	{
		OnInteractDuringFiltering();
		const TArray<FContentBrowserItem> SelectedItems = GetSelectedItems();
		OnItemsActivated.Execute(SelectedItems, EAssetTypeActivationMethod::Opened);
	}
}

void SAssetView::OnPreviewAssets()
{
	if (OnItemsActivated.IsBound())
	{
		OnInteractDuringFiltering();
		const TArray<FContentBrowserItem> SelectedItems = GetSelectedItems();
		OnItemsActivated.Execute(SelectedItems, EAssetTypeActivationMethod::Previewed);
	}
}

void SAssetView::ClearSelection(bool bForceSilent)
{
	const bool bTempBulkSelectingValue = bForceSilent ? true : bBulkSelecting;
	TGuardValue<bool> Guard(bBulkSelecting, bTempBulkSelectingValue);
	switch ( GetCurrentViewType() )
	{
		case EAssetViewType::List: ListView->ClearSelection(); break;
		case EAssetViewType::Tile: TileView->ClearSelection(); break;
		case EAssetViewType::Column: ColumnView->ClearSelection(); break;
		case EAssetViewType::Custom: ViewExtender->ClearSelection(); break;
	}
}

TSharedRef<ITableRow> SAssetView::MakeListViewWidget(TSharedPtr<FAssetViewItem> AssetItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	if ( !ensure(AssetItem.IsValid()) )
	{
		return SNew( STableRow<TSharedPtr<FAssetViewItem>>, OwnerTable );
	}

	if (UE::Editor::ContentBrowser::IsNewStyleEnabled() && CurrentViewType == EAssetViewType::Column)
	{
		// Update the cached custom data
		AssetItem->CacheCustomColumns(CustomColumns, /* bUpdateSortData */ false, /* bUpdateDisplayText */ true, /* bUpdateExisting */ false);
	}

	VisibleItems.Add(AssetItem);
	bPendingUpdateThumbnails = true;
	const FTableRowStyle& ListViewStyle = UE::Editor::ContentBrowser::IsNewStyleEnabled()
		                               ? UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetWidgetStyle<FTableRowStyle>("ContentBrowser.AssetListView.ColumnListTableRow")
		                               : FAppStyle::GetWidgetStyle<FTableRowStyle>("ContentBrowser.AssetListView.ColumnListTableRow");
	if (AssetItem->IsFolder())
	{
		return
			SNew(SAssetListViewRow, OwnerTable)
			.Style(&ListViewStyle)
			.OnDragDetected(this, &SAssetView::OnDraggingAssetItem)
			.Cursor(bAllowDragging ? EMouseCursor::GrabHand : EMouseCursor::Default)
			.Padding(this, &SAssetView::GetListViewItemPadding)
			.AssetListItem(
				SNew(SAssetListItem)
					.AssetItem(AssetItem)
					.ItemHeight(this, &SAssetView::GetListViewItemHeight)
					.CurrentThumbnailSize(this, &SAssetView::GetThumbnailSize)
					.OnRenameBegin(this, &SAssetView::AssetRenameBegin)
					.OnRenameCommit(this, &SAssetView::AssetRenameCommit)
					.OnVerifyRenameCommit(this, &SAssetView::AssetVerifyRenameCommit)
					.OnItemDestroyed(this, &SAssetView::AssetItemWidgetDestroyed)
					.ShouldAllowToolTip(this, &SAssetView::ShouldAllowToolTips)
					.HighlightText(HighlightedText)
					.ShowAssetAccessSpecifier(bShowAssetAccessSpecifier)
			);
	}
	else
	{
		TSharedPtr<FAssetThumbnail>& AssetThumbnail = RelevantThumbnails.FindOrAdd(AssetItem);
		if (!AssetThumbnail)
		{
			AssetThumbnail = MakeShared<FAssetThumbnail>(FAssetData(), ListViewThumbnailResolution, ListViewThumbnailResolution, AssetThumbnailPool);
			AssetItem->GetItem().UpdateThumbnail(*AssetThumbnail);
			AssetThumbnail->GetViewportRenderTargetTexture(); // Access the texture once to trigger it to render
		}
		
		return
			SNew(SAssetListViewRow, OwnerTable)
			.Style(&ListViewStyle)
			.OnDragDetected(this, &SAssetView::OnDraggingAssetItem)
			.Cursor(bAllowDragging ? EMouseCursor::GrabHand : EMouseCursor::Default)
			.Padding(this, &SAssetView::GetListViewItemPadding)
			.AssetListItem(
				SNew(SAssetListItem)
					.AssetThumbnail(AssetThumbnail)
					.AssetItem(AssetItem)
					.ThumbnailPadding((float)ListViewThumbnailPadding)
					.ItemHeight(this, &SAssetView::GetListViewItemHeight)
					.CurrentThumbnailSize(this, &SAssetView::GetThumbnailSize)
					.OnRenameBegin(this, &SAssetView::AssetRenameBegin)
					.OnRenameCommit(this, &SAssetView::AssetRenameCommit)
					.OnVerifyRenameCommit(this, &SAssetView::AssetVerifyRenameCommit)
					.OnItemDestroyed(this, &SAssetView::AssetItemWidgetDestroyed)
					.ShouldAllowToolTip(this, &SAssetView::ShouldAllowToolTips)
					.HighlightText(HighlightedText)
					.ThumbnailEditMode(this, &SAssetView::IsThumbnailEditMode)
					.ThumbnailLabel(ThumbnailLabel)
					.ThumbnailHintColorAndOpacity( this, &SAssetView::GetThumbnailHintColorAndOpacity )
					.AllowThumbnailHintLabel(AllowThumbnailHintLabel)
					.OnIsAssetValidForCustomToolTip(OnIsAssetValidForCustomToolTip)
					.OnGetCustomAssetToolTip(OnGetCustomAssetToolTip)
					.OnVisualizeAssetToolTip(OnVisualizeAssetToolTip)
					.OnAssetToolTipClosing(OnAssetToolTipClosing)
					.ShowAssetAccessSpecifier(bShowAssetAccessSpecifier)
			);
	}
}

TSharedRef<ITableRow> SAssetView::MakeTileViewWidget(TSharedPtr<FAssetViewItem> AssetItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	if ( !ensure(AssetItem.IsValid()) )
	{
		return SNew( STableRow<TSharedPtr<FAssetViewItem>>, OwnerTable );
	}

	VisibleItems.Add(AssetItem);
	bPendingUpdateThumbnails = true;

	if (AssetItem->IsFolder())
	{
		TSharedPtr< STableRow<TSharedPtr<FAssetViewItem>> > TableRowWidget;
		SAssignNew( TableRowWidget, STableRow<TSharedPtr<FAssetViewItem>>, OwnerTable )
			.Style(UE::ContentBrowser::Private::FContentBrowserStyle::Get(), "ContentBrowser.AssetListView.TileTableRow" )
			.Cursor( bAllowDragging ? EMouseCursor::GrabHand : EMouseCursor::Default )
			.OnDragDetected( this, &SAssetView::OnDraggingAssetItem );

		TSharedRef<SAssetTileItem> Item =
			SNew(SAssetTileItem)
			.AssetItem(AssetItem)
			.CurrentThumbnailSize(this, &SAssetView::GetThumbnailSize)
			.ThumbnailDimension(this, &SAssetView::GetTileViewThumbnailDimension)
			.ThumbnailPadding((float)TileViewThumbnailPadding)
			.ItemWidth(this, &SAssetView::GetTileViewItemWidth)
			.OnRenameBegin(this, &SAssetView::AssetRenameBegin)
			.OnRenameCommit(this, &SAssetView::AssetRenameCommit)
			.OnVerifyRenameCommit(this, &SAssetView::AssetVerifyRenameCommit)
			.OnItemDestroyed(this, &SAssetView::AssetItemWidgetDestroyed)
			.ShouldAllowToolTip(this, &SAssetView::ShouldAllowToolTips)
			.HighlightText( HighlightedText )
			.IsSelected(FIsSelected::CreateSP(TableRowWidget.Get(), &STableRow<TSharedPtr<FAssetViewItem>>::IsSelected))
			.IsSelectedExclusively(FIsSelected::CreateSP(TableRowWidget.Get(), &STableRow<TSharedPtr<FAssetViewItem>>::IsSelectedExclusively))
			.AddMetaData<FTagMetaData>(AssetItem->GetItem().GetItemName())
			.ShowAssetAccessSpecifier(bShowAssetAccessSpecifier);

		TableRowWidget->SetContent(Item);

		return TableRowWidget.ToSharedRef();
	}
	else
	{
		TSharedPtr<FAssetThumbnail>& AssetThumbnail = RelevantThumbnails.FindOrAdd(AssetItem);
		if (!AssetThumbnail)
		{
			AssetThumbnail = MakeShared<FAssetThumbnail>(FAssetData(), CurrentThumbnailSize, CurrentThumbnailSize, AssetThumbnailPool);
			AssetItem->GetItem().UpdateThumbnail(*AssetThumbnail);
			AssetThumbnail->GetViewportRenderTargetTexture(); // Access the texture once to trigger it to render
		}

		TSharedPtr< STableRow<TSharedPtr<FAssetViewItem>> > TableRowWidget;
		SAssignNew( TableRowWidget, STableRow<TSharedPtr<FAssetViewItem>>, OwnerTable )
		.Style(UE::ContentBrowser::Private::FContentBrowserStyle::Get(), "ContentBrowser.AssetListView.TileTableRow")
		.Cursor( bAllowDragging ? EMouseCursor::GrabHand : EMouseCursor::Default )
		.OnDragDetected( this, &SAssetView::OnDraggingAssetItem );

		TSharedRef<SAssetTileItem> Item =
			SNew(SAssetTileItem)
			.AssetThumbnail(AssetThumbnail)
			.AssetItem(AssetItem)
			.ThumbnailPadding((float)TileViewThumbnailPadding)
			.CurrentThumbnailSize(this, &SAssetView::GetThumbnailSize)
			.ThumbnailDimension(this, &SAssetView::GetTileViewThumbnailDimension)
			.ItemWidth(this, &SAssetView::GetTileViewItemWidth)
			.OnRenameBegin(this, &SAssetView::AssetRenameBegin)
			.OnRenameCommit(this, &SAssetView::AssetRenameCommit)
			.OnVerifyRenameCommit(this, &SAssetView::AssetVerifyRenameCommit)
			.OnItemDestroyed(this, &SAssetView::AssetItemWidgetDestroyed)
			.ShouldAllowToolTip(this, &SAssetView::ShouldAllowToolTips)
			.HighlightText( HighlightedText )
			.ThumbnailEditMode(this, &SAssetView::IsThumbnailEditMode)
			.ThumbnailLabel( ThumbnailLabel )
			.ThumbnailHintColorAndOpacity( this, &SAssetView::GetThumbnailHintColorAndOpacity )
			.AllowThumbnailHintLabel( AllowThumbnailHintLabel )
			.IsSelected(FIsSelected::CreateSP(TableRowWidget.Get(), &STableRow<TSharedPtr<FAssetViewItem>>::IsSelected))
			.IsSelectedExclusively( FIsSelected::CreateSP(TableRowWidget.Get(), &STableRow<TSharedPtr<FAssetViewItem>>::IsSelectedExclusively) )
			.OnIsAssetValidForCustomToolTip(OnIsAssetValidForCustomToolTip)
			.OnGetCustomAssetToolTip(OnGetCustomAssetToolTip)
			.OnVisualizeAssetToolTip( OnVisualizeAssetToolTip )
			.OnAssetToolTipClosing( OnAssetToolTipClosing )
			.ShowType(bShowTypeInTileView)
			.ShowAssetAccessSpecifier(bShowAssetAccessSpecifier)
			.AddMetaData<FTagMetaData>(AssetItem->GetItem().GetItemName());
		
		TableRowWidget->SetContent(Item);

		return TableRowWidget.ToSharedRef();
	}
}

TSharedRef<ITableRow> SAssetView::MakeColumnViewWidget(TSharedPtr<FAssetViewItem> AssetItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	if ( !ensure(AssetItem.IsValid()) )
	{
		return SNew( STableRow<TSharedPtr<FAssetViewItem>>, OwnerTable )
			.Style(UE::ContentBrowser::Private::FContentBrowserStyle::Get(), "ContentBrowser.AssetListView.ColumnListTableRow");
	}

	// Update the cached custom data
	AssetItem->CacheCustomColumns(CustomColumns, /* bUpdateSortData */ false, /* bUpdateDisplayText */ true, /* bUpdateExisting */ false);
	
	return
		SNew( SAssetColumnViewRow, OwnerTable )
		.OnDragDetected( this, &SAssetView::OnDraggingAssetItem )
		.Cursor( bAllowDragging ? EMouseCursor::GrabHand : EMouseCursor::Default )
		.AssetColumnItem(
			SNew(SAssetColumnItem)
				.AssetItem(AssetItem)
				.OnRenameBegin(this, &SAssetView::AssetRenameBegin)
				.OnRenameCommit(this, &SAssetView::AssetRenameCommit)
				.OnVerifyRenameCommit(this, &SAssetView::AssetVerifyRenameCommit)
				.OnItemDestroyed(this, &SAssetView::AssetItemWidgetDestroyed)
				.HighlightText( HighlightedText )
				.OnIsAssetValidForCustomToolTip(OnIsAssetValidForCustomToolTip)
				.OnGetCustomAssetToolTip(OnGetCustomAssetToolTip)
				.OnVisualizeAssetToolTip( OnVisualizeAssetToolTip )
				.OnAssetToolTipClosing( OnAssetToolTipClosing )
				.ShowAssetAccessSpecifier(bShowAssetAccessSpecifier)
		);
}


void SAssetView::AssetItemWidgetDestroyed(const TSharedPtr<FAssetViewItem>& Item)
{
	if(RenamingAsset.Pin().Get() == Item.Get())
	{
		CancelPendingRename();
	}

	if ( VisibleItems.Remove(Item) != INDEX_NONE )
	{
		bPendingUpdateThumbnails = true;
	}
}

void SAssetView::UpdateThumbnails()
{
	int32 MinItemIdx = INDEX_NONE;
	int32 MaxItemIdx = INDEX_NONE;
	int32 MinVisibleItemIdx = INDEX_NONE;
	int32 MaxVisibleItemIdx = INDEX_NONE;

	const int32 HalfNumOffscreenThumbnails = NumOffscreenThumbnails / 2;
	for ( auto ItemIt = VisibleItems.CreateConstIterator(); ItemIt; ++ItemIt )
	{
		int32 ItemIdx = FilteredAssetItems.Find(*ItemIt);
		if ( ItemIdx != INDEX_NONE )
		{
			const int32 ItemIdxLow = FMath::Max<int32>(0, ItemIdx - HalfNumOffscreenThumbnails);
			const int32 ItemIdxHigh = FMath::Min<int32>(FilteredAssetItems.Num() - 1, ItemIdx + HalfNumOffscreenThumbnails);
			if ( MinItemIdx == INDEX_NONE || ItemIdxLow < MinItemIdx )
			{
				MinItemIdx = ItemIdxLow;
			}
			if ( MaxItemIdx == INDEX_NONE || ItemIdxHigh > MaxItemIdx )
			{
				MaxItemIdx = ItemIdxHigh;
			}
			if ( MinVisibleItemIdx == INDEX_NONE || ItemIdx < MinVisibleItemIdx )
			{
				MinVisibleItemIdx = ItemIdx;
			}
			if ( MaxVisibleItemIdx == INDEX_NONE || ItemIdx > MaxVisibleItemIdx )
			{
				MaxVisibleItemIdx = ItemIdx;
			}
		}
	}

	if ( MinItemIdx != INDEX_NONE && MaxItemIdx != INDEX_NONE && MinVisibleItemIdx != INDEX_NONE && MaxVisibleItemIdx != INDEX_NONE )
	{
		// We have a new min and a new max, compare it to the old min and max so we can create new thumbnails
		// when appropriate and remove old thumbnails that are far away from the view area.
		TMap< TSharedPtr<FAssetViewItem>, TSharedPtr<FAssetThumbnail> > NewRelevantThumbnails;

		// Operate on offscreen items that are furthest away from the visible items first since the thumbnail pool processes render requests in a LIFO order.
		while (MinItemIdx < MinVisibleItemIdx || MaxItemIdx > MaxVisibleItemIdx)
		{
			const int32 LowEndDistance = MinVisibleItemIdx - MinItemIdx;
			const int32 HighEndDistance = MaxItemIdx - MaxVisibleItemIdx;

			if ( HighEndDistance > LowEndDistance )
			{
				if(FilteredAssetItems.IsValidIndex(MaxItemIdx) && FilteredAssetItems[MaxItemIdx]->IsFile())
				{
					AddItemToNewThumbnailRelevancyMap(FilteredAssetItems[MaxItemIdx], NewRelevantThumbnails);
				}
				MaxItemIdx--;
			}
			else
			{
				if(FilteredAssetItems.IsValidIndex(MinItemIdx) && FilteredAssetItems[MinItemIdx]->IsFile())
				{
					AddItemToNewThumbnailRelevancyMap(FilteredAssetItems[MinItemIdx], NewRelevantThumbnails);
				}
				MinItemIdx++;
			}
		}

		// Now operate on VISIBLE items then prioritize them so they are rendered first
		TArray< TSharedPtr<FAssetThumbnail> > ThumbnailsToPrioritize;
		for ( int32 ItemIdx = MinVisibleItemIdx; ItemIdx <= MaxVisibleItemIdx; ++ItemIdx )
		{
			if(FilteredAssetItems.IsValidIndex(ItemIdx) && FilteredAssetItems[ItemIdx]->IsFile())
			{
				TSharedPtr<FAssetThumbnail> Thumbnail = AddItemToNewThumbnailRelevancyMap( FilteredAssetItems[ItemIdx], NewRelevantThumbnails );
				if ( Thumbnail.IsValid() )
				{
					ThumbnailsToPrioritize.Add(Thumbnail);
				}
			}
		}

		// Now prioritize all thumbnails there were in the visible range
		if ( ThumbnailsToPrioritize.Num() > 0 )
		{
			AssetThumbnailPool->PrioritizeThumbnails(ThumbnailsToPrioritize, CurrentThumbnailSize, CurrentThumbnailSize);
		}

		// Assign the new map of relevant thumbnails. This will remove any entries that were no longer relevant.
		RelevantThumbnails = NewRelevantThumbnails;
	}
}

TSharedPtr<FAssetThumbnail> SAssetView::AddItemToNewThumbnailRelevancyMap(const TSharedPtr<FAssetViewItem>& Item, TMap< TSharedPtr<FAssetViewItem>, TSharedPtr<FAssetThumbnail> >& NewRelevantThumbnails)
{
	checkf(Item->IsFile(), TEXT("Only files can have thumbnails!"));

	TSharedPtr<FAssetThumbnail> Thumbnail = RelevantThumbnails.FindRef(Item);
	if (!Thumbnail)
	{
		if (!ensure(CurrentThumbnailSize > 0 && CurrentThumbnailSize <= MAX_THUMBNAIL_SIZE))
		{
			// Thumbnail size must be in a sane range
			CurrentThumbnailSize = 64;
		}

		// The thumbnail newly relevant, create a new thumbnail
		int32 ThumbnailResolution = UE::Editor::ContentBrowser::IsNewStyleEnabled()
			? CurrentThumbnailSize
			: FMath::TruncToInt((float)CurrentThumbnailSize * MaxThumbnailScale);

		Thumbnail = MakeShared<FAssetThumbnail>(FAssetData(), ThumbnailResolution, ThumbnailResolution, AssetThumbnailPool);
		Item->GetItem().UpdateThumbnail(*Thumbnail);
		Thumbnail->GetViewportRenderTargetTexture(); // Access the texture once to trigger it to render
	}

	if (Thumbnail)
	{
		NewRelevantThumbnails.Add(Item, Thumbnail);
	}

	return Thumbnail;
}

void SAssetView::AssetSelectionChanged( TSharedPtr< FAssetViewItem > AssetItem, ESelectInfo::Type SelectInfo )
{
	if (!bBulkSelecting)
	{
		if (AssetItem)
		{
			OnItemSelectionChanged.ExecuteIfBound(AssetItem->GetItem(), SelectInfo);
		}
		else
		{
			OnItemSelectionChanged.ExecuteIfBound(FContentBrowserItem(), SelectInfo);
		}
	}
}

void SAssetView::ItemScrolledIntoView(TSharedPtr<FAssetViewItem> AssetItem, const TSharedPtr<ITableRow>& Widget )
{
	if (AssetItem == AwaitingScrollIntoViewForRename)
	{
		AwaitingScrollIntoViewForRename.Reset();

		// Make sure we have window focus to avoid the inline text editor from canceling itself if we try to click on it
		// This can happen if creating an asset opens an intermediary window which steals our focus, 
		// eg, the blueprint and slate widget style class windows (TTP# 314240)
		TSharedPtr<SWindow> OwnerWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		if (OwnerWindow.IsValid())
		{
			OwnerWindow->BringToFront();
		}

		AwaitingRename = AssetItem;
	}
}

TSharedPtr<SWidget> SAssetView::OnGetContextMenuContent()
{
	if (CanOpenContextMenu())
	{
		CancelPendingRename();

		OnInteractDuringFiltering();
		const TArray<FContentBrowserItem> SelectedItems = GetSelectedItems();
		return OnGetItemContextMenu.Execute(SelectedItems);
	}

	return nullptr;
}

bool SAssetView::CanOpenContextMenu() const
{
	if (!OnGetItemContextMenu.IsBound())
	{
		// You can only a summon a context menu if one is set up
		return false;
	}

	if (IsThumbnailEditMode())
	{
		// You can not summon a context menu for assets when in thumbnail edit mode because right clicking may happen inadvertently while adjusting thumbnails.
		return false;
	}

	TArray<TSharedPtr<FAssetViewItem>> SelectedItems = GetSelectedViewItems();

	// Detect if at least one temporary item was selected. If there is only a temporary item selected, then deny the context menu.
	int32 NumTemporaryItemsSelected = 0;
	int32 NumCollectionFoldersSelected = 0;
	for (const TSharedPtr<FAssetViewItem>& Item : SelectedItems)
	{
		if (Item->IsTemporary())
		{
			++NumTemporaryItemsSelected;
		}

		if (Item->IsFolder() && EnumHasAnyFlags(Item->GetItem().GetItemCategory(), EContentBrowserItemFlags::Category_Collection))
		{
			++NumCollectionFoldersSelected;
		}
	}

	// If there are only a temporary items selected, deny the context menu
	if (SelectedItems.Num() > 0 && SelectedItems.Num() == NumTemporaryItemsSelected)
	{
		return false;
	}

	// If there are any collection folders selected, deny the context menu
	if (NumCollectionFoldersSelected > 0)
	{
		return false;
	}

	return true;
}

void SAssetView::OnListMouseButtonDoubleClick(TSharedPtr<FAssetViewItem> AssetItem)
{
	if ( !ensure(AssetItem.IsValid()) )
	{
		return;
	}

	if ( IsThumbnailEditMode() )
	{
		// You can not activate assets when in thumbnail edit mode because double clicking may happen inadvertently while adjusting thumbnails.
		return;
	}

	if ( AssetItem->IsTemporary() )
	{
		// You may not activate temporary items, they are just for display.
		return;
	}

	if (OnItemsActivated.IsBound())
	{
		OnInteractDuringFiltering();
		OnItemsActivated.Execute(MakeArrayView(&AssetItem->GetItem(), 1), EAssetTypeActivationMethod::DoubleClicked);
	}
}

FReply SAssetView::OnDraggingAssetItem( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (bAllowDragging)
	{
		OnInteractDuringFiltering();
		// Use the custom drag handler?
		if (FEditorDelegates::OnAssetDragStarted.IsBound())
		{
			TArray<FAssetData> SelectedAssets = GetSelectedAssets();
			SelectedAssets.RemoveAll([](const FAssetData& InAssetData)
			{
				return InAssetData.IsRedirector();
			});

			if (SelectedAssets.Num() > 0)
			{
				FEditorDelegates::OnAssetDragStarted.Broadcast(SelectedAssets, nullptr);
				return FReply::Handled();
			}
		}

		// Use the standard drag handler?
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			TArray<FContentBrowserItem> SelectedItems = GetSelectedItems();
			SelectedItems.RemoveAll([](const FContentBrowserItem& InItem)
			{
				return InItem.IsFolder() && EnumHasAnyFlags(InItem.GetItemCategory(), EContentBrowserItemFlags::Category_Collection);
			});

			if (TSharedPtr<FDragDropOperation> DragDropOp = DragDropHandler::CreateDragOperation(SelectedItems))
			{
				OnItemsDragged.ExecuteIfBound(SelectedItems);
				return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
			}
		}
	}

	return FReply::Unhandled();
}

bool SAssetView::AssetVerifyRenameCommit(const TSharedPtr<FAssetViewItem>& Item, const FText& NewName, const FSlateRect& MessageAnchor, FText& OutErrorMessage)
{
	const FString& NewItemName = NewName.ToString();

	if (DeferredItemToCreate.IsValid() && DeferredItemToCreate->bWasAddedToView)
	{
		checkf(FContentBrowserItemKey(Item->GetItem()) == FContentBrowserItemKey(DeferredItemToCreate->ItemContext.GetItem()), TEXT("DeferredItemToCreate was still set when attempting to rename a different item!"));

		return DeferredItemToCreate->ItemContext.ValidateItem(NewItemName, &OutErrorMessage);
	}
	else if (!Item->GetItem().GetItemName().ToString().Equals(NewItemName))
	{
		UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

		return Item->GetItem().CanRename(&NewItemName, ContentBrowserData ->CreateHideFolderIfEmptyFilter().Get(), &OutErrorMessage);
	}

	return true;
}

void SAssetView::AssetRenameBegin(const TSharedPtr<FAssetViewItem>& Item, const FString& NewName, const FSlateRect& MessageAnchor)
{
	check(!RenamingAsset.IsValid());
	RenamingAsset = Item;

	OnInteractDuringFiltering();

	if (DeferredItemToCreate.IsValid())
	{
		UE_LOGF(LogContentBrowser, Log, "Renaming the item being created (Deferred Item: %ls).", *Item->GetItem().GetItemName().ToString());
	}
}

void SAssetView::AssetRenameCommit(const TSharedPtr<FAssetViewItem>& Item, const FString& NewName, const FSlateRect& MessageAnchor, const ETextCommit::Type CommitType)
{
	FText ErrorMessage;
	TSharedPtr<FAssetViewItem> UpdatedItem;

	UE_LOGF(LogContentBrowser, Log, "Attempting asset rename: %ls -> %ls", *Item->GetItem().GetItemName().ToString(), *NewName);

	if (DeferredItemToCreate.IsValid() && DeferredItemToCreate->bWasAddedToView)
	{
		const bool bFinalize = CommitType != ETextCommit::OnCleared; // Clearing the rename box on a newly created item cancels the entire creation process

		FContentBrowserItem NewItem = EndCreateDeferredItem(Item, NewName, bFinalize, ErrorMessage);
		if (NewItem.IsValid())
		{
			// Add result to view
			UpdatedItem = Items->CreateItemFromUser(MoveTemp(NewItem), FilteredAssetItems);
		}
	}
	else if (CommitType != ETextCommit::OnCleared && !Item->GetItem().GetItemName().ToString().Equals(NewName))
	{
		UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
		FScopedSuppressContentBrowserDataTick TickSuppression(ContentBrowserData);

		FContentBrowserItem NewItem;
		if (Item->GetItem().CanRename(&NewName, ContentBrowserData->CreateHideFolderIfEmptyFilter().Get(), &ErrorMessage) && Item->GetItem().Rename(NewName, &NewItem))
		{
			// Add result to view (the old item will be removed via the notifications, as not all data sources may have been able to perform the rename)
			UpdatedItem = Items->CreateItemFromUser(MoveTemp(NewItem), FilteredAssetItems);
		}
	}
	
	if (UpdatedItem)
	{
		// Sort in the new item
		bPendingSortFilteredItems = true;

		if (UpdatedItem->IsFile())
		{
			// Refresh the thumbnail
			if (TSharedPtr<FAssetThumbnail> AssetThumbnail = RelevantThumbnails.FindRef(Item))
			{
				if (UpdatedItem != Item)
				{
					// This item was newly created - move the thumbnail over from the temporary item
					RelevantThumbnails.Remove(Item);
					RelevantThumbnails.Add(UpdatedItem, AssetThumbnail);
					UpdatedItem->GetItem().UpdateThumbnail(*AssetThumbnail);
				}
				if (AssetThumbnail->GetAssetData().IsValid())
				{
					AssetThumbnailPool->RefreshThumbnail(AssetThumbnail);
				}
			}
		}
		
		// Sync the view
		{
			TArray<FContentBrowserItem> ItemsToSync;
			ItemsToSync.Add(UpdatedItem->GetItem());

			if (OnItemRenameCommitted.IsBound() && !bUserSearching)
			{
				// If our parent wants to potentially handle the sync, let it, but only if we're not currently searching (or it would cancel the search)
				OnItemRenameCommitted.Execute(ItemsToSync);
			}
			else
			{
				// Otherwise, sync just the view
				SyncToItems(ItemsToSync);
			}
		}
	}
	else if (!ErrorMessage.IsEmpty())
	{
		// Prompt the user with the reason the rename/creation failed
		ContentBrowserUtils::DisplayMessage(ErrorMessage, MessageAnchor, SharedThis(this), ContentBrowserUtils::EDisplayMessageType::Error);
	}

	RenamingAsset.Reset();
}

bool SAssetView::IsRenamingAsset() const
{
	return RenamingAsset.IsValid();
}

bool SAssetView::ShouldAllowToolTips() const
{
	bool bIsRightClickScrolling = false;
	switch( CurrentViewType )
	{
		case EAssetViewType::List:
			bIsRightClickScrolling = ListView->IsRightClickScrolling();
			break;

		case EAssetViewType::Tile:
			bIsRightClickScrolling = TileView->IsRightClickScrolling();
			break;

		case EAssetViewType::Column:
			bIsRightClickScrolling = ColumnView->IsRightClickScrolling();
			break;
		
		case EAssetViewType::Custom:
			bIsRightClickScrolling = ViewExtender->IsRightClickScrolling();
			break;

		default:
			bIsRightClickScrolling = false;
			break;
	}

	return !bIsRightClickScrolling && !IsThumbnailEditMode() && !IsRenamingAsset();
}

bool SAssetView::IsThumbnailEditMode() const
{
	return IsThumbnailEditModeAllowed() && bThumbnailEditMode;
}

bool SAssetView::IsThumbnailEditModeAllowed() const
{
	return bAllowThumbnailEditMode && (UE::Editor::ContentBrowser::IsNewStyleEnabled() || GetCurrentViewType() != EAssetViewType::Column);
}

FReply SAssetView::EndThumbnailEditModeClicked()
{
	bThumbnailEditMode = false;

	return FReply::Handled();
}

FText SAssetView::GetAssetCountText() const
{
	const int32 NumAssets = FilteredAssetItems.Num();
	const int32 NumSelectedAssets = GetSelectedViewItems().Num();

	FText AssetCount = FText::GetEmpty();
	if ( NumSelectedAssets == 0 )
	{
		if ( NumAssets == 1 )
		{
			AssetCount = LOCTEXT("AssetCountLabelSingular", "1 item");
		}
		else
		{
			AssetCount = FText::Format( LOCTEXT("AssetCountLabelPlural", "{0} items"), FText::AsNumber(NumAssets) );
		}
	}
	else
	{
		if ( NumAssets == 1 )
		{
			AssetCount = FText::Format( LOCTEXT("AssetCountLabelSingularPlusSelection", "1 item ({0} selected)"), FText::AsNumber(NumSelectedAssets) );
		}
		else
		{
			AssetCount = FText::Format( LOCTEXT("AssetCountLabelPluralPlusSelection", "{0} items ({1} selected)"), FText::AsNumber(NumAssets), FText::AsNumber(NumSelectedAssets) );
		}
	}

	return AssetCount;
}

EVisibility SAssetView::GetMajorityAssetTypeTextVisibility() const
{
	return (CurrentViewType == EAssetViewType::Column) ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SAssetView::GetMajorityAssetTypeText() const
{
	if ((CurrentViewType != EAssetViewType::Column) || MajorityAssetType.IsNone() || MajorityAssetTypeText.IsEmpty())
	{
		return FText::GetEmpty();
	}

	const uint32 NumVisibleColumns = IntCastChecked<uint32>(Algo::CountBy(ColumnView->GetHeaderRow()->GetColumns(), /*InValue = */true, &SHeaderRow::FColumn::bIsVisible));

	return FText::Format(LOCTEXT("MajorityAssetTypeText", "Showing {0} {0}|plural(one=column,other=columns) for asset type \"{1}\""), 
		FText::AsNumber(NumVisibleColumns), MajorityAssetTypeText);
}

FText SAssetView::GetMajorityAssetTypeToolTipText() const
{
	if ((CurrentViewType != EAssetViewType::Column) || MajorityAssetType.IsNone() || MajorityAssetTypeText.IsEmpty())
	{
		return FText::GetEmpty();
	}

	return LOCTEXT("MajorityAssetTypeTooltip", "This view shows columns for the asset type with the most occurrences. \n\r"
		"\n\r"
		"If you need to see data for a different asset type, try filtering the view to show only assets of that type.");
}

EVisibility SAssetView::GetEditModeLabelVisibility() const
{
	return IsThumbnailEditMode() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SAssetView::GetListViewVisibility() const
{
	return GetCurrentViewType() == EAssetViewType::List ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SAssetView::GetTileViewVisibility() const
{
	return GetCurrentViewType() == EAssetViewType::Tile ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SAssetView::GetColumnViewVisibility() const
{
	return GetCurrentViewType() == EAssetViewType::Column ? EVisibility::Visible : EVisibility::Collapsed;
}

void SAssetView::ToggleTooltipExpandedState()
{
	bool bNewState = !GetDefault<UContentBrowserSettings>()->GetAlwaysExpandTooltips();

	GetMutableDefault<UContentBrowserSettings>()->SetAlwaysExpandTooltips(bNewState);
	GetMutableDefault<UContentBrowserSettings>()->PostEditChange();
}

bool SAssetView::IsTooltipExpandedByDefault()
{
	return GetDefault<UContentBrowserSettings>()->GetAlwaysExpandTooltips();
}

void SAssetView::ToggleThumbnailEditMode()
{
	bThumbnailEditMode = !bThumbnailEditMode;
}

void SAssetView::OnThumbnailSizeChanged(EThumbnailSize NewThumbnailSize)
{
	ThumbnailSizes[CurrentViewType] = NewThumbnailSize;
	UpdateThumbnailSizeValue();

	if (FAssetViewInstanceConfig* Config = GetAssetViewConfig())
	{
		Config->ThumbnailSize = (uint8) NewThumbnailSize;
		UAssetViewConfig::Get()->SaveEditorConfig();
	}

	RefreshList();
}

bool SAssetView::IsThumbnailSizeChecked(EThumbnailSize InThumbnailSize) const
{
	return ThumbnailSizes[CurrentViewType] == InThumbnailSize;
}

float SAssetView::GetThumbnailScale() const
{
	float BaseScale;
	switch (ThumbnailSizes[CurrentViewType])
	{
	case EThumbnailSize::Tiny:
		BaseScale = 0.1f;
		break;
	case EThumbnailSize::Small:
		BaseScale = 0.25f;
		break;
	case EThumbnailSize::Medium:
		BaseScale = 0.5f;
		break;
	case EThumbnailSize::Large:
		BaseScale = 0.75f;
		break;
	case EThumbnailSize::XLarge:
		BaseScale = 0.9f;
		break;
	case EThumbnailSize::Huge:
		BaseScale = 1.0f;
		break;
	default:
		BaseScale = 0.5f;
		break;
	}

	return BaseScale * GetTickSpaceGeometry().Scale;
}

float SAssetView::GetThumbnailSizeValue() const
{
	return FMath::Lerp(MinThumbnailSize, MaxThumbnailSize, ZoomScale);
}

void SAssetView::UpdateThumbnailSizeValue()
{
	switch (ThumbnailSizes[CurrentViewType])
	{
	case EThumbnailSize::Tiny:
		MinThumbnailSize = 64.f;
		MaxThumbnailSize = 80.f;
		ListViewItemHeight = 16.f;
		break;
	case EThumbnailSize::Small:
		MinThumbnailSize = 80.f;
		MaxThumbnailSize = 96.f;
		ListViewItemHeight = 24.f;
		break;
	case EThumbnailSize::Medium:
		MinThumbnailSize = 96.f;
		MaxThumbnailSize = 112.f;
		ListViewItemHeight = 32.f;
		break;
	case EThumbnailSize::Large:
		MinThumbnailSize = 112.f;
		MaxThumbnailSize = 128.f;
		ListViewItemHeight = 48.f;
		break;
	case EThumbnailSize::XLarge:
		MinThumbnailSize = 128.f;
		MaxThumbnailSize = 136.f;
		ListViewItemHeight = 64.f;
		break;
	case EThumbnailSize::Huge:
		MinThumbnailSize = 136.f;
		MaxThumbnailSize = 160.f;
		ListViewItemHeight = 80.f;
		break;
	default:
		MinThumbnailSize = 64.f;
		MaxThumbnailSize = 80.f;
		ListViewItemHeight = 22.f;
		break;
	}
}

bool SAssetView::IsThumbnailScalingAllowed() const
{
	return (UE::Editor::ContentBrowser::IsNewStyleEnabled() || GetCurrentViewType() != EAssetViewType::Column) && GetCurrentViewType() != EAssetViewType::Custom;
}

float SAssetView::GetTileViewTypeNameHeight() const
{
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		if (ThumbnailSizes[CurrentViewType] == EThumbnailSize::Tiny)
		{
			return 0;
		}
		return 67.f;
	}
	else
	{
		float TypeNameHeight = 0;
		if (bShowTypeInTileView)
		{
			TypeNameHeight = 50;
		}
		else
		{
			if (ThumbnailSizes[CurrentViewType] == EThumbnailSize::Small)
			{
				TypeNameHeight = 25;
			}
			else if (ThumbnailSizes[CurrentViewType] == EThumbnailSize::Medium)
			{
				TypeNameHeight = -5;
			}
			else if (ThumbnailSizes[CurrentViewType] > EThumbnailSize::Medium)
			{
				TypeNameHeight = -25;
			}
			else
			{
				TypeNameHeight = -40;
			}
		}
		return TypeNameHeight;
	}
}

float SAssetView::GetSourceControlIconHeight() const
{
	return (float)(ThumbnailSizes[CurrentViewType] != EThumbnailSize::Tiny && ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable() && !bShowTypeInTileView ? 17.0 : 0.0);
}

float SAssetView::GetListViewItemHeight() const
{
	return UE::Editor::ContentBrowser::IsNewStyleEnabled()
		? ListViewItemHeight
		: (float)(ListViewThumbnailSize + ListViewThumbnailPadding * 2) * FMath::Lerp(MinThumbnailScale, MaxThumbnailScale, GetThumbnailScale());
}

FMargin SAssetView::GetListViewItemPadding() const
{
	return UE::Editor::ContentBrowser::IsNewStyleEnabled()
		? ThumbnailSizes[CurrentViewType] == EThumbnailSize::Tiny ? FMargin(0.f) : FMargin(0.f, (float)ListViewThumbnailPadding)
		: FMargin(0.f);
}

float SAssetView::GetTileViewItemHeight() const
{
	return UE::Editor::ContentBrowser::IsNewStyleEnabled()
		? GetTileViewItemBaseWidth() + GetTileViewTypeNameHeight() + TileViewHeightPadding
		: (((float)TileViewNameHeight + GetTileViewTypeNameHeight()) * FMath::Lerp(MinThumbnailScale, MaxThumbnailScale, GetThumbnailScale())) + GetTileViewItemBaseHeight() * FillScale + GetSourceControlIconHeight();
}

float SAssetView::GetTileViewItemBaseHeight() const
{

	return UE::Editor::ContentBrowser::IsNewStyleEnabled()
		? GetTileViewItemBaseWidth()
		: (float)(TileViewThumbnailSize + TileViewThumbnailPadding * 2) * FMath::Lerp(MinThumbnailScale, MaxThumbnailScale, GetThumbnailScale());
}

float SAssetView::GetTileViewItemWidth() const
{
	return UE::Editor::ContentBrowser::IsNewStyleEnabled()
		? GetTileViewItemBaseWidth() + TileViewWidthPadding
		: GetTileViewItemBaseWidth() * FillScale;
}

float SAssetView::GetTileViewThumbnailDimension() const
{
	return GetThumbnailSizeValue();
}

float SAssetView::GetTileViewItemBaseWidth() const
{
	return UE::Editor::ContentBrowser::IsNewStyleEnabled()
		? GetTileViewThumbnailDimension()
		: (float)( TileViewThumbnailSize + TileViewThumbnailPadding * 2 ) * FMath::Lerp( MinThumbnailScale, MaxThumbnailScale, GetThumbnailScale() );
}

EColumnSortMode::Type SAssetView::GetColumnSortMode(const FName ColumnId) const
{
	for (int32 PriorityIdx = 0; PriorityIdx < EColumnSortPriority::Max; PriorityIdx++)
	{
		const EColumnSortPriority::Type SortPriority = static_cast<EColumnSortPriority::Type>(PriorityIdx);
		if (ColumnId == SortManager->GetSortColumnId(SortPriority))
		{
			return SortManager->GetSortMode(SortPriority);
		}
	}

	static constexpr EColumnSortMode::Type DefaultSortMode = EColumnSortMode::None;
	return DefaultSortMode;
}

EColumnSortPriority::Type SAssetView::GetColumnSortPriority(const FName ColumnId) const
{
	for (int32 PriorityIdx = 0; PriorityIdx < EColumnSortPriority::Max; PriorityIdx++)
	{
		const EColumnSortPriority::Type SortPriority = static_cast<EColumnSortPriority::Type>(PriorityIdx);
		if (ColumnId == SortManager->GetSortColumnId(SortPriority))
		{
			return SortPriority;
		}
	}

	static constexpr EColumnSortPriority::Type DefaultSortPriority = EColumnSortPriority::Primary;
	return DefaultSortPriority;
}

void SAssetView::OnSortColumnHeader(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode)
{
	SortManager->SetSortColumnId(SortPriority, ColumnId);
	SortManager->SetSortMode(SortPriority, NewSortMode);
	SortList();
}

void SAssetView::SetSortParameters(
	const TOptional<const EColumnSortPriority::Type>& InSortPriority,
	const TOptional<const FName>& InColumnId,
	const TOptional<const EColumnSortMode::Type>& InNewSortMode)
{
	FName ColumnId = InColumnId.Get(NAME_None);

	static constexpr EColumnSortPriority::Type DefaultSortPriority = EColumnSortPriority::Primary;

	// Use specified priority OR default (primary)...
	EColumnSortPriority::Type SortPriority = InSortPriority.Get(DefaultSortPriority);

	// ... unless a ColumnId WAS specified and a priority was NOT
	if (InColumnId.IsSet() && !InSortPriority.IsSet())
	{
		SortPriority = GetColumnSortPriority(ColumnId);
	}

	if (!InColumnId.IsSet())
	{
		ColumnId = SortManager->GetSortColumnId(SortPriority);
	}

	// Use specified sort mode OR get from the ColumnId
	EColumnSortMode::Type DefaultSortMode = GetColumnSortMode(ColumnId);
	if (DefaultSortMode == EColumnSortMode::None)
	{
		DefaultSortMode = EColumnSortMode::Ascending;
	}
	const EColumnSortMode::Type SortMode = InNewSortMode.Get(DefaultSortMode);

	OnSortColumnHeader(SortPriority, ColumnId, SortMode);
}

EVisibility SAssetView::IsAssetShowWarningTextVisible() const
{
	const bool bTextSelectable = AssetShowWarningTextSelectable.Get(/* Default **/false);
	return (FilteredAssetItems.Num() > 0 || bQuickFrontendListRefreshRequested) ? EVisibility::Collapsed : (bTextSelectable ? EVisibility::SelfHitTestInvisible : EVisibility::HitTestInvisible);
}

FText SAssetView::GetAssetShowWarningText() const
{
	if (AssetShowWarningText.IsSet())
	{
		return AssetShowWarningText.Get();
	}
	
	if (InitialNumAmortizedTasks > 0)
	{
		return LOCTEXT("ApplyingFilter", "Applying filter...");
	}

	FText NothingToShowText, DropText;
	if (ShouldFilterRecursively())
	{
		NothingToShowText = LOCTEXT( "NothingToShowCheckFilter", "No results, check your filter." );
	}

	if (ContentSources.HasCollections() && !ContentSources.IsDynamicCollection() )
	{
		if (ContentSources.GetCollections()[0].Name.IsNone())
		{
			DropText = LOCTEXT("NoCollectionSelected", "No collection selected.");
		}
		else
		{
			DropText = LOCTEXT("DragAssetsHere", "Drag and drop assets here to add them to the collection.");
		}
	}
	else if ( OnGetItemContextMenu.IsBound() )
	{
		DropText = LOCTEXT( "DropFilesOrRightClick", "Drop files here or right click to create content." );
	}
	
	return NothingToShowText.IsEmpty() ? DropText : FText::Format(LOCTEXT("NothingToShowPattern", "{0}\n{1}"), NothingToShowText, DropText);
}

bool SAssetView::HasSingleCollectionSource() const
{
	return ContentSources.GetCollections().Num() == 1;
}

void SAssetView::SetUserSearching(bool bInSearching)
{
	bUserSearching = bInSearching;

	// If we're changing between recursive and non-recursive data, we need to fully refresh the source items
	if (ShouldFilterRecursively() != bWereItemsRecursivelyFiltered)
	{
		RequestSlowFullListRefresh();
	}
}

void SAssetView::HandleSettingChanged(FName PropertyName)
{
	if ((PropertyName == GET_MEMBER_NAME_CHECKED(UContentBrowserSettings, DisplayFolders)) ||
		(PropertyName == GET_MEMBER_NAME_CHECKED(UContentBrowserSettings, DisplayEmptyFolders)) ||
		(PropertyName == "DisplayDevelopersFolder") ||
		(PropertyName == "DisplayEngineFolder") ||
		(PropertyName == GET_MEMBER_NAME_CHECKED(UContentBrowserSettings, bDisplayContentFolderSuffix)) ||
		(PropertyName == GET_MEMBER_NAME_CHECKED(UContentBrowserSettings, bDisplayFriendlyNameForPluginFolders)) ||
		(PropertyName == NAME_None))	// @todo: Needed if PostEditChange was called manually, for now
	{
		RequestSlowFullListRefresh();
	}
}

FText SAssetView::GetQuickJumpTerm() const
{
	return FText::FromString(QuickJumpData.JumpTerm);
}

EVisibility SAssetView::IsQuickJumpVisible() const
{
	if (const UContentBrowserSettings* ContentBrowserSettings = GetDefault<UContentBrowserSettings>())
	{
		if (!ContentBrowserSettings->bDisplayQuickJumpText)
		{
			return EVisibility::Collapsed;
		}
	}

	return (QuickJumpData.JumpTerm.IsEmpty()) ? EVisibility::Collapsed : EVisibility::HitTestInvisible;
}

FSlateColor SAssetView::GetQuickJumpColor() const
{
	return FAppStyle::GetColor((QuickJumpData.bHasValidMatch) ? "InfoReporting.BackgroundColor" : "ErrorReporting.BackgroundColor");
}

void SAssetView::ResetQuickJump()
{
	QuickJumpData.JumpTerm.Empty();
	QuickJumpData.bIsJumping = false;
	QuickJumpData.bHasChangedSinceLastTick = false;
	QuickJumpData.bHasValidMatch = false;
}

FReply SAssetView::HandleQuickJumpKeyDown(const TCHAR InCharacter, const bool bIsControlDown, const bool bIsAltDown, const bool bTestOnly)
{
	// Check for special characters
	if(bIsControlDown || bIsAltDown)
	{
		return FReply::Unhandled();
	}

	// Check for invalid characters
	for(int InvalidCharIndex = 0; InvalidCharIndex < UE_ARRAY_COUNT(INVALID_OBJECTNAME_CHARACTERS) - 1; ++InvalidCharIndex)
	{
		if(InCharacter == INVALID_OBJECTNAME_CHARACTERS[InvalidCharIndex])
		{
			return FReply::Unhandled();
		}
	}

	switch(InCharacter)
	{
	// Ignore some other special characters that we don't want to be entered into the buffer
	case 0:		// Any non-character key press, e.g. f1-f12, Delete, Pause/Break, etc.
				// These should be explicitly not handled so that their input bindings are handled higher up the chain.

	case 8:		// Backspace
	case 13:	// Enter
	case 27:	// Esc
		return FReply::Unhandled();

	default:
		break;
	}

	// Any other character!
	if(!bTestOnly)
	{
		QuickJumpData.JumpTerm.AppendChar(InCharacter);
		QuickJumpData.bHasChangedSinceLastTick = true;
	}

	return FReply::Handled();
}

bool SAssetView::PerformQuickJump(const bool bWasJumping)
{
	auto JumpToNextMatch = [this](const int StartIndex, const int EndIndex) -> bool
	{
		check(StartIndex >= 0);
		check(EndIndex <= FilteredAssetItems.Num());

		for(int NewSelectedItemIndex = StartIndex; NewSelectedItemIndex < EndIndex; ++NewSelectedItemIndex)
		{
			TSharedPtr<FAssetViewItem>& NewSelectedItem = FilteredAssetItems[NewSelectedItemIndex];
			const FString& NewSelectedItemName = NewSelectedItem->GetItem().GetDisplayName().ToString();
			if(NewSelectedItemName.StartsWith(QuickJumpData.JumpTerm, ESearchCase::IgnoreCase))
			{
				ClearSelection(true);
				RequestScrollIntoView(NewSelectedItem);
				ClearSelection();
				// Consider it derived from a keypress because otherwise it won't update the navigation selector
				SetItemSelection(NewSelectedItem, true, ESelectInfo::Type::OnKeyPress);
				return true;
			}
		}

		return false;
	};

	TArray<TSharedPtr<FAssetViewItem>> SelectedItems = GetSelectedViewItems();
	TSharedPtr<FAssetViewItem> SelectedItem = (SelectedItems.Num()) ? SelectedItems[0] : nullptr;

	// If we have a selection, and we were already jumping, first check to see whether 
	// the current selection still matches the quick-jump term; if it does, we do nothing
	if(bWasJumping && SelectedItem.IsValid())
	{
		const FString& SelectedItemName = SelectedItem->GetItem().GetDisplayName().ToString();
		if(SelectedItemName.StartsWith(QuickJumpData.JumpTerm, ESearchCase::IgnoreCase))
		{
			return true;
		}
	}

	// We need to move on to the next match in FilteredAssetItems that starts with the given quick-jump term
	const int SelectedItemIndex = (SelectedItem.IsValid()) ? FilteredAssetItems.Find(SelectedItem) : INDEX_NONE;
	const int StartIndex = (SelectedItemIndex == INDEX_NONE) ? 0 : SelectedItemIndex + 1;
	
	bool ValidMatch = JumpToNextMatch(StartIndex, FilteredAssetItems.Num());
	if(!ValidMatch && StartIndex > 0)
	{
		// If we didn't find a match, we need to loop around and look again from the start (assuming we weren't already)
		return JumpToNextMatch(0, StartIndex);
	}

	return ValidMatch;
}

void SAssetView::ResetColumns()
{
	TSharedPtr<SListView<TSharedPtr<FAssetViewItem>>> ViewToUse = ColumnView;
	TArray<FString>* HiddenColumnsToUse = &HiddenColumnNames;
	TArray<FString> DefaultHiddenColumnToUse = DefaultHiddenColumnNames;
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		if (CurrentViewType == EAssetViewType::List)
		{
			ViewToUse = ListView;
			HiddenColumnsToUse = &ListHiddenColumnNames;
			DefaultHiddenColumnToUse = DefaultListHiddenColumnNames;
			// When resetting list view columns, reset also this to use the default
			bListViewColumnsManuallyChangedOnce = false;
		}
		else
		{
			// When resetting column view columns, reset also this to use the default
			bColumnViewColumnsManuallyChangedOnce = false;
		}
	}

	for (const SHeaderRow::FColumn &Column : ViewToUse->GetHeaderRow()->GetColumns())
	{
		ViewToUse->GetHeaderRow()->SetShowGeneratedColumn(Column.ColumnId, !DefaultHiddenColumnToUse.Contains(Column.ColumnId.ToString()));
	}

	// This is set after updating the column visibilties, because SetShowGeneratedColumn calls OnHiddenColumnsChanged indirectly which can mess up the list
	HiddenColumnsToUse->Reset(DefaultHiddenColumnToUse.Num());
	HiddenColumnsToUse->Append(DefaultHiddenColumnToUse);
	ViewToUse->GetHeaderRow()->RefreshColumns();
	ViewToUse->RebuildList();
}

void SAssetView::ExportColumns()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

	const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

	const FText Title = LOCTEXT("ExportToCSV", "Export columns as CSV...");
	const FString FileTypes = TEXT("Data Table CSV (*.csv)|*.csv");

	TArray<FString> OutFilenames;
	DesktopPlatform->SaveFileDialog(
		ParentWindowWindowHandle,
		Title.ToString(),
		TEXT(""),
		TEXT("Report.csv"),
		FileTypes,
		EFileDialogFlags::None,
		OutFilenames
	);

	if (OutFilenames.Num() > 0)
	{
		const TIndirectArray<SHeaderRow::FColumn>& Columns = ColumnView->GetHeaderRow()->GetColumns();

		TArray<FName> ColumnNames;
		for (const SHeaderRow::FColumn& Column : Columns)
		{
			ColumnNames.Add(Column.ColumnId);
		}

		FString SaveString;
		SortManager->ExportColumnsToCSV(FilteredAssetItems, ColumnNames, CustomColumns, SaveString);

		FFileHelper::SaveStringToFile(SaveString, *OutFilenames[0]);
	}
}

void SAssetView::OnHiddenColumnsChanged()
{
	const bool bIsUsingNewStyle = UE::Editor::ContentBrowser::IsNewStyleEnabled();

	// Early out if this is called before creation or during LoadSettings due to SetShowGeneratedColumn(due to loading config etc)
	if (bIsUsingNewStyle && bLoadingSettings)
	{
		return;
	}

	// Early out if this is called before creation (due to loading config etc)
	TSharedPtr<SListView<TSharedPtr<FAssetViewItem>>> ViewToUse = ColumnView;
	TArray<FString>* HiddenColumnsToUse = &HiddenColumnNames;
	if (bIsUsingNewStyle)
	{
		if (CurrentViewType == EAssetViewType::List)
		{
			ViewToUse = ListView;
			HiddenColumnsToUse = &ListHiddenColumnNames;
		}
	}

	if (!ViewToUse.IsValid())
	{
		return;
	}

	if (bIsUsingNewStyle)
	{
		if (CurrentViewType == EAssetViewType::List)
		{
			// Set this to true as soon as the first column is modified by the user
			bListViewColumnsManuallyChangedOnce = true;
		}
		else
		{
			// Set this to true as soon as the first column is modified by the user
			bColumnViewColumnsManuallyChangedOnce = true;
		}
	}

	// We can't directly update the hidden columns list, because some columns maybe hidden, but not created yet
	TArray<FName> NewHiddenColumns = ViewToUse->GetHeaderRow()->GetHiddenColumnIds();

	// So instead for each column that currently exists, we update its visibility state in the HiddenColumnNames array
	for (const SHeaderRow::FColumn& Column : ViewToUse->GetHeaderRow()->GetColumns())
	{
		const bool bIsColumnVisible = NewHiddenColumns.Contains(Column.ColumnId);

		if (bIsColumnVisible)
		{
			HiddenColumnsToUse->AddUnique(Column.ColumnId.ToString());
		}
		else
		{
			HiddenColumnsToUse->Remove(Column.ColumnId.ToString());
		}
	}

	if (FAssetViewInstanceConfig* Config = GetAssetViewConfig())
	{
		if (CurrentViewType == EAssetViewType::List)
		{
			Config->ListHiddenColumns.Reset();
			Algo::Transform(*HiddenColumnsToUse, Config->ListHiddenColumns, [](const FString& Str) { return FName(*Str); });
		}
		else
		{
			Config->HiddenColumns.Reset();
			Algo::Transform(*HiddenColumnsToUse, Config->HiddenColumns, [](const FString& Str) { return FName(*Str); });
		}

		UAssetViewConfig::Get()->SaveEditorConfig();
	}

	OnColumnVisibilityChanged.ExecuteIfBound(*HiddenColumnsToUse);
}

bool SAssetView::ShouldColumnGenerateWidget(const FString ColumnName) const
{
	return !HiddenColumnNames.Contains(ColumnName);
}

void SAssetView::ForceShowPluginFolder(bool bEnginePlugin)
{
	if (bEnginePlugin && !IsShowingEngineContent())
	{
		ToggleShowEngineContent();
	}

	if (!IsShowingPluginContent())
	{
		ToggleShowPluginContent();
	}
}

void SAssetView::OverrideShowEngineContent()
{
	if (!IsShowingEngineContent())
	{
		ToggleShowEngineContent();
	}

}

void SAssetView::OverrideShowDeveloperContent()
{
	if (!IsShowingDevelopersContent())
	{
		ToggleShowDevelopersContent();
	}
}

void SAssetView::OverrideShowPluginContent()
{
	if (!IsShowingPluginContent())
	{
		ToggleShowPluginContent();
	}
}

void SAssetView::OverrideShowLocalizedContent()
{
	if (!IsShowingLocalizedContent())
	{
		ToggleShowLocalizedContent();
	}
}

void SAssetView::HandleItemDataUpdated(TArrayView<const FContentBrowserItemDataUpdate> InUpdatedItems)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SAssetView::HandleItemDataUpdated);

	if (InUpdatedItems.Num() == 0)
	{
		return;
	}

	const double HandleItemDataUpdatedStartTime = FPlatformTime::Seconds();

	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

	TArray<FContentBrowserDataCompiledFilter> CompiledDataFilters;
	if (ContentSources.IsIncludingVirtualPaths())
	{
		const bool bInvalidateFilterCache = false;
		const FContentBrowserDataFilter DataFilter = CreateBackendDataFilter(bInvalidateFilterCache);

		static const FName RootPath = "/";
		const TArrayView<const FName> DataSourcePaths = ContentSources.HasVirtualPaths() ? MakeArrayView(ContentSources.GetVirtualPaths()) : MakeArrayView(&RootPath, 1);
		for (const FName& DataSourcePath : DataSourcePaths)
		{
			FContentBrowserDataCompiledFilter& CompiledDataFilter = CompiledDataFilters.AddDefaulted_GetRef();
			ContentBrowserData->CompileFilter(DataSourcePath, DataFilter, CompiledDataFilter);
		}
	}

	bool bRefreshView = false;
	TSet<TSharedPtr<FAssetViewItem>> ItemsPendingInplaceFrontendFilter;

	auto AddItem = [this, &ItemsPendingInplaceFrontendFilter](FContentBrowserItemData&& InItemData)
	{
		TSharedPtr<FAssetViewItem> ItemToUpdate = Items->UpdateData(MoveTemp(InItemData));
		// Update the custom column data if it exists
		ItemToUpdate->CacheCustomColumns(CustomColumns, /* bUpdateSortData */ true, /* bUpdateDisplayText */ true, /* bUpdateExisting */ true);
	};

	auto RemoveItem = [this, &bRefreshView, &ItemsPendingInplaceFrontendFilter](const FContentBrowserMinimalItemData& ItemDataKey)
	{
		TSharedPtr<FAssetViewItem> RemovedItem = Items->RemoveItemData(ItemDataKey);
		if (RemovedItem.IsValid())
		{
			// Need to refresh manually after removing items, as adding relies on the pending filter lists to trigger this
			bRefreshView = true;
		}
	};

	auto GetBackendFilterCompliantItem = [this, &CompiledDataFilters](const FContentBrowserItemData& InItemData, bool& bOutPassFilter)
	{
		UContentBrowserDataSource* ItemDataSource = InItemData.GetOwnerDataSource();
		FContentBrowserItemData ItemData = InItemData;
		for (const FContentBrowserDataCompiledFilter& DataFilter : CompiledDataFilters)
		{
			// We only convert the item if this is the right filter for the data source
			if (ItemDataSource->ConvertItemForFilter(ItemData, DataFilter))
			{
				bOutPassFilter = ItemDataSource->DoesItemPassFilter(ItemData, DataFilter);

				return ItemData;
			}

			if (ItemDataSource->DoesItemPassFilter(ItemData, DataFilter))
			{
				bOutPassFilter = true;
				return ItemData;
			}
		}

		bOutPassFilter = false;
		return ItemData;
	};

	// Process the main set of updates
	for (const FContentBrowserItemDataUpdate& ItemDataUpdate : InUpdatedItems)
	{
		bool bItemPassFilter = false;
		FContentBrowserItemData ItemData = GetBackendFilterCompliantItem(ItemDataUpdate.GetItemData(), bItemPassFilter);

		switch (ItemDataUpdate.GetUpdateType())
		{
		case EContentBrowserItemUpdateType::Added:
			if (bItemPassFilter)
			{
				AddItem(MoveTemp(ItemData));
			}
			break;

		case EContentBrowserItemUpdateType::Modified:
			if (bItemPassFilter)
			{
				AddItem(MoveTemp(ItemData));
			}
			else
			{
				RemoveItem(FContentBrowserMinimalItemData(ItemData));
			}
			break;

		case EContentBrowserItemUpdateType::Moved:
			{
				const FContentBrowserMinimalItemData OldItemDataKey(ItemData.GetItemType(), ItemDataUpdate.GetPreviousVirtualPath(), ItemData.GetOwnerDataSource());
				RemoveItem(OldItemDataKey);
				if (bItemPassFilter)
				{
					AddItem(MoveTemp(ItemData));
				}
				else
				{
					checkAssetList(!AvailableBackendItems.Contains(ItemDataKey));
				}
			}
			break;

		case EContentBrowserItemUpdateType::Removed:
			RemoveItem(FContentBrowserMinimalItemData(ItemData));
			break;

		case EContentBrowserItemUpdateType::Unmounted:
			RemoveItem(FContentBrowserMinimalItemData(ItemData));
			break;

		default:
			checkf(false, TEXT("Unexpected EContentBrowserItemUpdateType!"));
			break;
		}
	}

	FAssetViewFrontendFilterHelper FrontendFilterHelper(this);
	if (Items->PerformPriorityFiltering(FrontendFilterHelper, FilteredAssetItems))
	{
		bRefreshView = true;
	}

	if (bRefreshView)
	{
		RefreshList();
	}

	UE_LOGF(LogContentBrowser, VeryVerbose, "AssetView - HandleItemDataUpdated completed in %0.4f seconds for %d items (%d available items)", 
		FPlatformTime::Seconds() - HandleItemDataUpdatedStartTime, InUpdatedItems.Num(), Items->Num());
}

void SAssetView::HandleItemDataDiscoveryComplete()
{
	if (bPendingSortFilteredItems)
	{
		// If we have a sort pending, then force this to happen next frame now that discovery has finished
		LastSortTime = 0;
	}
}

void SAssetView::SetFilterBar(TSharedPtr<SFilterList> InFilterBar)
{
	FilterBar = InFilterBar;
}

void SAssetView::SetShouldFilterItem(FOnShouldFilterItem InCallback)
{
	OnShouldFilterItem = MoveTemp(InCallback);
	RequestQuickFrontendListRefresh();
}

void SAssetView::SetAssetSearchOverride(TSharedPtr<UE::Editor::ContentBrowser::Extension::ICBSearchOverrideInstance> InOverride)
{
	AssetSearchOverride = MoveTemp(InOverride);
	if (AssetSearchOverride.IsValid())
	{
		AssetSearchOverride->GetInternalApi().OnRequestSetUserSearching.BindSP(this, &SAssetView::SetUserSearching);
		AssetSearchOverride->GetInternalApi().OnRequestQuickFrontendListRefresh.BindSP(this, &SAssetView::RequestQuickFrontendListRefresh);
	}
	RequestQuickFrontendListRefresh();
}

TWeakPtr<FAssetViewSortManager> SAssetView::GetSortManager() const
{
	return SortManager;
}

void SAssetView::OnCompleteFiltering(double InAmortizeDuration)
{
	CurrentFrontendFilterTelemetry.AmortizeDuration = InAmortizeDuration;
	CurrentFrontendFilterTelemetry.bCompleted = true;
	FTelemetryRouter::Get().ProvideTelemetry(CurrentFrontendFilterTelemetry);
	CurrentFrontendFilterTelemetry = {};
}

void SAssetView::OnInterruptFiltering()
{
	if (CurrentFrontendFilterTelemetry.FilterSessionCorrelationGuid.IsValid())
	{
		CurrentFrontendFilterTelemetry.AmortizeDuration = FPlatformTime::Seconds() - AmortizeStartTime;
		CurrentFrontendFilterTelemetry.bCompleted = false;
		FTelemetryRouter::Get().ProvideTelemetry(CurrentFrontendFilterTelemetry);
		CurrentFrontendFilterTelemetry = {};
	}
}

void SAssetView::OnInteractDuringFiltering()
{
	if (CurrentFrontendFilterTelemetry.FilterSessionCorrelationGuid.IsValid() && !CurrentFrontendFilterTelemetry.TimeUntilInteraction.IsSet())
	{
		CurrentFrontendFilterTelemetry.TimeUntilInteraction = FPlatformTime::Seconds() - AmortizeStartTime;
	}
}

void SAssetView::CancelPendingRename()
{
	if (IsRenamingAsset())
	{
		if (TSharedPtr<FAssetViewItem> Item = RenamingAsset.Pin())
		{
			/**
			 * Check if the item is in a temp state and if it is, commit using the default name so that it does not entirely vanish on the user.
			 * This keeps the functionality consistent for content to never be in a temporary state
			 */

			if (Item && Item->IsTemporary())
			{
				if (Item->IsFile())
				{
					FText OutErrorText;
					EndCreateDeferredItem(Item, Item->GetItem().GetItemName().ToString(), /*bFinalize*/true, OutErrorText);
				}
				else
				{
					DeferredItemToCreate.Reset();
					Item->OnRenameCanceled().ExecuteIfBound();
				}
			}
			else
			{
				Item->OnRenameCanceled().ExecuteIfBound();
			}

			RenamingAsset.Reset();
		}
	}
}

#undef checkAssetList
#undef ASSET_VIEW_PARANOIA_LIST_CHECKS

#undef LOCTEXT_NAMESPACE
