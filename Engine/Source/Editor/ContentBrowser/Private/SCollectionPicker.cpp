// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCollectionPicker.h"

#include "Layout/Children.h"
#include "SCollectionView.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

void SCollectionPicker::Construct( const FArguments& InArgs )
{
	ChildSlot
	[
		SNew(SCollectionView)
		.AllowCollectionButtons(InArgs._CollectionPickerConfig.AllowCollectionButtons)
		.OnCollectionSelected(InArgs._CollectionPickerConfig.OnCollectionSelected)
		.AllowContextMenu( InArgs._CollectionPickerConfig.AllowRightClickMenu )
		.CollectionContainer(InArgs._CollectionPickerConfig.CollectionContainer)
	];
}

#undef LOCTEXT_NAMESPACE
