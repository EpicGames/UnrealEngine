// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IContentBrowserSingleton.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

/**
 * A sources view designed for collection picking
 */
class SCollectionPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SCollectionPicker ){}

		/** A struct containing details about how the collection picker should behave */
		SLATE_ARGUMENT(FCollectionPickerConfig, CollectionPickerConfig)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );
};
