// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCollectionView.h"

#include "Algo/AllOf.h"
#include "Algo/Transform.h"
#include "AssetRegistry/AssetData.h"
#include "CollectionAssetManagement.h"
#include "CollectionContextMenu.h"
#include "CollectionViewTypes.h"
#include "CollectionViewUtils.h"
#include "ContentBrowserConfig.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "ContentBrowserPluginFilters.h"
#include "ContentBrowserStyle.h"
#include "ContentBrowserUtils.h"
#include "CoreGlobals.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragAndDrop/CollectionDragDropOp.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Views/ITypedTableView.h"
#include "GenericPlatform/ICursor.h"
#include "HistoryManager.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "Input/DragAndDrop.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Layout/SlateRect.h"
#include "Layout/WidgetPath.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/CString.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/TextFilterUtils.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "SAssetTagItem.h"
#include "SlotBase.h"
#include "SourcesSearch.h"
#include "SourcesViewWidgets.h"
#include "TelemetryRouter.h"
#include "Settings/ContentBrowserSettings.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Tuple.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/STableRow.h"

class SWidget;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "ContentBrowser"

namespace CollectionViewFilter
{

void GetBasicStrings(const FCollectionItem& InCollection, TArray<FString>& OutBasicStrings)
{
	OutBasicStrings.Add(InCollection.CollectionName.ToString());
}

bool TestComplexExpression(const FCollectionItem& InCollection, const FName& InKey, const FTextFilterString& InValue, ETextFilterComparisonOperation InComparisonOperation, ETextFilterTextComparisonMode InTextComparisonMode)
{
	static const FName NameKeyName = "Name";
	static const FName TypeKeyName = "Type";

	// Handle the collection name
	if (InKey == NameKeyName)
	{
		// Names can only work with Equal or NotEqual type tests
		if (InComparisonOperation != ETextFilterComparisonOperation::Equal && InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
		{
			return false;
		}

		const bool bIsMatch = TextFilterUtils::TestBasicStringExpression(InCollection.CollectionName.ToString(), InValue, InTextComparisonMode);
		return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bIsMatch : !bIsMatch;
	}

	// Handle the collection type
	if (InKey == TypeKeyName)
	{
		// Types can only work with Equal or NotEqual type tests
		if (InComparisonOperation != ETextFilterComparisonOperation::Equal && InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
		{
			return false;
		}

		const bool bIsMatch = TextFilterUtils::TestBasicStringExpression(ECollectionShareType::ToString(InCollection.CollectionType), InValue, InTextComparisonMode);
		return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bIsMatch : !bIsMatch;
	}

	return false;
}

} // namespace CollectionViewFilter

SCollectionView::~SCollectionView()
{
	if (CollectionContainer)
	{
		CollectionContainer->OnCollectionCreated().RemoveAll(this);
		CollectionContainer->OnCollectionRenamed().RemoveAll(this);
		CollectionContainer->OnCollectionReparented().RemoveAll(this);
		CollectionContainer->OnCollectionDestroyed().RemoveAll(this);
		CollectionContainer->OnCollectionUpdated().RemoveAll(this);
		CollectionContainer->OnAssetsAddedToCollection().RemoveAll(this);
		CollectionContainer->OnAssetsRemovedFromCollection().RemoveAll(this);
	}

	if (ISourceControlModule* SourceControlModule = ISourceControlModule::GetPtr())
	{
		SourceControlModule->UnregisterProviderChanged(SourceControlProviderChangedHandle);
		SourceControlModule->GetProvider().UnregisterSourceControlStateChanged_Handle(SourceControlStateChangedDelegateHandle);
	}

	if (ExternalSearchPtr)
	{
		ExternalSearchPtr->OnSearchChanged().RemoveAll(this);
	}

	UContentBrowserSettings::OnSettingChanged().RemoveAll(this);
}

void SCollectionView::Construct( const FArguments& InArgs )
{
	OnCollectionSelected = InArgs._OnCollectionSelected;
	bAllowCollectionButtons = InArgs._AllowCollectionButtons;
	bAllowRightClickMenu = InArgs._AllowRightClickMenu;
	bAllowCollectionDrag = InArgs._AllowCollectionDrag;
	bAllowExternalSearch = false;
	bDraggedOver = false;

	bQueueCollectionItemsUpdate = false;
	bQueueSCCRefresh = true;

	IsDocked = InArgs._IsDocked;
	CollectionContainer = InArgs._CollectionContainer;

	if (CollectionContainer)
	{
		CollectionContainer->OnCollectionCreated().AddSP(this, &SCollectionView::HandleCollectionCreated);
		CollectionContainer->OnCollectionRenamed().AddSP(this, &SCollectionView::HandleCollectionRenamed);
		CollectionContainer->OnCollectionReparented().AddSP(this, &SCollectionView::HandleCollectionReparented);
		CollectionContainer->OnCollectionDestroyed().AddSP(this, &SCollectionView::HandleCollectionDestroyed);
		CollectionContainer->OnCollectionUpdated().AddSP(this, &SCollectionView::HandleCollectionUpdated);
		CollectionContainer->OnAssetsAddedToCollection().AddSP(this, &SCollectionView::HandleAssetsAddedToCollection);
		CollectionContainer->OnAssetsRemovedFromCollection().AddSP(this, &SCollectionView::HandleAssetsRemovedFromCollection);
	}

	SourceControlProviderChangedHandle = ISourceControlModule::Get().RegisterProviderChanged(FSourceControlProviderChanged::FDelegate::CreateSP(this, &SCollectionView::HandleSourceControlProviderChanged));
	SourceControlStateChangedDelegateHandle = ISourceControlModule::Get().GetProvider().RegisterSourceControlStateChanged_Handle(FSourceControlStateChanged::FDelegate::CreateSP(this, &SCollectionView::HandleSourceControlStateChanged));

	Commands = TSharedPtr< FUICommandList >(new FUICommandList);
	CollectionContextMenu = MakeShareable(new FCollectionContextMenu( SharedThis(this) ));
	CollectionContextMenu->BindCommands(Commands);

	CollectionItemTextFilter = MakeShareable(new FCollectionItemTextFilter(
		FCollectionItemTextFilter::FItemToStringArray::CreateStatic(&CollectionViewFilter::GetBasicStrings), 
		FCollectionItemTextFilter::FItemTestComplexExpression::CreateStatic(&CollectionViewFilter::TestComplexExpression)
		));
	CollectionItemTextFilter->OnChanged().AddSP(this, &SCollectionView::UpdateFilteredCollectionItems);

	if ( CollectionContainer && InArgs._AllowQuickAssetManagement )
	{
		QuickAssetManagement = MakeShareable(new FCollectionAssetManagement(CollectionContainer.ToSharedRef()));
	}

	FOnContextMenuOpening CollectionListContextMenuOpening;
	if ( InArgs._AllowContextMenu )
	{
		CollectionListContextMenuOpening = FOnContextMenuOpening::CreateSP( this, &SCollectionView::MakeCollectionTreeContextMenu );
	}

	ExternalSearchPtr = InArgs._ExternalSearch;
	if (ExternalSearchPtr)
	{
		ExternalSearchPtr->OnSearchChanged().AddSP(this, &SCollectionView::SetCollectionsSearchFilterText);
	}

	TitleContent = SNew(SHorizontalBox);

	UContentBrowserCollectionProjectSettings* CollectionProjectSettings = GetMutableDefault<UContentBrowserCollectionProjectSettings>();
	if (CollectionProjectSettings)
	{
		CollectionProjectSettings->OnSettingChanged().AddSPLambda(this, [this](UObject* CollectionProjectSettings_, struct FPropertyChangedEvent&)
		{
			UpdateFilteredCollectionItems();
		});
	}

	PreventSelectionChangedDelegateCount = 0;

	TSharedRef< SWidget > HeaderContent = SNew(SHorizontalBox)
			.Visibility(this, &SCollectionView::GetHeaderVisibility)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(0.0f)
			[
				TitleContent.ToSharedRef()
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("AddCollectionButtonTooltip", "Add a collection."))
				.OnClicked(this, &SCollectionView::OnAddCollectionClicked)
				.ContentPadding( FMargin(2, 2) )
				.Visibility(this, &SCollectionView::GetAddCollectionButtonVisibility)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];

	TSharedRef< SWidget > BodyContent = SNew(SVerticalBox)
			// Collections tree
			+SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				SAssignNew(CollectionTreePtr, STreeView< TSharedPtr<FCollectionItem> >)
				.TreeItemsSource(&VisibleRootCollectionItems)
				.OnGenerateRow(this, &SCollectionView::GenerateCollectionRow)
				.OnGetChildren(this, &SCollectionView::GetCollectionItemChildren)
				.SelectionMode(ESelectionMode::Multi)
				.OnSelectionChanged(this, &SCollectionView::CollectionSelectionChanged)
				.OnContextMenuOpening(CollectionListContextMenuOpening)
				.OnItemScrolledIntoView(this, &SCollectionView::CollectionItemScrolledIntoView)
				.ClearSelectionOnClick(false)
				.Visibility(this, &SCollectionView::GetCollectionTreeVisibility)
			];

	ChildSlot
	[
		SNew(SOverlay)

		// Main content
		+SOverlay::Slot()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(12.0f, 0.0f, 0.0f, 0.0f))
			[
				HeaderContent
			]
			+ SVerticalBox::Slot()
			[
				BodyContent
			]
		]
		// Drop target overlay
		+SOverlay::Slot()
		[
			SNew(SBorder)
			.Padding(0)
			.Visibility(EVisibility::HitTestInvisible)
			.BorderImage(this, &SCollectionView::GetCollectionViewDropTargetBorder)
			.BorderBackgroundColor(FLinearColor::Yellow)
			[
				SNullWidget::NullWidget
			]
		]
	];

	UContentBrowserSettings::OnSettingChanged().AddSP(this, &SCollectionView::HandleSettingChanged);

	UpdateCollectionItems();
}

bool SCollectionView::IsEmpty()
{
	// If the SCollectionView is hidden because it is empty, it won't tick, so we need to make sure AvailableCollections is up to date.
	if (bQueueCollectionItemsUpdate)
	{
		bQueueCollectionItemsUpdate = false;
		UpdateCollectionItems();
	}

	return AvailableCollections.IsEmpty();
}

void SCollectionView::HandleCollectionCreated( ICollectionContainer& InCollectionContainer, const FCollectionNameType& Collection )
{
	bQueueCollectionItemsUpdate = true;
}

void SCollectionView::HandleCollectionRenamed( ICollectionContainer& InCollectionContainer, const FCollectionNameType& OriginalCollection, const FCollectionNameType& NewCollection )
{
	bQueueCollectionItemsUpdate = true;

	// Rename the item in-place so we can maintain its expansion and selection states correctly once the view is refreshed on the next Tick
	TSharedPtr<FCollectionItem> CollectionItem = AvailableCollections.FindRef(OriginalCollection);
	if (CollectionItem.IsValid())
	{
		CollectionItem->CollectionName = NewCollection.Name;
		CollectionItem->CollectionType = NewCollection.Type;

		AvailableCollections.Remove(OriginalCollection);
		AvailableCollections.Add(NewCollection, CollectionItem);
	}
}

void SCollectionView::HandleCollectionReparented( ICollectionContainer& InCollectionContainer, const FCollectionNameType& Collection, const TOptional<FCollectionNameType>& OldParent, const TOptional<FCollectionNameType>& NewParent )
{
	bQueueCollectionItemsUpdate = true;
}

void SCollectionView::HandleCollectionDestroyed( ICollectionContainer& InCollectionContainer, const FCollectionNameType& Collection )
{
	bQueueCollectionItemsUpdate = true;
}

void SCollectionView::HandleCollectionUpdated( ICollectionContainer& InCollectionContainer, const FCollectionNameType& Collection )
{
	TSharedPtr<FCollectionItem> CollectionItemToUpdate = AvailableCollections.FindRef(Collection);
	if (CollectionItemToUpdate.IsValid())
	{
		bQueueSCCRefresh = true;
		CollectionItemToUpdate->CollectionColor = CollectionViewUtils::ResolveColor(InCollectionContainer, Collection.Name, Collection.Type);
		UpdateCollectionItemStatus(CollectionItemToUpdate.ToSharedRef());
	}
}

void SCollectionView::HandleAssetsAddedToCollection( ICollectionContainer& InCollectionContainer, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> AssetsAdded )
{
	HandleCollectionUpdated(InCollectionContainer, Collection);
}

void SCollectionView::HandleAssetsRemovedFromCollection( ICollectionContainer& InCollectionContainer, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> AssetsRemoved )
{
	HandleCollectionUpdated(InCollectionContainer, Collection);
}

void SCollectionView::HandleSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider)
{
	OldProvider.UnregisterSourceControlStateChanged_Handle(SourceControlStateChangedDelegateHandle);
	SourceControlStateChangedDelegateHandle = NewProvider.RegisterSourceControlStateChanged_Handle(FSourceControlStateChanged::FDelegate::CreateSP(this, &SCollectionView::HandleSourceControlStateChanged));
	
	bQueueSCCRefresh = true;
	HandleSourceControlStateChanged();
}

void SCollectionView::HandleSourceControlStateChanged()
{
	bQueueItemStatusUpdate = true;
}

void SCollectionView::UpdateCollectionItemStatus( const TSharedRef<FCollectionItem>& CollectionItem )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SCollectionView::UpdateCollectionItemStatus);

	int32 NewObjectCount = 0;
	TOptional<ECollectionItemStatus> NewStatus;

	// Check IsModuleAvailable as we might be in the process of shutting down, and were notified due to the SCC provider being nulled out...
	if (ensure(CollectionItem->CollectionContainer))
	{
		FCollectionStatusInfo StatusInfo;
		if (CollectionItem->CollectionContainer->GetCollectionStatusInfo(CollectionItem->CollectionName, CollectionItem->CollectionType, StatusInfo))
		{
			NewObjectCount = StatusInfo.NumObjects;

			// Test the SCC state first as this should take priority when reporting the status back to the user
			if (StatusInfo.bUseSCC)
			{
				if (StatusInfo.SCCState.IsValid() && StatusInfo.SCCState->IsSourceControlled())
				{
					if (StatusInfo.SCCState->IsCheckedOutOther())
					{
						NewStatus = ECollectionItemStatus::IsCheckedOutByAnotherUser;
					}
					else if (StatusInfo.SCCState->IsConflicted())
					{
						NewStatus = ECollectionItemStatus::IsConflicted;
					}
					else if (!StatusInfo.SCCState->IsCurrent())
					{
						NewStatus = ECollectionItemStatus::IsOutOfDate;
					}
					else if (StatusInfo.SCCState->IsModified())
					{
						NewStatus = ECollectionItemStatus::HasLocalChanges;
					}
				}
				else
				{
					NewStatus = ECollectionItemStatus::IsMissingSCCProvider;
				}
			}

			// Not set by the SCC status, so check just use the local state
			if (!NewStatus.IsSet())
			{
				if (StatusInfo.bIsDirty)
				{
					NewStatus = ECollectionItemStatus::HasLocalChanges;
				}
				else if (StatusInfo.bIsEmpty)
				{
					NewStatus = ECollectionItemStatus::IsUpToDateAndEmpty;
				}
				else
				{
					NewStatus = ECollectionItemStatus::IsUpToDateAndPopulated;
				}
			}
		}
	}

	CollectionItem->NumObjects = NewObjectCount;
	CollectionItem->CurrentStatus = NewStatus.Get(ECollectionItemStatus::IsUpToDateAndEmpty);
}

void SCollectionView::UpdateCollectionItems()
{
	struct FGatherCollectionItems
	{
		static void GatherCollectionItems(const TSharedRef<ICollectionContainer>& InCollectionContainer, FAvailableCollectionsMap& OutAvailableCollections)
		{
			TArray<FCollectionNameType> RootCollections;
			InCollectionContainer->GetRootCollections(RootCollections);

			ProcessGatheredCollectionsAndRecurse(InCollectionContainer, RootCollections, nullptr, OutAvailableCollections);
		}

	private:
		static void GatherChildCollectionItems(const TSharedRef<ICollectionContainer>& InCollectionContainer, const TSharedPtr<FCollectionItem>& InParentCollectionItem, FAvailableCollectionsMap& OutAvailableCollections)
		{
			TArray<FCollectionNameType> ChildCollections;
			InCollectionContainer->GetChildCollections(InParentCollectionItem->CollectionName, InParentCollectionItem->CollectionType, ChildCollections);

			ProcessGatheredCollectionsAndRecurse(InCollectionContainer, ChildCollections, InParentCollectionItem, OutAvailableCollections);
		}

		static void ProcessGatheredCollectionsAndRecurse(const TSharedRef<ICollectionContainer>& InCollectionContainer, const TArray<FCollectionNameType>& InCollections, const TSharedPtr<FCollectionItem>& InParentCollectionItem, FAvailableCollectionsMap& OutAvailableCollections)
		{
			for (const FCollectionNameType& Collection : InCollections)
			{
				// Never display system collections
				if (Collection.Type == ECollectionShareType::CST_System)
				{
					continue;
				}

				TSharedRef<FCollectionItem> CollectionItem = MakeShareable(new FCollectionItem(InCollectionContainer, Collection.Name, Collection.Type));
				OutAvailableCollections.Add(Collection, CollectionItem);

				InCollectionContainer->GetCollectionStorageMode(Collection.Name, Collection.Type, CollectionItem->StorageMode);
				CollectionItem->CollectionColor = CollectionViewUtils::ResolveColor(*InCollectionContainer, Collection.Name, Collection.Type);

				SCollectionView::UpdateCollectionItemStatus(CollectionItem);

				if (InParentCollectionItem.IsValid())
				{
					// Fixup the parent and child pointers
					InParentCollectionItem->ChildCollections.Add(CollectionItem);
					CollectionItem->ParentCollection = InParentCollectionItem;
				}

				// Recurse
				GatherChildCollectionItems(InCollectionContainer, CollectionItem, OutAvailableCollections);
			}
		}
	};

	// Backup the current selection and expansion state of our collections
	// We're about to re-create the tree, so we'll need to re-apply this again afterwards
	TArray<FCollectionNameType> SelectedCollections;
	TArray<FCollectionNameType> ExpandedCollections;
	{
		const auto SelectedCollectionItems = CollectionTreePtr->GetSelectedItems();
		SelectedCollections.Reserve(SelectedCollectionItems.Num());
		for (const TSharedPtr<FCollectionItem>& SelectedCollectionItem : SelectedCollectionItems)
		{
			SelectedCollections.Add(FCollectionNameType(SelectedCollectionItem->CollectionName, SelectedCollectionItem->CollectionType));
		}
	}
	{
		TSet<TSharedPtr<FCollectionItem>> ExpandedCollectionItems;
		CollectionTreePtr->GetExpandedItems(ExpandedCollectionItems);
		ExpandedCollections.Reserve(ExpandedCollectionItems.Num());
		for (const TSharedPtr<FCollectionItem>& ExpandedCollectionItem : ExpandedCollectionItems)
		{
			ExpandedCollections.Add(FCollectionNameType(ExpandedCollectionItem->CollectionName, ExpandedCollectionItem->CollectionType));
		}
	}

	AvailableCollections.Reset();

	if (CollectionContainer)
	{
		FGatherCollectionItems::GatherCollectionItems(CollectionContainer.ToSharedRef(), AvailableCollections);
	}

	UpdateFilteredCollectionItems();

	// Restore selection and expansion
	SetSelectedCollections(SelectedCollections, false);
	SetExpandedCollections(ExpandedCollections);

	bQueueSCCRefresh = true;
}

void SCollectionView::UpdateFilteredCollectionItems()
{
	VisibleCollections.Reset();
	VisibleRootCollectionItems.Reset();

	const UContentBrowserCollectionProjectSettings* CollectionProjectSettings = GetDefault<UContentBrowserCollectionProjectSettings>();
	const UContentBrowserSettings* ContentBrowserSettings = GetDefault<UContentBrowserSettings>();

	auto AddVisibleCollection = [&](const FCollectionNameType& NameTypePair, const TSharedPtr<FCollectionItem>& InCollectionItem)
	{
		if (!ContentBrowserSettings->bDisplayExcludedCollections)
		{
			const int32 FoundIndex = CollectionProjectSettings->ExcludedCollectionsFromView.Find(NameTypePair.Name);
			if (FoundIndex != INDEX_NONE)
			{
				return;
			}
		}
		
		VisibleCollections.Add(FCollectionNameType(InCollectionItem->CollectionName, InCollectionItem->CollectionType));
		if (!InCollectionItem->ParentCollection.IsValid())
		{
			VisibleRootCollectionItems.AddUnique(InCollectionItem);
		}
	};

	auto AddVisibleCollectionRecursive = [&](const FCollectionNameType& NameTypePair, const TSharedPtr<FCollectionItem>& InCollectionItem)
	{
		TSharedPtr<FCollectionItem> CollectionItemToAdd = InCollectionItem;
		do
		{
			AddVisibleCollection(NameTypePair, CollectionItemToAdd);
			CollectionItemToAdd = CollectionItemToAdd->ParentCollection.Pin();
		}
		while(CollectionItemToAdd.IsValid());
	};

	// Do we have an active filter to test against?
	if (CollectionItemTextFilter->GetRawFilterText().IsEmpty())
	{
		// No filter, just mark everything as visible
		for (const auto& AvailableCollectionInfo : AvailableCollections)
		{
			AddVisibleCollection(AvailableCollectionInfo.Key, AvailableCollectionInfo.Value);
		}
	}
	else
	{
		TArray<TSharedRef<FCollectionItem>> CollectionsToExpandTo;

		// Test everything against the filter - a visible child needs to make sure its parents are also marked as visible
		for (const auto& AvailableCollectionInfo : AvailableCollections)
		{
			const TSharedPtr<FCollectionItem>& CollectionItem = AvailableCollectionInfo.Value;
			if (CollectionItemTextFilter->PassesFilter(*CollectionItem))
			{
				AddVisibleCollectionRecursive(AvailableCollectionInfo.Key, CollectionItem);
				CollectionsToExpandTo.Add(CollectionItem.ToSharedRef());
			}
		}

		// Make sure all matching items have their parents expanded so they can be seen
		for (const TSharedRef<FCollectionItem>& CollectionItem : CollectionsToExpandTo)
		{
			ExpandParentItems(CollectionItem);
		}
	}

	VisibleRootCollectionItems.Sort(FCollectionItem::FCompareFCollectionItemByName());
	CollectionTreePtr->RequestTreeRefresh();
}

void SCollectionView::SetCollectionsSearchFilterText( const FText& InSearchText, TArray<FText>& OutErrors )
{
	CollectionItemTextFilter->SetRawFilterText( InSearchText );
	
	const FText ErrorText = CollectionItemTextFilter->GetFilterErrorText();
	if (!ErrorText.IsEmpty())
	{
		OutErrors.Add(ErrorText);
	}
}

FText SCollectionView::GetCollectionsSearchFilterText() const
{
	return CollectionItemTextFilter->GetRawFilterText();
}

void SCollectionView::SetSelectedCollections(const TArray<FCollectionNameType>& CollectionsToSelect, const bool bEnsureVisible)
{
	// Prevent the selection changed delegate since the invoking code requested it
	FScopedPreventSelectionChangedDelegate DelegatePrevention( SharedThis(this) );

	// Clear the selection to start, then add the selected items as they are found
	CollectionTreePtr->ClearSelection();

	if (CollectionsToSelect.IsEmpty())
	{
		return;
	}

	if (bQueueCollectionItemsUpdate)
	{
		bQueueCollectionItemsUpdate = false;
		UpdateCollectionItems();
	}

	for (const FCollectionNameType& CollectionToSelect : CollectionsToSelect)
	{
		TSharedPtr<FCollectionItem> CollectionItemToSelect = AvailableCollections.FindRef(CollectionToSelect);
		if (CollectionItemToSelect.IsValid())
		{
			if (bEnsureVisible)
			{
				ExpandParentItems(CollectionItemToSelect.ToSharedRef());
				CollectionTreePtr->RequestScrollIntoView(CollectionItemToSelect);
			}

			CollectionTreePtr->SetItemSelection(CollectionItemToSelect, true);

			// If the selected collection doesn't pass our current filter, we need to clear it
			if (bEnsureVisible && ExternalSearchPtr.IsValid() && !CollectionItemTextFilter->PassesFilter(*CollectionItemToSelect))
			{
				ExternalSearchPtr->ClearSearch();
			}
		}
	}
}

void SCollectionView::SetExpandedCollections(const TArray<FCollectionNameType>& CollectionsToExpand)
{
	// Clear the expansion to start, then add the expanded items as they are found
	CollectionTreePtr->ClearExpandedItems();

	for (const FCollectionNameType& CollectionToExpand : CollectionsToExpand)
	{
		TSharedPtr<FCollectionItem> CollectionItemToExpand = AvailableCollections.FindRef(CollectionToExpand);
		if (CollectionItemToExpand.IsValid())
		{
			CollectionTreePtr->SetItemExpansion(CollectionItemToExpand, true);
		}
	}
}

void SCollectionView::ClearSelection()
{
	// Prevent the selection changed delegate since the invoking code requested it
	FScopedPreventSelectionChangedDelegate DelegatePrevention( SharedThis(this) );

	// Clear the selection to start, then add the selected paths as they are found
	CollectionTreePtr->ClearSelection();
}

const TSharedPtr<ICollectionContainer>& SCollectionView::GetCollectionContainer() const
{
	return CollectionContainer;
}

TArray<FCollectionNameType> SCollectionView::GetSelectedCollections() const
{
	TArray<FCollectionNameType> RetArray;

	TArray<TSharedPtr<FCollectionItem>> Items = CollectionTreePtr->GetSelectedItems();
	for ( int32 ItemIdx = 0; ItemIdx < Items.Num(); ++ItemIdx )
	{
		const TSharedPtr<FCollectionItem>& Item = Items[ItemIdx];
		RetArray.Add(FCollectionNameType(Item->CollectionName, Item->CollectionType));
	}

	return RetArray;
}

void SCollectionView::SetSelectedAssetPaths(const TArray<FSoftObjectPath>& SelectedAssets)
{
	if ( QuickAssetManagement.IsValid() )
	{
		QuickAssetManagement->SetCurrentAssetPaths(SelectedAssets);
	}
}

void SCollectionView::ApplyHistoryData( const FHistoryData& History )
{	
	// Prevent the selection changed delegate because it would add more history when we are just setting a state
	FScopedPreventSelectionChangedDelegate DelegatePrevention( SharedThis(this) );

	CollectionTreePtr->ClearSelection();
	for (const FCollectionRef& HistoryCollection : History.ContentSources.GetCollections())
	{
		if (CollectionContainer == HistoryCollection.Container)
		{
			TSharedPtr<FCollectionItem> CollectionHistoryItem = AvailableCollections.FindRef(FCollectionNameType(HistoryCollection.Name, HistoryCollection.Type));
			if (CollectionHistoryItem.IsValid())
			{
				ExpandParentItems(CollectionHistoryItem.ToSharedRef());
				CollectionTreePtr->RequestScrollIntoView(CollectionHistoryItem);
				CollectionTreePtr->SetItemSelection(CollectionHistoryItem, true);
			}
		}
	}
}

void SCollectionView::SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const
{
	auto SaveCollectionsArrayToIni = [&](const FString& InSubKey, const TArray<TSharedPtr<FCollectionItem>>& InCollectionItems)
	{
		FString CollectionsString;

		for (const TSharedPtr<FCollectionItem>& CollectionItem : InCollectionItems)
		{
			if (CollectionsString.Len() > 0)
			{
				CollectionsString += TEXT(",");
			}

			CollectionsString += CollectionItem->CollectionName.ToString();
			CollectionsString += TEXT("?");
			CollectionsString += FString::FromInt(CollectionItem->CollectionType);
		}

		GConfig->SetString(*IniSection, *(SettingsString + InSubKey), *CollectionsString, IniFilename);
	};

	SaveCollectionsArrayToIni(TEXT(".SelectedCollections"), CollectionTreePtr->GetSelectedItems());
	{
		TSet<TSharedPtr<FCollectionItem>> ExpandedCollectionItems;
		CollectionTreePtr->GetExpandedItems(ExpandedCollectionItems);
		SaveCollectionsArrayToIni(TEXT(".ExpandedCollections"), ExpandedCollectionItems.Array());
	}
}

void SCollectionView::LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString)
{
	auto LoadCollectionsArrayFromIni = [&](const FString& InSubKey) -> TArray<FCollectionNameType>
	{
		TArray<FCollectionNameType> RetCollectionsArray;

		FString CollectionsArrayString;
		if (GConfig->GetString(*IniSection, *(SettingsString + InSubKey), CollectionsArrayString, IniFilename))
		{
			TArray<FString> CollectionStrings;
			CollectionsArrayString.ParseIntoArray(CollectionStrings, TEXT(","), /*bCullEmpty*/true);

			for (const FString& CollectionString : CollectionStrings)
			{
				FString CollectionName;
				FString CollectionTypeString;
				if (CollectionString.Split(TEXT("?"), &CollectionName, &CollectionTypeString))
				{
					const int32 CollectionType = FCString::Atoi(*CollectionTypeString);
					if (CollectionType >= 0 && CollectionType < ECollectionShareType::CST_All)
					{
						RetCollectionsArray.Add(FCollectionNameType(FName(*CollectionName), ECollectionShareType::Type(CollectionType)));
					}
				}
			}
		}

		return RetCollectionsArray;
	};

	// Selected Collections
	TArray<FCollectionNameType> NewSelectedCollections = LoadCollectionsArrayFromIni(TEXT(".SelectedCollections"));
	if (NewSelectedCollections.Num() > 0)
	{
		SetSelectedCollections(NewSelectedCollections);

		const TArray<TSharedPtr<FCollectionItem>> SelectedCollectionItems = CollectionTreePtr->GetSelectedItems();
		if (SelectedCollectionItems.Num() > 0)
		{
			CollectionSelectionChanged(SelectedCollectionItems[0], ESelectInfo::Direct);
		}
	}

	// Expanded Collections
	TArray<FCollectionNameType> NewExpandedCollections = LoadCollectionsArrayFromIni(TEXT(".ExpandedCollections"));
	if (NewExpandedCollections.Num() > 0)
	{
		SetExpandedCollections(NewExpandedCollections);
	}
}

void SCollectionView::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (bQueueCollectionItemsUpdate)
	{
		bQueueCollectionItemsUpdate = false;
		UpdateCollectionItems();
	}

	if (bQueueSCCRefresh && CollectionContainer && ISourceControlModule::Get().IsEnabled())
	{
		bQueueSCCRefresh = false;

		TArray<FString> CollectionFilesToRefresh;
		for (const auto& AvailableCollectionInfo : AvailableCollections)
		{
			FCollectionStatusInfo StatusInfo;
			if (CollectionContainer->GetCollectionStatusInfo(AvailableCollectionInfo.Value->CollectionName, AvailableCollectionInfo.Value->CollectionType, StatusInfo))
			{
				if (StatusInfo.bUseSCC && StatusInfo.SCCState.IsValid() && StatusInfo.SCCState->IsSourceControlled())
				{
					CollectionFilesToRefresh.Add(StatusInfo.SCCState->GetFilename());
				}
			}
		}

		if (CollectionFilesToRefresh.Num() > 0)
		{
			ISourceControlModule::Get().QueueStatusUpdate(CollectionFilesToRefresh);
		}
	}

	if (bQueueItemStatusUpdate)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("CollectionView Update Collection Items Status");

		bQueueItemStatusUpdate = false;
		// Update the status of each collection
		for (const TPair<FCollectionNameType, TSharedPtr<FCollectionItem>>& AvailableCollectionInfo : AvailableCollections)
		{
			UpdateCollectionItemStatus(AvailableCollectionInfo.Value.ToSharedRef());
		}
	}
}

FReply SCollectionView::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if( Commands->ProcessCommandBindings( InKeyEvent ) )
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SCollectionView::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	ValidateDragDropOnCollectionTree(MyGeometry, DragDropEvent, bDraggedOver); // updates bDraggedOver
}

void SCollectionView::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	bDraggedOver = false;
}

FReply SCollectionView::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	ValidateDragDropOnCollectionTree(MyGeometry, DragDropEvent, bDraggedOver); // updates bDraggedOver
	return (bDraggedOver) ? FReply::Handled() : FReply::Unhandled();
}

FReply SCollectionView::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if (ValidateDragDropOnCollectionTree(MyGeometry, DragDropEvent, bDraggedOver)) // updates bDraggedOver
	{
		bDraggedOver = false;
		return HandleDragDropOnCollectionTree(MyGeometry, DragDropEvent);
	}

	if (bDraggedOver)
	{
		// We were able to handle this operation, but could not due to another error - still report this drop as handled so it doesn't fall through to other widgets
		bDraggedOver = false;
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SCollectionView::MakeSaveDynamicCollectionMenu(FMenuBuilder& InMenuBuilder, FText InQueryString)
{
	CollectionContextMenu->UpdateProjectSourceControl();
	CollectionContextMenu->MakeSaveDynamicCollectionSubMenu(InMenuBuilder, InQueryString);
}

FReply SCollectionView::OnAddCollectionClicked()
{
	MakeAddCollectionMenu(AsShared());
	return FReply::Handled();
}

bool SCollectionView::ShouldAllowSelectionChangedDelegate() const
{
	return PreventSelectionChangedDelegateCount == 0;
}

void SCollectionView::MakeAddCollectionMenu(TSharedRef<SWidget> MenuParent)
{
	// Get all menu extenders for this context menu from the content browser module
	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>( TEXT("ContentBrowser") );
	TArray<FContentBrowserMenuExtender> MenuExtenderDelegates = ContentBrowserModule.GetAllCollectionViewContextMenuExtenders();

	TArray<TSharedPtr<FExtender>> Extenders;
	for (int32 i = 0; i < MenuExtenderDelegates.Num(); ++i)
	{
		if (MenuExtenderDelegates[i].IsBound())
		{
			Extenders.Add(MenuExtenderDelegates[i].Execute());
		}
	}
	TSharedPtr<FExtender> MenuExtender = FExtender::Combine(Extenders);

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, NULL, MenuExtender, true);

	CollectionContextMenu->UpdateProjectSourceControl();

	CollectionContextMenu->MakeNewCollectionSubMenu(MenuBuilder, ECollectionStorageMode::Static, SCollectionView::FCreateCollectionPayload());

	FSlateApplication::Get().PushMenu(
		MenuParent,
		FWidgetPath(),
		MenuBuilder.MakeWidget(),
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect( FPopupTransitionEffect::TopMenu )
		);
}

FVector2D SCollectionView::GetScrollDistance()
{
	if (!CollectionTreePtr.IsValid())
	{
		return FVector2D::ZeroVector;
	}

	return CollectionTreePtr->GetScrollDistance();
}

FVector2D SCollectionView::GetScrollDistanceRemaining()
{
	if (!CollectionTreePtr.IsValid())
	{
		return FVector2D::ZeroVector;
	}

	return CollectionTreePtr->GetScrollDistanceRemaining();
}

TSharedRef<SWidget> SCollectionView::GetScrollWidget()
{
	return SharedThis(this);
}

void SCollectionView::CreateCollectionItem( ECollectionShareType::Type CollectionType, ECollectionStorageMode::Type StorageMode, const FCreateCollectionPayload& InCreationPayload )
{
	if ( ensure(CollectionContainer && CollectionType != ECollectionShareType::CST_All) )
	{
		const FName BaseCollectionName = *LOCTEXT("NewCollectionName", "NewCollection").ToString();
		FName CollectionName;
		CollectionContainer->CreateUniqueCollectionName(BaseCollectionName, ECollectionShareType::CST_All, CollectionName);
		TSharedPtr<FCollectionItem> NewItem = MakeShareable(new FCollectionItem(CollectionContainer.ToSharedRef(), CollectionName, CollectionType));
		NewItem->StorageMode = StorageMode;

		// Adding a new collection now, so clear any filter we may have applied
		if (ExternalSearchPtr)
		{
			ExternalSearchPtr->ClearSearch();
		}

		if ( InCreationPayload.ParentCollection.IsSet() )
		{
			TSharedPtr<FCollectionItem> ParentCollectionItem = AvailableCollections.FindRef(InCreationPayload.ParentCollection.GetValue());
			if ( ParentCollectionItem.IsValid() )
			{
				ParentCollectionItem->ChildCollections.Add(NewItem);
				NewItem->ParentCollection = ParentCollectionItem;

				// Make sure the parent is expanded so we can see its newly added child item
				CollectionTreePtr->SetItemExpansion(ParentCollectionItem, true);
			}
		}

		// Mark the new collection for rename and that it is new so it will be created upon successful rename
		NewItem->bRenaming = true;
		NewItem->bNewCollection = true;
		NewItem->OnCollectionCreatedEvent = InCreationPayload.OnCollectionCreatedEvent;

		AvailableCollections.Add( FCollectionNameType(NewItem->CollectionName, NewItem->CollectionType), NewItem );
		UpdateFilteredCollectionItems();
		CollectionTreePtr->RequestScrollIntoView(NewItem);
		CollectionTreePtr->SetSelection( NewItem );
	}
}

void SCollectionView::RenameCollectionItem( const TSharedPtr<FCollectionItem>& ItemToRename )
{
	if ( ensure(ItemToRename.IsValid()) )
	{
		ItemToRename->bRenaming = true;
		CollectionTreePtr->RequestScrollIntoView(ItemToRename);
	}
}

void SCollectionView::DeleteCollectionItems( const TArray<TSharedPtr<FCollectionItem>>& ItemsToDelete )
{
	if (ItemsToDelete.Num() == 0)
	{
		return;
	}

	// Before we delete anything (as this will trigger a tree update) we need to work out what our new selection should be in the case that 
	// all of the selected items are removed
	const TArray<TSharedPtr<FCollectionItem>> PreviouslySelectedItems = CollectionTreePtr->GetSelectedItems();

	// Get the first selected item that will be deleted so we can find a suitable new selection
	TSharedPtr<FCollectionItem> FirstSelectedItemDeleted;
	for (const auto& ItemToDelete : ItemsToDelete)
	{
		if (PreviouslySelectedItems.Contains(ItemToDelete))
		{
			FirstSelectedItemDeleted = ItemToDelete;
			break;
		}
	}

	// Build up an array of potential new selections (in the case that we're deleting everything that's selected)
	// Earlier items should be considered first, we base this list on the first selected item that will be deleted, and include previous siblings, and then all parents and roots
	TArray<FCollectionNameType> PotentialNewSelections;
	if (FirstSelectedItemDeleted.IsValid())
	{
		TSharedPtr<FCollectionItem> RootSelectedItemDeleted = FirstSelectedItemDeleted;
		TSharedPtr<FCollectionItem> ParentCollectionItem = FirstSelectedItemDeleted->ParentCollection.Pin();

		if (ParentCollectionItem.IsValid())
		{
			// Add all the siblings until we find the item that will be deleted
			for (const auto& ChildItemWeakPtr : ParentCollectionItem->ChildCollections)
			{
				TSharedPtr<FCollectionItem> ChildItem = ChildItemWeakPtr.Pin();
				if (ChildItem.IsValid())
				{
					if (ChildItem == FirstSelectedItemDeleted)
					{
						break;
					}
					
					// We add siblings as the start, as the closest sibling should be the first match
					PotentialNewSelections.Insert(FCollectionNameType(ChildItem->CollectionName, ChildItem->CollectionType), 0);
				}
			}

			// Now add this parent, and all other parents too
			do
			{
				PotentialNewSelections.Add(FCollectionNameType(ParentCollectionItem->CollectionName, ParentCollectionItem->CollectionType));
				RootSelectedItemDeleted = ParentCollectionItem;
				ParentCollectionItem = ParentCollectionItem->ParentCollection.Pin();
			}
			while (ParentCollectionItem.IsValid());
		}

		if (RootSelectedItemDeleted.IsValid())
		{
			// Add all the root level items before this one
			const int32 InsertionPoint = PotentialNewSelections.Num();
			for (const auto& RootItem : VisibleRootCollectionItems)
			{
				if (RootItem == RootSelectedItemDeleted)
				{
					break;
				}
				
				// Add each root item at the insertion point, as the closest item should be a better match
				PotentialNewSelections.Insert(FCollectionNameType(RootItem->CollectionName, RootItem->CollectionType), InsertionPoint);
			}
		}
	}

	// Delete all given collections
	int32 NumSelectedItemsDeleted = 0;
	for (const TSharedPtr<FCollectionItem>& ItemToDelete : ItemsToDelete)
	{
		if (!ensure(ItemToDelete->CollectionContainer))
		{
			continue;
		}

		FText Error;
		if (ItemToDelete->CollectionContainer->DestroyCollection(ItemToDelete->CollectionName, ItemToDelete->CollectionType, &Error))
		{
			if (PreviouslySelectedItems.Contains(ItemToDelete))
			{
				++NumSelectedItemsDeleted;
			}
		}
		else
		{
			// Display a warning
			const FVector2f& CursorPos = FSlateApplication::Get().GetCursorPos();
			FSlateRect MessageAnchor(CursorPos.X, CursorPos.Y, CursorPos.X, CursorPos.Y);
			ContentBrowserUtils::DisplayMessage(
				FText::Format( LOCTEXT("CollectionDestroyFailed", "Failed to destroy collection. {0}"), Error),
				MessageAnchor,
				CollectionTreePtr.ToSharedRef(),
				ContentBrowserUtils::EDisplayMessageType::Error
				);
		}
	}

	// DestroyCollection will have triggered a notification that will have updated the tree, we now need to apply a suitable selection...

	// Did this delete change the list of selected items?
	if (NumSelectedItemsDeleted > 0 || PreviouslySelectedItems.Num() == 0)
	{
		// If we removed everything that was selected, we need to try and find a suitable replacement...
		if (NumSelectedItemsDeleted >= PreviouslySelectedItems.Num() && VisibleCollections.Num() > 1)
		{
			// Include the first visible item as an absolute last resort should everything else suitable have been removed from the tree
			PotentialNewSelections.Add(*VisibleCollections.CreateConstIterator());

			// Check the potential new selections array and try and select the first one that's still visible in the tree
			TArray<FCollectionNameType> NewItemSelection;
			for (const FCollectionNameType& PotentialNewSelection : PotentialNewSelections)
			{
				if (VisibleCollections.Contains(PotentialNewSelection))
				{
					NewItemSelection.Add(PotentialNewSelection);
					break;
				}
			}

			SetSelectedCollections(NewItemSelection, true);
		}

		// Broadcast the new selection
		const TArray<TSharedPtr<FCollectionItem>> UpdatedSelectedItems = CollectionTreePtr->GetSelectedItems();
		CollectionSelectionChanged((UpdatedSelectedItems.Num() > 0) ? UpdatedSelectedItems[0] : nullptr, ESelectInfo::Direct);
	}
}

EVisibility SCollectionView::GetCollectionTreeVisibility() const
{
	return AvailableCollections.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SCollectionView::GetHeaderVisibility() const
{
	return IsDocked.Get() ? EVisibility::Collapsed : EVisibility::SelfHitTestInvisible;
}

EVisibility SCollectionView::GetAddCollectionButtonVisibility() const
{
	return bAllowCollectionButtons && CollectionContainer.IsValid() && !CollectionContainer->IsReadOnly(ECollectionShareType::CST_All) ? EVisibility::Visible : EVisibility::Collapsed;
}

const FSlateBrush* SCollectionView::GetCollectionViewDropTargetBorder() const
{
	return bDraggedOver ? UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.CollectionTreeDragDropBorder") : FAppStyle::GetBrush("NoBorder");
}

TSharedRef<ITableRow> SCollectionView::GenerateCollectionRow( TSharedPtr<FCollectionItem> CollectionItem, const TSharedRef<STableViewBase>& OwnerTable )
{
	check(CollectionItem.IsValid());

	// Only bind the check box callbacks if we're allowed to show check boxes
	TAttribute<bool> IsCollectionCheckBoxEnabledAttribute;
	TAttribute<ECheckBoxState> IsCollectionCheckedAttribute;
	FOnCheckStateChanged OnCollectionCheckStateChangedDelegate;
	if ( QuickAssetManagement.IsValid() )
	{
		// Can only manage assets for static collections
		if (CollectionItem->StorageMode == ECollectionStorageMode::Static)
		{
			IsCollectionCheckBoxEnabledAttribute.Bind(TAttribute<bool>::FGetter::CreateSP(this, &SCollectionView::IsCollectionCheckBoxEnabled, CollectionItem));
			IsCollectionCheckedAttribute.Bind(TAttribute<ECheckBoxState>::FGetter::CreateSP(this, &SCollectionView::IsCollectionChecked, CollectionItem));
			OnCollectionCheckStateChangedDelegate.BindSP(this, &SCollectionView::OnCollectionCheckStateChanged, CollectionItem);
		}
		else
		{
			IsCollectionCheckBoxEnabledAttribute.Bind(TAttribute<bool>::FGetter::CreateLambda([]{ return false; }));
			IsCollectionCheckedAttribute.Bind(TAttribute<ECheckBoxState>::FGetter::CreateLambda([]{ return ECheckBoxState::Unchecked; }));
		}
	}

	TSharedPtr< SAssetTagItemTableRow< TSharedPtr<FCollectionItem> > > TableRow = SNew( SAssetTagItemTableRow< TSharedPtr<FCollectionItem> >, OwnerTable )
		.OnDragDetected(this, &SCollectionView::OnCollectionDragDetected);

	TSharedPtr<SCollectionTreeItem> CollectionTreeItem;

	TableRow->SetContent
		(
			SAssignNew(CollectionTreeItem, SCollectionTreeItem)
			.ParentWidget(SharedThis(this))
			.CollectionItem(CollectionItem)
			.OnNameChangeCommit(this, &SCollectionView::CollectionNameChangeCommit)
			.OnVerifyRenameCommit(this, &SCollectionView::CollectionVerifyRenameCommit)
			.OnValidateDragDrop(this, &SCollectionView::ValidateDragDropOnCollectionItem)
			.OnHandleDragDrop(this, &SCollectionView::HandleDragDropOnCollectionItem)
			.IsSelected(TableRow.Get(), &STableRow< TSharedPtr<FCollectionItem> >::IsSelectedExclusively)
			.IsReadOnly(this, &SCollectionView::IsCollectionNameReadOnly)
			.HighlightText(this, &SCollectionView::GetCollectionsSearchFilterText)
			.IsCheckBoxEnabled(IsCollectionCheckBoxEnabledAttribute)
			.IsCollectionChecked(IsCollectionCheckedAttribute)
			.OnCollectionCheckStateChanged(OnCollectionCheckStateChangedDelegate)
		);

	TableRow->SetIsDropTarget(MakeAttributeSP(CollectionTreeItem.Get(), &SCollectionTreeItem::IsDraggedOver));

	return TableRow.ToSharedRef();
}

void SCollectionView::GetCollectionItemChildren( TSharedPtr<FCollectionItem> InParentItem, TArray< TSharedPtr<FCollectionItem> >& OutChildItems ) const
{
	for (const auto& ChildItemWeakPtr : InParentItem->ChildCollections)
	{
		TSharedPtr<FCollectionItem> ChildItem = ChildItemWeakPtr.Pin();
		if (ChildItem.IsValid() && VisibleCollections.Contains(FCollectionNameType(ChildItem->CollectionName, ChildItem->CollectionType)))
		{
			OutChildItems.Add(ChildItem);
		}
	}
	OutChildItems.Sort(FCollectionItem::FCompareFCollectionItemByName());
}

FReply SCollectionView::OnCollectionDragDetected(const FGeometry& Geometry, const FPointerEvent& MouseEvent)
{
	if (bAllowCollectionDrag && MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		const TArray<FCollectionNameType> SelectedCollections = GetSelectedCollections();
		if (SelectedCollections.Num() > 0)
		{
			TArray<FCollectionRef> CollectionRefs;
			CollectionRefs.Reserve(SelectedCollections.Num());
			Algo::Transform(
				SelectedCollections,
				CollectionRefs,
				[this](const FCollectionNameType& Collection) { return FCollectionRef(CollectionContainer, Collection); });

			TSharedRef<FCollectionDragDropOp> DragDropOp = FCollectionDragDropOp::New(CollectionRefs);
			CurrentCollectionDragDropOp = DragDropOp;
			return FReply::Handled().BeginDragDrop(DragDropOp);
		}
	}

	return FReply::Unhandled();
}

bool SCollectionView::ValidateDragDropOnCollectionTree(const FGeometry& Geometry, const FDragDropEvent& DragDropEvent, bool& OutIsKnownDragOperation)
{
	OutIsKnownDragOperation = false;

	if (!CollectionContainer.IsValid())
	{
		return false;
	}

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return false;
	}

	if (Operation->IsOfType<FCollectionDragDropOp>())
	{
		TSharedPtr<FCollectionDragDropOp> DragDropOp = StaticCastSharedPtr<FCollectionDragDropOp>(Operation);

		OutIsKnownDragOperation = true;
		
		if (Algo::AllOf(DragDropOp->CollectionRefs, [this](const FCollectionRef& Collection)
			{
				return CollectionContainer == Collection.Container;
			}))
		{
			return true;
		}
		else
		{
			DragDropOp->SetToolTip(
				LOCTEXT("InvalidParentCollectionContainer", "A collection cannot be parented to a different collection container"),
				FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));

			Operation->SetCursorOverride(EMouseCursor::SlashedCircle);
		}
	}
	else if (Operation->IsOfType<FAssetDragDropOp>())
	{
		TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);

		if (!DragDropOp->HasAssets())
		{
			DragDropOp->SetCursorOverride(EMouseCursor::SlashedCircle);
			DragDropOp->SetToolTip(LOCTEXT("CollectionView_DragDrop_NoAsset", "There is no asset being dragged"), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
			return false;
		}

		const int32 NumDraggedItems = DragDropOp->GetAssets().Num();
		const FText FirstItemText = FText::FromName(DragDropOp->GetAssets()[0].AssetName);
		const FText AddToCollectionText = (NumDraggedItems > 1)
			? FText::Format(LOCTEXT("CollectionView_DragDrop_MultipleItems", "Add '{0}' and {1} {1}|plural(one=other,other=others)"), FirstItemText, NumDraggedItems - 1)
			: FText::Format(LOCTEXT("CollectionView_DragDrop_SingularItems", "Add '{0}'"), FirstItemText);

		DragDropOp->SetToolTip(AddToCollectionText, FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")));
		return true;

	}

	return false;
}

FReply SCollectionView::HandleDragDropOnCollectionTree(const FGeometry& Geometry, const FDragDropEvent& DragDropEvent)
{
	// Should have already called ValidateDragDropOnCollectionTree prior to calling this...
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	check(Operation.IsValid());

	if (Operation->IsOfType<FCollectionDragDropOp>())
	{
		TSharedPtr<FCollectionDragDropOp> DragDropOp = StaticCastSharedPtr<FCollectionDragDropOp>(Operation);

		// Reparent all of the collections in the drag drop so that they are root level items
		for (const FCollectionRef& NewChildCollection : DragDropOp->CollectionRefs)
		{
			check(CollectionContainer == NewChildCollection.Container);

			FText Error;
			if (!CollectionContainer->ReparentCollection(
					NewChildCollection.Name, NewChildCollection.Type,
					NAME_None, ECollectionShareType::CST_All, &Error
					))
			{
				ContentBrowserUtils::DisplayMessage(Error, Geometry.GetLayoutBoundingRect(), SharedThis(this), ContentBrowserUtils::EDisplayMessageType::Error);
			}
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SCollectionView::ValidateDragDropOnCollectionItem(TSharedRef<FCollectionItem> CollectionItem, const FGeometry& Geometry, const FDragDropEvent& DragDropEvent, bool& OutIsKnownDragOperation)
{
	OutIsKnownDragOperation = false;

	if (!ensure(CollectionItem->CollectionContainer))
	{
		return false;
	}

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (!Operation.IsValid())
	{
		return false;
	}

	bool bIsValidDrag = false;
	TOptional<EMouseCursor::Type> NewDragCursor;

	if (Operation->IsOfType<FCollectionDragDropOp>())
	{
		TSharedPtr<FCollectionDragDropOp> DragDropOp = StaticCastSharedPtr<FCollectionDragDropOp>(Operation);

		OutIsKnownDragOperation = true;

		bIsValidDrag = true;
		for (const FCollectionRef& PotentialChildCollection : DragDropOp->CollectionRefs)
		{
			FText Error;
			if (CollectionItem->CollectionContainer == PotentialChildCollection.Container)
			{
				bIsValidDrag = CollectionItem->CollectionContainer->IsValidParentCollection(
					PotentialChildCollection.Name, PotentialChildCollection.Type,
					CollectionItem->CollectionName, CollectionItem->CollectionType, &Error);
			}
			else
			{
				bIsValidDrag = false;
				Error = LOCTEXT("InvalidParentCollectionContainer", "A collection cannot be parented to a different collection container");
			}

			if (!bIsValidDrag)
			{
				DragDropOp->SetToolTip(Error, FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
				break;
			}
		}

		// If we are dragging over a child collection item, then this view as a whole should not be marked as dragged over
		bDraggedOver = false;
	}
	else if (Operation->IsOfType<FAssetDragDropOp>())
	{
		TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);
		OutIsKnownDragOperation = true;
		bIsValidDrag = DragDropOp->HasAssets();

		if (!bIsValidDrag)
		{
			DragDropOp->SetToolTip(LOCTEXT("CollectionViewItem_DragDrop_NoAsset", "There is no asset being dragged"), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
		}
		else
		{
			const int32 NumDraggedItems = DragDropOp->GetAssets().Num();
			const FText FirstItemText = FText::FromName(DragDropOp->GetAssets()[0].AssetName);
			const FText AddToCollectionText = (NumDraggedItems > 1)
				? FText::Format(LOCTEXT("CollectionViewItem_DragDrop_MultipleItems", "Add '{0}' and {1} {1}|plural(one=other,other=others)"), FirstItemText, NumDraggedItems - 1)
				: FText::Format(LOCTEXT("CollectionViewItem_DragDrop_SingularItems", "Add '{0}'"), FirstItemText);

			DragDropOp->SetToolTip(AddToCollectionText, FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")));
		}
	}

	// Set the default slashed circle if this drag is invalid and a drag operation hasn't set NewDragCursor to something custom
	if (!bIsValidDrag && !NewDragCursor.IsSet())
	{
		NewDragCursor = EMouseCursor::SlashedCircle;
	}
	Operation->SetCursorOverride(NewDragCursor);

	return bIsValidDrag;
}

FReply SCollectionView::HandleDragDropOnCollectionItem(TSharedRef<FCollectionItem> CollectionItem, const FGeometry& Geometry, const FDragDropEvent& DragDropEvent)
{
	// Should have already called ValidateDragDropOnCollectionItem prior to calling this...
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	check(Operation.IsValid());

	if (Operation->IsOfType<FCollectionDragDropOp>())
	{
		TSharedPtr<FCollectionDragDropOp> DragDropOp = StaticCastSharedPtr<FCollectionDragDropOp>(Operation);

		// Make sure our drop item is marked as expanded so that we'll be able to see the newly added children
		CollectionTreePtr->SetItemExpansion(CollectionItem, true);

		// Reparent all of the collections in the drag drop so that they are our immediate children
		for (const FCollectionRef& NewChildCollection : DragDropOp->CollectionRefs)
		{
			check(CollectionItem->CollectionContainer == NewChildCollection.Container);

			FText Error;
			if (!CollectionItem->CollectionContainer->ReparentCollection(
					NewChildCollection.Name, NewChildCollection.Type,
					CollectionItem->CollectionName, CollectionItem->CollectionType, &Error
					))
			{
				ContentBrowserUtils::DisplayMessage(Error, Geometry.GetLayoutBoundingRect(), SharedThis(this), ContentBrowserUtils::EDisplayMessageType::Error);
			}
		}

		return FReply::Handled();
	}
	else if (Operation->IsOfType<FAssetDragDropOp>())
	{
		TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);
		const TArray<FAssetData>& DroppedAssets = DragDropOp->GetAssets();

		TArray<FSoftObjectPath> ObjectPaths;
		ObjectPaths.Reserve(DroppedAssets.Num());
		for (const FAssetData& AssetData : DroppedAssets)
		{
			ObjectPaths.Add(AssetData.GetSoftObjectPath());
		}

		const double BeginTimeSec = FPlatformTime::Seconds();
		int32 NumAdded = 0;
		FText Message;
		ContentBrowserUtils::EDisplayMessageType MessageType = ContentBrowserUtils::EDisplayMessageType::Info;
		if (CollectionItem->CollectionContainer->AddToCollection(
				CollectionItem->CollectionName, CollectionItem->CollectionType, ObjectPaths, &NumAdded, &Message))
		{
			if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
			{
				Message = FText::Format(LOCTEXT("AddingToCollection", "{0} {0}|plural(one=item,other=items) added to '{1}'"), NumAdded, FText::FromName(CollectionItem->CollectionName));
				MessageType = ContentBrowserUtils::EDisplayMessageType::Successful;
			}
			else
			{
				if (DroppedAssets.Num() == 1)
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("AssetName"), FText::FromName(DroppedAssets[0].AssetName));
					Args.Add(TEXT("CollectionName"), FText::FromName(CollectionItem->CollectionName));
					Message = FText::Format(LOCTEXT("CollectionAssetAdded", "Added {AssetName} to {CollectionName}"), Args);
				}
				else
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("Number"), NumAdded);
					Args.Add(TEXT("CollectionName"), FText::FromName(CollectionItem->CollectionName));
					Message = FText::Format(LOCTEXT("CollectionAssetsAdded", "Added {Number} asset(s) to {CollectionName}"), Args);
				}
				MessageType = ContentBrowserUtils::EDisplayMessageType::Info;
			}

			const double DurationSec = FPlatformTime::Seconds() - BeginTimeSec;

			{
				FAssetAddedToCollectionTelemetryEvent AssetAdded;
				AssetAdded.DurationSec = DurationSec;
				AssetAdded.NumAdded = NumAdded;
				AssetAdded.CollectionShareType = CollectionItem->CollectionType;
				AssetAdded.Workflow = ECollectionTelemetryAssetAddedWorkflow::DragAndDrop;
				FTelemetryRouter::Get().ProvideTelemetry(AssetAdded);
			}
		}

		// Added items to the collection or failed. Either way, display the message.
		ContentBrowserUtils::DisplayMessage(Message, Geometry.GetLayoutBoundingRect(), SharedThis(this), MessageType);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SCollectionView::HandleSettingChanged(FName PropertyName)
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UContentBrowserSettings, bDisplayExcludedCollections))
	{
		UpdateFilteredCollectionItems();
	}
}

void SCollectionView::ExpandParentItems(const TSharedRef<FCollectionItem>& InCollectionItem)
{
	for (TSharedPtr<FCollectionItem> CollectionItemToExpand = InCollectionItem->ParentCollection.Pin(); 
		CollectionItemToExpand.IsValid(); 
		CollectionItemToExpand = CollectionItemToExpand->ParentCollection.Pin()
		)
	{
		CollectionTreePtr->SetItemExpansion(CollectionItemToExpand, true);
	}
}

TSharedPtr<SWidget> SCollectionView::MakeCollectionTreeContextMenu()
{
	if ( !bAllowRightClickMenu )
	{
		return NULL;
	}

	return CollectionContextMenu->MakeCollectionTreeContextMenu(Commands);
}

bool SCollectionView::IsCollectionCheckBoxEnabled( TSharedPtr<FCollectionItem> CollectionItem ) const
{
	return QuickAssetManagement.IsValid() && QuickAssetManagement->IsCollectionEnabled(FCollectionNameType(CollectionItem->CollectionName, CollectionItem->CollectionType));
}

ECheckBoxState SCollectionView::IsCollectionChecked( TSharedPtr<FCollectionItem> CollectionItem ) const
{
	if ( QuickAssetManagement.IsValid() )
	{
		return QuickAssetManagement->GetCollectionCheckState(FCollectionNameType(CollectionItem->CollectionName, CollectionItem->CollectionType));
	}
	return ECheckBoxState::Unchecked;
}

void SCollectionView::OnCollectionCheckStateChanged( ECheckBoxState NewState, TSharedPtr<FCollectionItem> CollectionItem )
{
	if ( QuickAssetManagement.IsValid() )
	{
		FCollectionNameType CollectionKey(CollectionItem->CollectionName, CollectionItem->CollectionType);
		if (QuickAssetManagement->GetCollectionCheckState(CollectionKey) == ECheckBoxState::Checked)
		{
			QuickAssetManagement->RemoveCurrentAssetsFromCollection(CollectionKey);
		}
		else
		{
			QuickAssetManagement->AddCurrentAssetsToCollection(CollectionKey);
		}
	}
}

void SCollectionView::CollectionSelectionChanged( TSharedPtr< FCollectionItem > CollectionItem, ESelectInfo::Type /*SelectInfo*/ )
{
	if ( ShouldAllowSelectionChangedDelegate() && OnCollectionSelected.IsBound() )
	{
		if ( CollectionItem.IsValid() )
		{
			OnCollectionSelected.Execute(FCollectionNameType(CollectionItem->CollectionName, CollectionItem->CollectionType));
		}
		else
		{
			OnCollectionSelected.Execute(FCollectionNameType(NAME_None, ECollectionShareType::CST_All));
		}
	}
}

void SCollectionView::CollectionItemScrolledIntoView( TSharedPtr<FCollectionItem> CollectionItem, const TSharedPtr<ITableRow>& Widget )
{
	if ( CollectionItem->bRenaming && Widget.IsValid() && Widget->GetContent().IsValid() )
	{
		CollectionItem->OnRenamedRequestEvent.Broadcast();
	}
}

bool SCollectionView::IsCollectionNameReadOnly() const
{
	// We can't rename collections while they're being dragged
	TSharedPtr<FCollectionDragDropOp> DragDropOp = CurrentCollectionDragDropOp.Pin();
	if (DragDropOp.IsValid())
	{
		TArray<TSharedPtr<FCollectionItem>> SelectedCollectionItems = CollectionTreePtr->GetSelectedItems();
		for (const auto& SelectedCollectionItem : SelectedCollectionItems)
		{
			if (DragDropOp->CollectionRefs.ContainsByPredicate([&SelectedCollectionItem](const FCollectionRef& Collection)
				{
					return SelectedCollectionItem->CollectionContainer == Collection.Container &&
						SelectedCollectionItem->CollectionName == Collection.Name &&
						SelectedCollectionItem->CollectionType == Collection.Type;
				}))
			{
				return true;
			}
		}
	}

	CollectionContextMenu->UpdateProjectSourceControl();
	return !CollectionContextMenu->CanRenameSelectedCollections();
}

bool SCollectionView::CollectionNameChangeCommit( const TSharedPtr< FCollectionItem >& CollectionItem, const FString& NewName, bool bChangeConfirmed, FText& OutWarningMessage )
{
	if (!ensure(CollectionItem->CollectionContainer))
	{
		return false;
	}

	// If new name is empty, set it back to the original name
	const FName NewNameFinal( NewName.IsEmpty() ? CollectionItem->CollectionName : FName(*NewName) );

	if ( CollectionItem->bNewCollection )
	{
		CollectionItem->bNewCollection = false;
		
		// Cache this here as CreateCollection will invalidate the current parent pointer
		TOptional<FCollectionNameType> NewCollectionParentKey;
		TSharedPtr<FCollectionItem> ParentCollectionItem = CollectionItem->ParentCollection.Pin();
		if ( ParentCollectionItem.IsValid() )
		{
			NewCollectionParentKey = FCollectionNameType(ParentCollectionItem->CollectionName, ParentCollectionItem->CollectionType);
		}

		double BeginTimeSec = FPlatformTime::Seconds();

		// If we canceled the name change when creating a new asset, we want to silently remove it
		if ( !bChangeConfirmed )
		{
			AvailableCollections.Remove(FCollectionNameType(CollectionItem->CollectionName, CollectionItem->CollectionType));
			UpdateFilteredCollectionItems();
			return false;
		}

		FText Error;
		if ( !CollectionItem->CollectionContainer->CreateCollection(NewNameFinal, CollectionItem->CollectionType, CollectionItem->StorageMode, &Error) )
		{
			// Failed to add the collection, remove it from the list
			AvailableCollections.Remove(FCollectionNameType(CollectionItem->CollectionName, CollectionItem->CollectionType));
			UpdateFilteredCollectionItems();

			OutWarningMessage = FText::Format( LOCTEXT("CreateCollectionFailed", "Failed to create the collection. {0}"), Error);
			return false;
		}

		// Since we're really adding a new collection (as our placeholder item is currently transient), we don't get a rename event from the collections manager
		// We'll spoof one here that so that our placeholder tree item is updated with the final name - this will preserve its expansion and selection state
		HandleCollectionRenamed(
			*CollectionItem->CollectionContainer,
			FCollectionNameType(CollectionItem->CollectionName, CollectionItem->CollectionType), 
			FCollectionNameType(NewNameFinal, CollectionItem->CollectionType)
			);

		if ( NewCollectionParentKey.IsSet() )
		{
			// Try and set the parent correctly (if this fails for any reason, the collection will still be added, but will just appear at the root)
			CollectionItem->CollectionContainer->ReparentCollection(NewNameFinal, CollectionItem->CollectionType, NewCollectionParentKey->Name, NewCollectionParentKey->Type);
		}

		// Notify anything that cares that this collection has been created now
		if ( CollectionItem->OnCollectionCreatedEvent.IsBound())
		{
			CollectionItem->OnCollectionCreatedEvent.Execute(FCollectionNameType(NewNameFinal, CollectionItem->CollectionType));
			CollectionItem->OnCollectionCreatedEvent.Unbind();
		}
		
		{
			FCollectionCreatedTelemetryEvent CollectionCreatedEvent;
			CollectionCreatedEvent.DurationSec = FPlatformTime::Seconds() - BeginTimeSec;
			CollectionCreatedEvent.CollectionShareType = CollectionItem->CollectionType;
			FTelemetryRouter::Get().ProvideTelemetry(CollectionCreatedEvent);
		}
	}
	else
	{
		// If the old name is the same as the new name, just early exit here.
		if ( CollectionItem->CollectionName == NewNameFinal )
		{
			return true;
		}

		// If the new name doesn't pass our current filter, we need to clear it
		if (ExternalSearchPtr.IsValid() && !CollectionItemTextFilter->PassesFilter(FCollectionItem(CollectionItem->CollectionContainer.ToSharedRef(), NewNameFinal, CollectionItem->CollectionType)))
		{
			ExternalSearchPtr->ClearSearch();
		}

		// Otherwise perform the rename
		FText Error;
		if ( !CollectionItem->CollectionContainer->RenameCollection(CollectionItem->CollectionName, CollectionItem->CollectionType, NewNameFinal, CollectionItem->CollectionType, &Error) )
		{
			// Failed to rename the collection
			OutWarningMessage = FText::Format( LOCTEXT("RenameCollectionFailed", "Failed to rename the collection. {0}"), Error);
			return false;
		}
	}

	// At this point CollectionItem is no longer a member of the CollectionItems list (as the list is repopulated by
	// UpdateCollectionItems, which is called by a broadcast from CollectionManagerModule::RenameCollection, above).
	// So search again for the item by name and type.
	auto NewCollectionItemPtr = AvailableCollections.Find( FCollectionNameType(NewNameFinal, CollectionItem->CollectionType) );

	// Reselect the path to notify that the selection has changed
	{
		FScopedPreventSelectionChangedDelegate DelegatePrevention( SharedThis(this) );
		CollectionTreePtr->ClearSelection();
	}

	// Set the selection
	if (NewCollectionItemPtr)
	{
		const auto& NewCollectionItem = *NewCollectionItemPtr;
		CollectionTreePtr->RequestScrollIntoView(NewCollectionItem);
		CollectionTreePtr->SetItemSelection(NewCollectionItem, true);
	}

	return true;
}

bool SCollectionView::CollectionVerifyRenameCommit(const TSharedPtr< FCollectionItem >& CollectionItem, const FString& NewName, const FSlateRect& MessageAnchor, FText& OutErrorMessage)
{
	// If the new name is the same as the old name, consider this to be unchanged, and accept it.
	if (CollectionItem->CollectionName.ToString() == NewName)
	{
		return true;
	}

	if (!ensure(CollectionItem->CollectionContainer) || !CollectionItem->CollectionContainer->IsValidCollectionName(NewName, ECollectionShareType::CST_All, &OutErrorMessage))
	{
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
