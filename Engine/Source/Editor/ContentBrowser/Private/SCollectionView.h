// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CollectionManagerTypes.h"
#include "CollectionViewTypes.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Misc/TextFilter.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateConstants.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

class FCollectionAssetManagement;
class FDragDropEvent;
class FMenuBuilder;
class FName;
class FSlateRect;
class FSourcesSearch;
class FUICommandList;
class ICollectionContainer;
class ITableRow;
class SHorizontalBox;
class SWidget;
struct FGeometry;
struct FHistoryData;
struct FKeyEvent;
struct FPointerEvent;
struct FSlateBrush;

template<>
struct TWidgetTypeTraits<class SCollectionView>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * The list view of collections.
 */
class SCollectionView
	: public SCompoundWidget
	, public IScrollableWidget
{
	friend class FCollectionContextMenu;
public:
	DECLARE_DELEGATE_OneParam( FOnCollectionSelected, const FCollectionNameType& /*SelectedCollection*/);

	SLATE_BEGIN_ARGS( SCollectionView )
		: _AllowCollectionButtons(true)
		, _AllowRightClickMenu(true)
		, _AllowContextMenu(true)
		, _IsDocked(false)
		, _AllowCollectionDrag(false)
		, _AllowQuickAssetManagement(false)
		, _ShowSeparator(true)
		{}

		/** Called when a collection was selected */
		SLATE_EVENT( FOnCollectionSelected, OnCollectionSelected )

		/** If true, collection buttons will be displayed */
		SLATE_ARGUMENT( bool, AllowCollectionButtons )
		SLATE_ARGUMENT( bool, AllowRightClickMenu )
		SLATE_ARGUMENT( bool, AllowContextMenu )
		SLATE_ATTRIBUTE( bool, IsDocked)

		/** If true, the user will be able to drag collections from this view */
		SLATE_ARGUMENT( bool, AllowCollectionDrag )

		/** If true, check boxes that let you quickly add/remove the current selection from a collection will be displayed */
		SLATE_ARGUMENT( bool, AllowQuickAssetManagement )

		/** If true, The tree search bar separator be displayed */
		SLATE_ARGUMENT( bool, ShowSeparator )

		/** The collection container to view */
		SLATE_ARGUMENT (TSharedPtr<ICollectionContainer>, CollectionContainer )

		/** Optional external search. Will hide and replace our internal search UI */
		SLATE_ARGUMENT( TSharedPtr<FSourcesSearch>, ExternalSearch )

	SLATE_END_ARGS()

	~SCollectionView();

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

	/** Returns true if there are no collections being presented in this widget. */
	bool IsEmpty();

	/** Selects the specified collections */
	void SetSelectedCollections(const TArray<FCollectionNameType>& CollectionsToSelect, const bool bEnsureVisible = true);

	/** Expands the specified collections */
	void SetExpandedCollections(const TArray<FCollectionNameType>& CollectionsToExpand);

	/** Clears selection of all collections */
	void ClearSelection();

	/** Gets all the collection container this widget was constructed with */
	const TSharedPtr<ICollectionContainer>& GetCollectionContainer() const;

	/** Gets all the currently selected collections */
	TArray<FCollectionNameType> GetSelectedCollections() const;

	/** Let the collections view know that the list of selected assets has changed, so that it can update the quick asset management check boxes */
	void SetSelectedAssetPaths(const TArray<FSoftObjectPath>& SelectedAssets);

	/** Sets the state of the collection view to the one described by the history data */
	void ApplyHistoryData ( const FHistoryData& History );

	/** Saves any settings to config that should be persistent between editor sessions */
	void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const;

	/** Loads any settings to config that should be persistent between editor sessions */
	void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString);

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	/** Creates the menu for the save dynamic collection button */
	void MakeSaveDynamicCollectionMenu(FMenuBuilder& InMenuBuilder, FText InQueryString);

	/** Creates the menu for the add collection button */
	FReply OnAddCollectionClicked();
	void MakeAddCollectionMenu(TSharedRef<SWidget> MenuParent);

	//~ Begin IScrollableWidget
	virtual FVector2D GetScrollDistance() override;
	virtual FVector2D GetScrollDistanceRemaining() override;
	virtual TSharedRef<SWidget> GetScrollWidget() override;
	//~ End IScrollableWidget

private:

	/** Payload data for CreateCollectionItem (likely from a delegate binding) */
	struct FCreateCollectionPayload
	{
		FCreateCollectionPayload()
			: ParentCollection()
			, OnCollectionCreatedEvent()
		{
		}

		explicit FCreateCollectionPayload(TOptional<FCollectionNameType> InParentCollection)
			: ParentCollection(InParentCollection)
			, OnCollectionCreatedEvent()
		{
		}

		explicit FCreateCollectionPayload(FCollectionItem::FCollectionCreatedEvent InCollectionCreatedEvent)
			: ParentCollection()
			, OnCollectionCreatedEvent(InCollectionCreatedEvent)
		{
		}

		FCreateCollectionPayload(TOptional<FCollectionNameType> InParentCollection, FCollectionItem::FCollectionCreatedEvent InCollectionCreatedEvent)
			: ParentCollection(InParentCollection)
			, OnCollectionCreatedEvent(InCollectionCreatedEvent)
		{
		}

		/** Should this collection be created as a child of another collection? */
		TOptional<FCollectionNameType> ParentCollection;

		/** Callback for after the collection has been fully created (delayed due to user naming, and potential cancellation) */
		FCollectionItem::FCollectionCreatedEvent OnCollectionCreatedEvent;
	};

	/** True if the selection changed delegate is allowed at the moment */
	bool ShouldAllowSelectionChangedDelegate() const;

	/** Sets up an inline creation process for a new collection of the specified type */
	void CreateCollectionItem( ECollectionShareType::Type CollectionType, ECollectionStorageMode::Type StorageMode, const FCreateCollectionPayload& InCreationPayload );

	/** Sets up an inline rename for the specified collection */
	void RenameCollectionItem( const TSharedPtr<FCollectionItem>& ItemToRename );

	/** Delete the given collections */
	void DeleteCollectionItems( const TArray<TSharedPtr<FCollectionItem>>& ItemsToDelete );

	/** Returns the visibility of the collection tree */
	EVisibility GetCollectionTreeVisibility() const;

	/** Returns the visibility of the collection tree header. Collapsed if the collection tree is docked */
	EVisibility GetHeaderVisibility() const;

	/** Returns the visibility of the add collection button */
	EVisibility GetAddCollectionButtonVisibility() const;

	/** Get the border of the collection tree */
	const FSlateBrush* GetCollectionViewDropTargetBorder() const;

	/** Creates a list item for the collection tree */
	TSharedRef<ITableRow> GenerateCollectionRow( TSharedPtr<FCollectionItem> CollectionItem, const TSharedRef<STableViewBase>& OwnerTable );

	/** Get the tree view children for the given item */
	void GetCollectionItemChildren( TSharedPtr<FCollectionItem> InParentItem, TArray< TSharedPtr<FCollectionItem> >& OutChildItems ) const;

	/** Handle starting a drag and drop operation for the currently selected collections */
	FReply OnCollectionDragDetected(const FGeometry& Geometry, const FPointerEvent& MouseEvent);

	/** Validate a drag drop operation on our collection tree */
	bool ValidateDragDropOnCollectionTree(const FGeometry& Geometry, const FDragDropEvent& DragDropEvent, bool& OutIsKnownDragOperation);

	/** Handle dropping something on collection tree */
	FReply HandleDragDropOnCollectionTree(const FGeometry& Geometry, const FDragDropEvent& DragDropEvent);

	/** Validate a drag drop operation on one of our collection items */
	bool ValidateDragDropOnCollectionItem(TSharedRef<FCollectionItem> CollectionItem, const FGeometry& Geometry, const FDragDropEvent& DragDropEvent, bool& OutIsKnownDragOperation);

	/** Handle dropping something on one of our collections */
	FReply HandleDragDropOnCollectionItem(TSharedRef<FCollectionItem> CollectionItem, const FGeometry& Geometry, const FDragDropEvent& DragDropEvent);

	void HandleSettingChanged(FName PropertyName);

	/** Recursively expand the parent items of this collection to ensure that it is visible */
	void ExpandParentItems(const TSharedRef<FCollectionItem>& InCollectionItem);

	/** Makes the context menu for the collection tree */
	TSharedPtr<SWidget> MakeCollectionTreeContextMenu();

	/** Whether the check box of the given collection item is currently enabled */
	bool IsCollectionCheckBoxEnabled( TSharedPtr<FCollectionItem> CollectionItem ) const;

	/** Whether the check box of the given collection item is currently in a checked state */
	ECheckBoxState IsCollectionChecked( TSharedPtr<FCollectionItem> CollectionItem ) const;

	/** Handler for when the checked state of the given collection item check box is changed */
	void OnCollectionCheckStateChanged( ECheckBoxState NewState, TSharedPtr<FCollectionItem> CollectionItem );

	/** Handler for collection list selection changes */
	void CollectionSelectionChanged( TSharedPtr< FCollectionItem > CollectionItem, ESelectInfo::Type SelectInfo );

	/** Handles focusing a collection item widget after it has been created with the intent to rename */
	void CollectionItemScrolledIntoView( TSharedPtr<FCollectionItem> CollectionItem, const TSharedPtr<ITableRow>& Widget );

	/** Checks whether the selected collection is not allowed to be renamed */
	bool IsCollectionNameReadOnly() const;

	/** Handler for when a name was given to a collection. Returns false if the rename or create failed and sets OutWarningMessage depicting what happened. */
	bool CollectionNameChangeCommit( const TSharedPtr< FCollectionItem >& CollectionItem, const FString& NewName, bool bChangeConfirmed, FText& OutWarningMessage );

	/** Checks if the collection name being committed is valid */
	bool CollectionVerifyRenameCommit(const TSharedPtr< FCollectionItem >& CollectionItem, const FString& NewName, const FSlateRect& MessageAnchor, FText& OutErrorMessage);

	/** Handles an on collection created event */
	void HandleCollectionCreated( ICollectionContainer& InCollectionContainer, const FCollectionNameType& Collection );

	/** Handles an on collection renamed event */
	void HandleCollectionRenamed( ICollectionContainer& InCollectionContainer, const FCollectionNameType& OriginalCollection, const FCollectionNameType& NewCollection );

	/** Handles an on collection reparented event */
	void HandleCollectionReparented( ICollectionContainer& InCollectionContainer, const FCollectionNameType& Collection, const TOptional<FCollectionNameType>& OldParent, const TOptional<FCollectionNameType>& NewParent );

	/** Handles an on collection destroyed event */
	void HandleCollectionDestroyed( ICollectionContainer& InCollectionContainer, const FCollectionNameType& Collection );

	/** Handles an on collection updated event */
	void HandleCollectionUpdated( ICollectionContainer& InCollectionContainer, const FCollectionNameType& Collection );

	/** Handles assets being added to a collection */
	void HandleAssetsAddedToCollection( ICollectionContainer& InCollectionContainer, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> AssetsAdded );

	/** Handles assets being removed from a collection */
	void HandleAssetsRemovedFromCollection( ICollectionContainer& InCollectionContainer, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> AssetsRemoved );

	/** Handles the source control provider changing */
	void HandleSourceControlProviderChanged(class ISourceControlProvider& OldProvider, class ISourceControlProvider& NewProvider);

	/** Handles the source control state changing */
	void HandleSourceControlStateChanged();

	/** Handles updating the status of the given collection item */
	static void UpdateCollectionItemStatus( const TSharedRef<FCollectionItem>& CollectionItem );

	/** Updates the collections shown in the tree view */
	void UpdateCollectionItems();

	/** Update the visible collections based on the active filter text */
	void UpdateFilteredCollectionItems();

	/** Set the active filter text */
	void SetCollectionsSearchFilterText( const FText& InSearchText, TArray<FText>& OutErrors );

	/** Get the active filter text */
	FText GetCollectionsSearchFilterText() const;

private:

	/** A helper class to manage PreventSelectionChangedDelegateCount by incrementing it when constructed (on the stack) and decrementing when destroyed */
	class FScopedPreventSelectionChangedDelegate
	{
	public:
		FScopedPreventSelectionChangedDelegate(const TSharedRef<SCollectionView>& InCollectionView)
			: CollectionView(InCollectionView)
		{
			CollectionView->PreventSelectionChangedDelegateCount++;
		}

		~FScopedPreventSelectionChangedDelegate()
		{
			check(CollectionView->PreventSelectionChangedDelegateCount > 0);
			CollectionView->PreventSelectionChangedDelegateCount--;
		}

	private:
		TSharedRef<SCollectionView> CollectionView;
	};

	/** The collection container to view */
	TSharedPtr<ICollectionContainer> CollectionContainer;

	/** The collection external search interface (if any) */
	TSharedPtr<FSourcesSearch> ExternalSearchPtr;

	/** The collection title content */
	TSharedPtr<SHorizontalBox> TitleContent;

	/** The collection tree widget */
	TSharedPtr< STreeView< TSharedPtr<FCollectionItem> > > CollectionTreePtr;

	/** A map of collection keys to their associated collection items - this map contains all available collections, even those that aren't currently visible */
	typedef TMap< FCollectionNameType, TSharedPtr<FCollectionItem> > FAvailableCollectionsMap;
	FAvailableCollectionsMap AvailableCollections;

	/** A set of collections that are currently visible, including parents that are only visible due to their children */
	TSet< FCollectionNameType > VisibleCollections;

	/** The list of root items to show in the collections tree - this will be filtered as required */
	TArray< TSharedPtr<FCollectionItem> > VisibleRootCollectionItems;

	/** The filter to apply to the available collections */
	typedef TTextFilter<const FCollectionItem&> FCollectionItemTextFilter;
	TSharedPtr< FCollectionItemTextFilter > CollectionItemTextFilter;

	/** The context menu logic and data */
	TSharedPtr<class FCollectionContextMenu> CollectionContextMenu;

	/** Delegate to invoke when selection changes. */
	FOnCollectionSelected OnCollectionSelected;

	/** Whether or not the collection view is docked under the sources panel */
	TAttribute<bool> IsDocked;

	/** If true, collection buttons (such as add) are allowed */
	bool bAllowCollectionButtons;

	/** If true, the user will be able to access the right click menu of a collection */
	bool bAllowRightClickMenu;

	/** If true, the user will be able to drag collections from this view */
	bool bAllowCollectionDrag;

	/** If true, we will use the external search if provided during construction */
	bool bAllowExternalSearch;

	/** True when a drag is over this view with a valid operation for drop */
	bool bDraggedOver;

	/** If > 0, the selection changed delegate will not be called. Used to update the tree from an external source or in certain bulk operations. */
	int32 PreventSelectionChangedDelegateCount;

	/** Commands handled by this widget */
	TSharedPtr< FUICommandList > Commands;

	/** Handles the collection management for the currently selected assets (if available) */
	TSharedPtr<FCollectionAssetManagement> QuickAssetManagement;

	/**
	 * This is set after this view has initiated a drag and drop for some collections in our tree.
	 * We keep a weak pointer to this so we can tell if that drag and drop is still ongoing, and if so, what collections it affects 
	 */
	TWeakPtr<class FCollectionDragDropOp> CurrentCollectionDragDropOp;

	/** Delegate handle for the HandleSourceControlProviderChanged function callback */
	FDelegateHandle SourceControlProviderChangedHandle;

	/** Delegate handle for the HandleSourceControlStateChanged function callback */
	FDelegateHandle SourceControlStateChangedDelegateHandle;

	/** True if we should queue a collection items update for the next Tick */
	bool bQueueCollectionItemsUpdate;

	/** True if we should queue an SCC refresh for the collections on the next Tick */
	bool bQueueSCCRefresh;

	/** True if we need to update the source control state of items on next tick */
	bool bQueueItemStatusUpdate = false;
};
