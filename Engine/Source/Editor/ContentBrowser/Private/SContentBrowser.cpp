// Copyright Epic Games, Inc. All Rights Reserved.


#include "SContentBrowser.h"

#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "AssetContextMenu.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataTagMap.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetTextFilter.h"
#include "AssetThumbnail.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "CollectionViewUtils.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "ContentBrowserCommands.h"
#include "ContentBrowserConfig.h"
#include "ContentBrowserDataFilter.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserDataUtils.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserItemData.h"
#include "ContentBrowserItemPath.h"
#include "ContentBrowserLog.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "ContentBrowserSingleton.h"
#include "ContentBrowserStyle.h"
#include "ContentBrowserUtils.h"
#include "ContentBrowserVirtualPathTree.h"
#include "AssetViewContentSources.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "FileHelpers.h"
#include "Filters.h"
#include "Filters/FilterBase.h"
#include "Filters/SAssetFilterBar.h"
#include "Filters/SBasicFilterBar.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/Notifications/NotificationManager.h"
#include "FrontendFilters.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformCrt.h"
#include "IAddContentDialogModule.h"
#include "IAssetTools.h"
#include "ICollectionContainer.h"
#include "ICollectionSource.h"
#include "ICollectionManager.h"
#include "IContentBrowserDataModule.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/ChildrenBase.h"
#include "Layout/Clipping.h"
#include "Layout/Margin.h"
#include "Layout/WidgetPath.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/MessageLog.h"
#include "Logging/TokenizedMessage.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/App.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/CString.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ExpressionParserTypes.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FilterCollection.h"
#include "Misc/NamePermissionList.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Modules/ModuleManager.h"
#include "NewAssetOrClassContextMenu.h"
#include "PathContextMenu.h"
#include "SActionButton.h"
#include "SAssetSearchBox.h"
#include "SAssetView.h"
#include "SCollectionView.h"
#include "SFilterList.h"
#include "SNavigationBar.h"
#include "SPathView.h"
#include "SPositiveActionButton.h"
#include "SSearchToggleButton.h"
#include "Selection.h"
#include "Settings/ContentBrowserSettings.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "SourcesSearch.h"
#include "StatusBarSubsystem.h"
#include "String/Find.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleDefaults.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "Templates/TypeHash.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuEntry.h"
#include "ToolMenuMisc.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "ContentSources/Widgets/SContentSourcesView.h"
#include "ContentSources/Widgets/SLegacyContentSource.h"
#include "Experimental/ContentBrowserExtensionUtils.h"
#include "Experimental/ContentBrowserSearchOverrideInternal.h"
#include "Misc/EnumerateRange.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "Trace/Detail/Channel.h"
#include "Types/ISlateMetaData.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SContentBrowserSourceTree.h"
#include "Widgets/SInvalidationPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

class FTreeItem;
struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "ContentBrowser"

namespace UE::Editor::ContentBrowser::Private
{
	// Find and return the slot index containing a widget with the given tag. Will return INDEX_NONE if not found.
	static int32 FindSlotByWidgetTag(const TSharedRef<SSplitter>& InSplitter, const FName InTag)
	{
		for (int32 SlotIndex = 0; SlotIndex < InSplitter->GetChildren()->Num(); ++SlotIndex)
		{
			TSharedRef<SWidget> WidgetInSlot = InSplitter->SlotAt(SlotIndex).GetWidget();
			if (WidgetInSlot->GetTag() == InTag)
			{
				return SlotIndex;
			}
		}

		return INDEX_NONE;
	}
}

const FString SContentBrowser::SettingsIniSection = TEXT("ContentBrowser");

class SContentBrowser::FCollectionSource
{
public:
	FCollectionSource(SContentBrowser& InContentBrowser, const TSharedRef<ICollectionContainer>& InCollectionContainer)
	{
		CollectionSearch = MakeShared<FSourcesSearch>();
		CollectionSearch->Initialize();
		CollectionSearch->SetHintText(LOCTEXT("CollectionsViewSearchBoxHint", "Search Collections"));

		CollectionViewPtr = SNew(SCollectionView)
			.OnCollectionSelected_Lambda([ContentBrowser = StaticCastWeakPtr<SContentBrowser>(InContentBrowser.AsWeak()), CollectionContainer = InCollectionContainer](const FCollectionNameType& SelectedCollection)
			{
				if (TSharedPtr<SContentBrowser> PinnedContentBrowser = ContentBrowser.Pin())
				{
					PinnedContentBrowser->CollectionSelected(CollectionContainer, SelectedCollection);
				}
			})
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserCollections")))
			.AllowCollectionDrag(true)
			.AllowQuickAssetManagement(true)
			.CollectionContainer(InCollectionContainer)
			.IsDocked(true)
			.ExternalSearch(CollectionSearch);
	}

	FCollectionSource(const FCollectionSource&) = delete;

	FCollectionSource& operator=(const FCollectionSource&) = delete;

	const TSharedPtr<ICollectionContainer>& GetCollectionContainer() const
	{
		return CollectionViewPtr->GetCollectionContainer();
	}

	bool IsProjectCollectionContainer() const
	{
		return GetCollectionContainer() == FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();
	}

	void LoadSettings(const FName& InInstanceName)
	{
		const FString SettingsString = GetSettingsString(InInstanceName);
		const FString EditorPerProjectIni = GetCollectionContainer()->GetCollectionSource()->GetEditorPerProjectIni();

		CollectionViewPtr->LoadSettings(EditorPerProjectIni, SettingsIniSection, SettingsString);
		CollectionArea->LoadSettings(EditorPerProjectIni, SettingsIniSection, SettingsString);
	}

	void SaveSettings(const FName& InInstanceName) const
	{
		const FString SettingsString = GetSettingsString(InInstanceName);
		const FString EditorPerProjectIni = GetCollectionContainer()->GetCollectionSource()->GetEditorPerProjectIni();

		CollectionViewPtr->SaveSettings(EditorPerProjectIni, SettingsIniSection, SettingsString);
		CollectionArea->SaveSettings(EditorPerProjectIni, SettingsIniSection, SettingsString);
	}

private:
	FString GetSettingsString(const FName& InInstanceName) const
	{
		if (IsProjectCollectionContainer())
		{
			// Maintain backwards compatibility with previous version of the SContentBrowser which had a single collection view.
			return InInstanceName.ToString();
		}
		else
		{
			return InInstanceName.ToString() + TEXT(".") + GetCollectionContainer()->GetCollectionSource()->GetName().ToString();
		}
	}

public:
	/** The sources search for collections*/
	TSharedPtr<FSourcesSearch> CollectionSearch;

	/** The collection widget */
	TSharedPtr<SCollectionView> CollectionViewPtr;

	/** Collection area widget */
	TSharedPtr<UE::Editor::ContentBrowser::Private::SContentBrowserSourceTreeArea> CollectionArea;
};

SContentBrowser::SContentBrowser() = default;

SContentBrowser::~SContentBrowser()
{
	IConsoleManager::Get().UnregisterConsoleVariableSink_Handle(CVarSinkHandle);

	// Remove the listener for when view settings are changed
	UContentBrowserSettings::OnSettingChanged().RemoveAll( this );

	// Remove listeners for when collections/paths are renamed/deleted
	if (FCollectionManagerModule::IsModuleAvailable())
	{
		FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();

		TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
		CollectionManagerModule.Get().GetCollectionContainers(CollectionContainers);

		for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
		{
			CollectionContainer->OnIsHiddenChanged().RemoveAll(this);
			CollectionContainer->OnCollectionRenamed().RemoveAll(this);
			CollectionContainer->OnCollectionDestroyed().RemoveAll(this);
			CollectionContainer->OnCollectionUpdated().RemoveAll(this);
		}

		CollectionManagerModule.Get().OnCollectionContainerCreated().RemoveAll(this);
		CollectionManagerModule.Get().OnCollectionContainerDestroyed().RemoveAll(this);
	}

	if (IContentBrowserDataModule* ContentBrowserDataModule = IContentBrowserDataModule::GetPtr())
	{
		if (UContentBrowserDataSubsystem* ContentBrowserData = ContentBrowserDataModule->GetSubsystem())
		{
			ContentBrowserData->OnItemDataUpdated().RemoveAll(this);
		}
	}

	if (bIsPrimaryBrowser && GEditor)
	{
		if (USelection* EditorSelection = GEditor->GetSelectedObjects())
		{
			EditorSelection->DeselectAll();
		}
	}
}

bool SContentBrowser::ShouldShowRedirectors() const
{
	return LegacyContentSourceWidgets->FilterListPtr.IsValid() ? ContentBrowserUtils::ShouldShowRedirectors(LegacyContentSourceWidgets->FilterListPtr) : false;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SContentBrowser::Construct( const FArguments& InArgs, const FName& InInstanceName, const FContentBrowserConfig* Config )
{
	using namespace UE::Editor::ContentBrowser::Private;

	InstanceName = InInstanceName;

	// Store a copy of the init config if specified so we can re-create the asset view widgets dynamically
	if (Config)
	{
		// We have to disable deprecation warnings because the default assignment operator copies a deprecated member
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		InitConfig = *Config;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		bHasInitConfig = true;
	}

	JumpMRU.MaxItems = 30;

	// Init favorites settings object
	UContentBrowserFavoriteProjectSettings::Get();

	UContentBrowserConfig::Initialize();
	UContentBrowserConfig::Get()->LoadEditorConfig();
	const FContentBrowserInstanceConfig* EditorConfig = CreateEditorConfigIfRequired();

	if ( InArgs._ContainingTab.IsValid() )
	{
		// For content browsers that are placed in tabs, save settings when the tab is closing.
		ContainingTab = InArgs._ContainingTab;
		InArgs._ContainingTab->SetOnPersistVisualState( SDockTab::FOnPersistVisualState::CreateSP( this, &SContentBrowser::OnContainingTabSavingVisualState ) );
		InArgs._ContainingTab->SetOnTabClosed( SDockTab::FOnTabClosedCallback::CreateSP( this, &SContentBrowser::OnContainingTabClosed ) );
		InArgs._ContainingTab->SetOnTabActivated( SDockTab::FOnTabActivatedCallback::CreateSP( this, &SContentBrowser::OnContainingTabActivated ) );
	}
	
	LegacyContentSource = SNew(UE::Editor::ContentBrowser::SLegacyContentSource);

 	bIsLocked = InArgs._InitiallyLocked;
	bCanSetAsPrimaryBrowser = Config != nullptr ? Config->bCanSetAsPrimaryBrowser : true;
	bIsDrawer = InArgs._IsDrawer;

	HistoryManager.SetOnApplyHistoryData(FOnApplyHistoryData::CreateSP(this, &SContentBrowser::OnApplyHistoryData));
	HistoryManager.SetOnUpdateHistoryData(FOnUpdateHistoryData::CreateSP(this, &SContentBrowser::OnUpdateHistoryData));

	FrontendFilters = MakeShareable(new FAssetFilterCollectionType());
	TextFilter = MakeShared<FAssetTextFilter>();

	PluginPathFilters = MakeShareable(new FPluginFilterCollectionType());

	FavoritesSearch = MakeShared<FSourcesSearch>();
	FavoritesSearch->Initialize();
	FavoritesSearch->SetHintText(LOCTEXT("SearchFavoritesHint", "Search Favorites"));

	SourcesSearch = MakeShared<FSourcesSearch>();
	SourcesSearch->Initialize();
	SourcesSearch->SetHintText(LOCTEXT("SearchPathsHint", "Search Paths"));

	static const FName DefaultForegroundName("DefaultForeground");
	
	UContentBrowserSettings::OnSettingChanged().AddSP(this, &SContentBrowser::OnContentBrowserSettingsChanged);
	
	// Register console variable sink for private content setting changing
	CVarSinkHandle = IConsoleManager::Get().RegisterConsoleVariableSink_Handle(FConsoleCommandDelegate::CreateSP(this, &SContentBrowser::OnConsoleVariableChanged));
	UpdatePrivateContentFeatureEnabled(false /* bUpdateFilterIfChanged */);

	ChildSlot
	[
		// The legacy content source will be activated by default, which will call OnLegacyContentSourceEnabled which ends up creating the asset view
		// widgets and initiatializing all settings and so on
		SAssignNew(ContentSourcesContainer, UE::Editor::ContentBrowser::SContentSourcesView)
			.LegacyContentSource(LegacyContentSource)
			.OnLegacyContentSourceEnabled(this, &SContentBrowser::OnLegacyContentSourceEnabled)
			.OnLegacyContentSourceDisabled(this, &SContentBrowser::OnLegacyContentSourceDisabled)
		
	];

	ExtendViewOptionsMenu(Config);
	
	// Set the initial history data
	HistoryManager.AddHistoryData();

	// We want to be able to search the feature packs in the super search so we need the module loaded 
	IAddContentDialogModule& AddContentDialogModule = FModuleManager::LoadModuleChecked<IAddContentDialogModule>("AddContentDialog");

	// Update the breadcrumb trail path
	OnContentBrowserSettingsChanged(NAME_None);
}

void SContentBrowser::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	SCompoundWidget::OnDragLeave(DragDropEvent);
	// Always dismiss the ContentDrawer if the drag leave the ContentBrowser
	if (bIsDrawer)
	{
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->DismissContentBrowserDrawer();
	}
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SContentBrowser::BindCommands()
{
	Commands = TSharedPtr< FUICommandList >(new FUICommandList);

	FContentBrowserCommands& ContentBrowserCommands = FContentBrowserCommands::Get();
	FGenericCommands& GenericCommands = FGenericCommands::Get();

	Commands->MapAction(GenericCommands.Rename, FUIAction(
		FExecuteAction::CreateSP(this, &SContentBrowser::HandleRenameCommand),
		FCanExecuteAction::CreateSP(this, &SContentBrowser::HandleRenameCommandCanExecute)
	));

	Commands->MapAction(GenericCommands.Delete, FUIAction(
		FExecuteAction::CreateSP(this, &SContentBrowser::HandleDeleteCommandExecute),
		FCanExecuteAction::CreateSP(this, &SContentBrowser::HandleDeleteCommandCanExecute)
	));

	Commands->MapAction(ContentBrowserCommands.OpenAssetsOrFolders, FUIAction(
		FExecuteAction::CreateSP(this, &SContentBrowser::HandleOpenAssetsOrFoldersCommandExecute)
	));

	Commands->MapAction(ContentBrowserCommands.PreviewAssets, FUIAction(
		FExecuteAction::CreateSP(this, &SContentBrowser::HandlePreviewAssetsCommandExecute)
	));

	Commands->MapAction(ContentBrowserCommands.CreateNewFolder, FUIAction(
		FExecuteAction::CreateSP(this, &SContentBrowser::HandleCreateNewFolderCommandExecute)
	));

	Commands->MapAction(ContentBrowserCommands.GoUpToParentFolder, FUIAction(
		FExecuteAction::CreateSP(this, &SContentBrowser::HandleGoUpToParentFolder),
		FCanExecuteAction::CreateSP(this, &SContentBrowser::HandleCanGoUpToParentFolder)
	));

	Commands->MapAction(ContentBrowserCommands.SaveSelectedAsset, FUIAction(
		FExecuteAction::CreateSP(this, &SContentBrowser::HandleSaveAssetCommand),
		FCanExecuteAction::CreateSP(this, &SContentBrowser::HandleSaveAssetCommandCanExecute)
	));

	Commands->MapAction(ContentBrowserCommands.SaveAllCurrentFolder, FUIAction(
		FExecuteAction::CreateSP(this, &SContentBrowser::HandleSaveAllCurrentFolderCommand)
	));

	Commands->MapAction(ContentBrowserCommands.ResaveAllCurrentFolder, FUIAction(
		FExecuteAction::CreateSP(this, &SContentBrowser::HandleResaveAllCurrentFolderCommand)
	));

	Commands->MapAction(ContentBrowserCommands.EditPath, FUIAction(
		FExecuteAction::CreateSP(this, &SContentBrowser::EditPathCommand)
	));

	Commands->MapAction(ContentBrowserCommands.FilterBySelection, FUIAction(
		FExecuteAction::CreateSP(this, &SContentBrowser::HandleFilterBySelectionCommandExecute),
		FCanExecuteAction::CreateSP(this, &SContentBrowser::HandleFilterBySelectionCommandCanExecute)
	));
	
	// Favorites shortcuts
	for (const TSharedPtr<FUICommandInfo>& FavoriteShortcut : ContentBrowserCommands.FavoriteShortcuts)
	{
		Commands->MapAction(
			FavoriteShortcut, 
			FUIAction(
				FExecuteAction::CreateSP(this, &SContentBrowser::HandleFavoriteShortcutCommandExecute, FavoriteShortcut->GetCommandName())
			)
		);
	}

	// Allow extenders to add commands
	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	TArray<FContentBrowserCommandExtender>& CommandExtenderDelegates = ContentBrowserModule.GetAllContentBrowserCommandExtenders();

	for (int32 i = 0; i < CommandExtenderDelegates.Num(); ++i)
	{
		if (CommandExtenderDelegates[i].IsBound())
		{
			CommandExtenderDelegates[i].Execute(Commands.ToSharedRef(), FOnContentBrowserGetSelection::CreateSP(this, &SContentBrowser::GetSelectionState));
		}
	}

	FInputBindingManager::Get().RegisterCommandList(FContentBrowserCommands::Get().GetContextName(), Commands.ToSharedRef());
}

void SContentBrowser::UnbindCommands()
{
	Commands.Reset();
}

EVisibility SContentBrowser::GetFavoriteFolderVisibility() const
{
	if (const FContentBrowserInstanceConfig* Config = GetConstInstanceConfig())
	{
		return Config->bShowFavorites ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return GetDefault<UContentBrowserSettings>()->GetDisplayFavorites() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SContentBrowser::GetLockButtonVisibility() const
{
	return IsLocked() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SContentBrowser::UpdateFolderFavorite(const TArray<FString>& FolderPaths, const FString& InFolder, EItemDropZone InPosition)
{
	const FContentBrowserItemPath TargetPath(InFolder, EContentBrowserPathType::Virtual);
	
	auto InsertOrMoveFolder = [&TargetPath, &InPosition](const FString& InPath)
		{
			const FContentBrowserItemPath ItemPath(InPath, EContentBrowserPathType::Virtual);
			if (ContentBrowserUtils::IsFavoriteFolder(ItemPath))
			{
				ContentBrowserUtils::MoveFavoriteFolder(ItemPath, TargetPath, InPosition);
			}
			else
			{
				ContentBrowserUtils::InsertFavoriteFolder(ItemPath, TargetPath, InPosition);
			}
		};

	// When dropping onto/below an item, insert in reverse to keep order
	if (InPosition == EItemDropZone::OntoItem || InPosition == EItemDropZone::BelowItem)
	{
		for (int32 Index = FolderPaths.Num() - 1; Index >= 0; --Index)
		{
			InsertOrMoveFolder(FolderPaths[Index]);
		}
	}
	else
	{
		for (const FString& FolderPath : FolderPaths)
		{
			InsertOrMoveFolder(FolderPath);
		}
	}
	
	SaveAndShowNewFolderFavorites(FolderPaths);
}

void SContentBrowser::ToggleFolderFavorite(const TArray<FString>& FolderPaths)
{
	TArray<FString> FolderPathsAdded;
	for (const FString& FolderPath : FolderPaths)
	{
		const FContentBrowserItemPath ItemPath(FolderPath, EContentBrowserPathType::Virtual);
		if (ContentBrowserUtils::IsFavoriteFolder(ItemPath))
		{
			ContentBrowserUtils::RemoveFavoriteFolder(ItemPath);
		}
		else
		{
			ContentBrowserUtils::AddFavoriteFolder(ItemPath);
			FolderPathsAdded.Add(FolderPath);
		}
	}

	SaveAndShowNewFolderFavorites(FolderPathsAdded);
}

void SContentBrowser::EditFolderFavorite(const FContentBrowserItemPath& InItemPath)
{
	if (const FContentBrowserFavoriteItem* FavoriteItem = ContentBrowserUtils::FindFavoriteFolder(InItemPath))
	{
		FContentBrowserSingleton::Get().OpenEditFavoriteItemWindow(*FavoriteItem);
	}
}

void SContentBrowser::SetFilterLayout(const EFilterBarLayout InFilterBarLayout) const
{
	if (const TSharedPtr<SFilterList>& FilterBar = LegacyContentSourceWidgets->FilterListPtr; FilterBar && LegacyContentSourceWidgets->AssetViewPtr)
	{
		FilterBar->SetFilterLayout(InFilterBarLayout);
		LegacyContentSourceWidgets->AssetViewPtr->SetFilterBar(FilterBar);
	}
	else
	{
		UE_LOGF(LogContentBrowser, Warning, "SetFilterLayout failed: %ls is invalid. FilterListPtr: %ls, AssetViewPtr: %ls",
		       TEXT("SContentBrowser::SetFilterLayout"),
		       LegacyContentSourceWidgets->FilterListPtr.IsValid() ? TEXT("Valid") : TEXT("Invalid"),
		       LegacyContentSourceWidgets->AssetViewPtr.IsValid() ? TEXT("Valid") : TEXT("Invalid"));
	}
}

EFilterBarLayout SContentBrowser::GetFilterLayout() const
{
	if (const TSharedPtr<SFilterList>& FilterBar = LegacyContentSourceWidgets->FilterListPtr)
	{
		return FilterBar->GetFilterLayout();
	}

	UE_LOGF(LogContentBrowser, Warning, "GetFilterLayout: FilterListPtr is invalid, returning default layout.");
	return EFilterBarLayout::Vertical;
}

TSharedPtr<SWidget> SContentBrowser::GetActiveFilterContainer() const
{
	if (const TSharedPtr<SFilterList>& FilterBar = LegacyContentSourceWidgets->FilterListPtr)
	{
		return FilterBar->GetActiveFilterContainer();
	}

	UE_LOGF(LogContentBrowser, Warning, "GetFilterLayout: FilterListPtr is invalid, returning nullptr.");
	return nullptr;
}

void SContentBrowser::SaveAndShowNewFolderFavorites(const TArray<FString>& FolderPaths)
{
	// If the legacy content source isn't active - the settings will get updated when it is made active
	if (!ContentSourcesContainer->IsLegacyContentSourceActive())
	{
		return;
	}
	
	LegacyContentSourceWidgets->FavoritePathViewPtr->SaveSettings(GEditorPerProjectIni, SettingsIniSection, InstanceName.ToString() + TEXT(".Favorites"));
	LegacyContentSourceWidgets->FavoritePathViewPtr->Populate();

	if (!FolderPaths.IsEmpty())
	{	
		LegacyContentSourceWidgets->FavoritePathViewPtr->SetSelectedPaths(FolderPaths);
		if (GetFavoriteFolderVisibility() == EVisibility::Collapsed)
		{
			UContentBrowserSettings* Settings = GetMutableDefault<UContentBrowserSettings>();
			Settings->SetDisplayFavorites(true);
			Settings->SaveConfig();
		}
	}
}

void SContentBrowser::TogglePrivateContentEdit(const TArray<FString>& FolderPaths)
{
	for (const FString& FolderPath : FolderPaths)
	{
		ensure(FContentBrowserSingleton::Get().IsFolderShowPrivateContentToggleable(FolderPath));

		if (FContentBrowserSingleton::Get().IsShowingPrivateContent(FolderPath))
		{
			ContentBrowserUtils::RemoveShowPrivateContentFolder(FolderPath, TEXT("ContentBrowser"));
		}
		else
		{
			ContentBrowserUtils::AddShowPrivateContentFolder(FolderPath, TEXT("ContentBrowser"));
		}
	}

	OnAssetViewRefreshRequested();
}

void SContentBrowser::HandleAssetViewSearchOptionsChanged()
{
	bool bIncludeClassName = LegacyContentSourceWidgets->AssetViewPtr->IsIncludingClassNames();
	bool bIncludeAssetPath = LegacyContentSourceWidgets->AssetViewPtr->IsIncludingAssetPaths();
	bool bIncludeCollectionNames = LegacyContentSourceWidgets->AssetViewPtr->IsIncludingCollectionNames();
	
	TextFilter->SetIncludeClassName(bIncludeClassName);
	TextFilter->SetIncludeAssetPath(bIncludeAssetPath);
	TextFilter->SetIncludeCollectionNames(bIncludeCollectionNames);

	// Make sure all custom text filters get the updated Asset View Search Options
	LegacyContentSourceWidgets->FilterListPtr->UpdateCustomTextFilterIncludes(bIncludeClassName, bIncludeAssetPath, bIncludeCollectionNames);
}

TSharedRef<SWidget> SContentBrowser::CreateToolBar(const FContentBrowserConfig* Config)
{
	FToolMenuContext MenuContext;

	UContentBrowserToolbarMenuContext* CommonContextObject = NewObject<UContentBrowserToolbarMenuContext>();
	CommonContextObject->ContentBrowser = SharedThis(this);
	CommonContextObject->AssetView = LegacyContentSourceWidgets->AssetViewPtr;
	CommonContextObject->ContentBrowserConfig = Config;

	MenuContext.AddObject(CommonContextObject);

	return UToolMenus::Get()->GenerateWidget("ContentBrowser.ToolBar", MenuContext);
}

TSharedRef<SWidget> SContentBrowser::CreateNavigationToolBar(const FContentBrowserConfig* Config)
{
	FToolMenuContext MenuContext;

	UContentBrowserToolbarMenuContext* CommonContextObject = NewObject<UContentBrowserToolbarMenuContext>();
	CommonContextObject->ContentBrowser = SharedThis(this);
	CommonContextObject->AssetView = LegacyContentSourceWidgets->AssetViewPtr;
	CommonContextObject->ContentBrowserConfig = Config;

	MenuContext.AddObject(CommonContextObject);

	return UToolMenus::Get()->GenerateWidget("ContentBrowser.NavigationBar", MenuContext);
}

TSharedRef<SWidget> SContentBrowser::CreateLockButton(const FContentBrowserConfig* Config)
{
	if(Config == nullptr || Config->bCanShowLockButton)
	{
		return
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("LockToggleTooltip", "Toggle lock. If locked, this browser will ignore Find in Content Browser requests."))
			.OnClicked(this, &SContentBrowser::ToggleLockClicked)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserLock")))
			.Visibility(this, &SContentBrowser::GetLockButtonVisibility)
			[
				SNew(SImage)
				.Image(this, &SContentBrowser::GetLockIconBrush)
				.ColorAndOpacity(FSlateColor::UseStyle())
			];
	}

	return SNullWidget::NullWidget;
}

void SContentBrowser::OnFilterBarLayoutChanging(EFilterBarLayout NewLayout)
{
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		using namespace UE::ContentBrowser::Private;
		
		// Identify filter view locations by widget tag, so we don't assume slot index
		static FName HorizontalFilterViewTagName = "HorizontalFilterView";
		static FName VerticalFilterViewTagName = "VerticalFilterView";
	
		if (NewLayout == EFilterBarLayout::Horizontal)
		{
			const int32 FoundVerticalFilterViewSlotIndex = UE::Editor::ContentBrowser::Private::FindSlotByWidgetTag(
				LegacyContentSourceWidgets->PathAssetSplitterPtr.ToSharedRef(),
				VerticalFilterViewTagName);
			
			// Remove from Vertical layout (if needed)
			if (FoundVerticalFilterViewSlotIndex != INDEX_NONE)
			{
				LegacyContentSourceWidgets->PathAssetSplitterPtr->RemoveAt(FoundVerticalFilterViewSlotIndex);
			}
		}
		else
		{
			const int32 HorizontalFilterViewSlotIndex = UE::Editor::ContentBrowser::Private::FindSlotByWidgetTag(
				LegacyContentSourceWidgets->PathAssetSplitterPtr.ToSharedRef(),
				HorizontalFilterViewTagName);

			// Remove from Horizontal layout (if needed)
			if (HorizontalFilterViewSlotIndex != INDEX_NONE)
			{
				LegacyContentSourceWidgets->PathAssetSplitterPtr->RemoveAt(HorizontalFilterViewSlotIndex);
			}

			// This can differ to the desired index, depending on widget state
			const int32 FoundVerticalFilterViewSlotIndex = UE::Editor::ContentBrowser::Private::FindSlotByWidgetTag(
				LegacyContentSourceWidgets->PathAssetSplitterPtr.ToSharedRef(),
				VerticalFilterViewTagName);

			// Check for Existing
			if (FoundVerticalFilterViewSlotIndex != INDEX_NONE)
			{
				LegacyContentSourceWidgets->PathAssetSplitterPtr->RemoveAt(FoundVerticalFilterViewSlotIndex);
			}

			constexpr int32 DesiredVerticalFilterViewSlotIndex = 1;

			LegacyContentSourceWidgets->PathAssetSplitterPtr->AddSlot(DesiredVerticalFilterViewSlotIndex)
			.MinSize(95.0f)
			.Resizable(true)
			.SizeRule(SSplitter::SizeToContent)
			.OnSlotResized(this, &SContentBrowser::OnFilterBoxColumnResized)
			[
				// Vertical Filter View
				SNew(SBox)
				.Tag(VerticalFilterViewTagName)
				.WidthOverride(this, &SContentBrowser::GetFilterViewBoxWidthOverride)
				// Don't take up space when there are no filters
				.Visibility_Lambda([this]
				{
					return LegacyContentSourceWidgets->FilterListPtr->HasAnyFilters() ? EVisibility::Visible : EVisibility::Collapsed;
				})
				[
					SNew(SVerticalBox)

					// Header
					+ SVerticalBox::Slot()
					.Padding(0.0f, 2.0f)
					.AutoHeight()
					[
						SNew(SBorder)
						.BorderImage(FContentBrowserStyle::Get().GetBrush("ContentBrowser.VerticalFilterViewHeaderBrush"))
						.Padding(FContentBrowserStyle::Get().GetMargin("ContentBrowser.VerticalFilterViewHeaderPadding"))
						.Content()
						[
							// Enforce widget height
							SNew(SBox)
							.HeightOverride(FContentBrowserStyle::Get().GetFloat("ContentBrowser.VerticalFilterViewHeaderTextHeight"))
							.Padding(0)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("FilterListVerticalHeader", "Filters"))
								.TextStyle(FAppStyle::Get(), "ButtonText")
								.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))	
							]
						]
					]

					// Filter List
					+ SVerticalBox::Slot()
					.Padding(0.0f)
					.FillHeight(1.0f)
					[
						SNew(SBorder)
						.BorderImage(FContentBrowserStyle::Get().GetBrush("ContentBrowser.VerticalFilterViewBodyBrush"))
						.Padding(0.0f)
						[
							LegacyContentSourceWidgets->FilterListPtr.ToSharedRef()	
						]
					]
				]
			];
		}
	}
	else
	{
		const FOptionalSize SearchBoxDesiredWidth = 500.0f;
		constexpr float SearchBoxMaxWidth = 0.0f;

		if (NewLayout == EFilterBarLayout::Horizontal)
		{
			TSharedPtr<SHorizontalBox> SearchBoxSlot =
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Top)
				.Padding(6, 4, 2, 0)
				[
					FilterComboButton.ToSharedRef()
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0)
				.VAlign(VAlign_Top)
				.Padding(0, 4, 0, 0)
				[
					SNew(SBox)
					.VAlign(VAlign_Center)
					.WidthOverride(SearchBoxDesiredWidth)
					[
						LegacyContentSourceWidgets->SearchBoxPtr.ToSharedRef()
					]
				];

			/** We add the Combo Button and the Search Box to the FilterList itself, so that the filters wrap with them
			 *	properly in the Horizontal Layout
			 */
			LegacyContentSourceWidgets->FilterListPtr->AddWidgetToCurrentLayout(SearchBoxSlot.ToSharedRef());

			LegacyContentSourceWidgets->AssetViewBorder->SetContent(

				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.Padding(0.0f)
				.AutoHeight()
				[
					LegacyContentSourceWidgets->FilterListPtr.ToSharedRef()
				]

				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.Padding(0, 0)
				[
					LegacyContentSourceWidgets->AssetViewPtr.ToSharedRef()
				]
			);
		}
		else
		{
			LegacyContentSourceWidgets->AssetViewBorder->SetContent(

				SNew(SSplitter)
				.PhysicalSplitterHandleSize(2.0f)

				/* Filters in an SScrollBox */
				+ SSplitter::Slot()
				.MinSize(95.f)
				.Resizable(true)
				.SizeRule(SSplitter::SizeToContent)
				.OnSlotResized(this, &SContentBrowser::OnFilterBoxColumnResized)
				[
					SNew(SBox)
					.WidthOverride(this, &SContentBrowser::GetFilterViewBoxWidthOverride)
					// Don't take up space when there are no filters
					.Visibility_Lambda([this]
					{
						return LegacyContentSourceWidgets->FilterListPtr->HasAnyFilters() ? EVisibility::Visible : EVisibility::Collapsed;
					})
					[
						SNew(SVerticalBox)

						// Header
						+ SVerticalBox::Slot()
						.Padding(0.0f, 2.0f)
						.AutoHeight()
						[

							SNew(SBorder)
							.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
							.Padding(FMargin(8.0f, 6.0f))
							.Content()
							[
								SNew(STextBlock)
								.Text(LOCTEXT("FilterListVerticalHeader", "Filters"))
								.TextStyle(FAppStyle::Get(), "ButtonText")
								.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
							]
						]

						// Filter List
						+ SVerticalBox::Slot()
						.Padding(0.0f)
						.FillHeight(1.0f)
						[
							LegacyContentSourceWidgets->FilterListPtr.ToSharedRef()
						]
					]
				]

				+ SSplitter::Slot()
				.Value(0.88f)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.Padding(6, 4, 0, 0)
					.AutoHeight()
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0, 0, 2, 0)
						[
							FilterComboButton.ToSharedRef()
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.MaxWidth(SearchBoxMaxWidth)
						[
							SNew(SBox)
							.VAlign(VAlign_Center)
							.WidthOverride(SearchBoxDesiredWidth)
							[
								LegacyContentSourceWidgets->SearchBoxPtr.ToSharedRef()
							]
						]
					]

					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					.Padding(0, 0)
					[
						LegacyContentSourceWidgets->AssetViewPtr.ToSharedRef()
					]
				]
			);
		}
	}
}

TSharedRef<SWidget> SContentBrowser::CreateAssetView(const FContentBrowserConfig* Config)
{
	// Create the Filter Bar Widget
	LegacyContentSourceWidgets->FilterListPtr = SNew(SFilterList)
					.CommandList(Commands)
					.OnFilterChanged(this, &SContentBrowser::OnFilterChanged)
					.Visibility((Config != nullptr ? Config->bCanShowFilters : true) ? EVisibility::Visible : EVisibility::Collapsed)
					.FrontendFilters(FrontendFilters)
					.FilterBarIdentifier(InstanceName)
					.FilterBarLayout(EFilterBarLayout::Vertical)
					.CanChangeOrientation(true)
					.OnFilterBarLayoutChanging(this, &SContentBrowser::OnFilterBarLayoutChanging)
					.UseSharedSettings(true)
					.CreateTextFilter(SFilterList::FCreateTextFilter::CreateSPLambda(this, [this]
					{
						TSharedPtr<FFrontendFilter_CustomText> NewFilter = MakeShared<FFrontendFilter_CustomText>();

						// Make sure the new filter has the right search options from the AssetView. We only have to set it once, SFilterList handles syncing it on change
						NewFilter->UpdateCustomTextFilterIncludes(LegacyContentSourceWidgets->AssetViewPtr->IsIncludingClassNames(),
							LegacyContentSourceWidgets->AssetViewPtr->IsIncludingAssetPaths(),
							LegacyContentSourceWidgets->AssetViewPtr->IsIncludingCollectionNames());

						return NewFilter;
					}))
					.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserFilters")));
	
	// Create the Filter Combo Button
	FilterComboButton = SFilterList::MakeAddFilterButton(LegacyContentSourceWidgets->FilterListPtr.ToSharedRef());
	TSharedPtr<ISlateMetaData> FilterComboButtonMetaData = MakeShared<FTagMetaData>(TEXT("ContentBrowserFiltersCombo"));
	FilterComboButton->AddMetadata(FilterComboButtonMetaData.ToSharedRef());
	FilterComboButton->SetVisibility((Config != nullptr ? Config->bCanShowFilters : true) ? EVisibility::Visible : EVisibility::Collapsed);

	LegacyContentSourceWidgets->AssetViewPtr->SetFilterBar(LegacyContentSourceWidgets->FilterListPtr);

	// If a search override is registered, create a per-view instance and retrieve its toggle widget
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	if (UE::Editor::ContentBrowser::Extension::IAssetSearchOverride* ActiveOverride = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").GetActiveAssetSearchOverride())
	{
		LegacyContentSourceWidgets->AssetSearchOverrideInstance = ActiveOverride->MakePerViewOverride();
		LegacyContentSourceWidgets->AssetSearchOverrideInstance->GetInternalApi().OnOverrideActiveChanged.AddSP(this, &SContentBrowser::OnSearchOverrideToggled);
		// Pass the instance to the asset view so it can take over the last step of the filtering and the sort.
		LegacyContentSourceWidgets->AssetViewPtr->SetAssetSearchOverride(LegacyContentSourceWidgets->AssetSearchOverrideInstance);
	}

	TSharedPtr<SWidget> SearchOverrideToggleWidget;
	if (LegacyContentSourceWidgets->AssetSearchOverrideInstance.IsValid())
	{
		SearchOverrideToggleWidget = LegacyContentSourceWidgets->AssetSearchOverrideInstance->GetOverrideModeToggleWidget();
	}
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	LegacyContentSourceWidgets->SearchBoxPtr = SNew(SAssetSearchBox)
				   .SuggestionListPlacement(EMenuPlacement::MenuPlacement_AboveAnchor) // Avoid showing suggestion on filtered content browser results...
				   .HintText(this, &SContentBrowser::GetSearchAssetsHintText)
				   .ShowSearchHistory(true)
				   .OnTextChanged(this, &SContentBrowser::OnSearchBoxChanged)
				   .OnTextCommitted(this, &SContentBrowser::OnSearchBoxCommitted)
				   .OnKeyDownHandler(this, &SContentBrowser::OnSearchKeyDown)
				   .OnSaveSearchClicked(this, &SContentBrowser::OnSaveSearchButtonClicked)
				   .OnAssetSearchBoxSuggestionFilter(this, &SContentBrowser::OnAssetSearchSuggestionFilter)
				   .OnAssetSearchBoxSuggestionChosen(this, &SContentBrowser::OnAssetSearchSuggestionChosen)
				   .DelayChangeNotificationsWhileTyping(true)
				   .ExtraWidget(SearchOverrideToggleWidget)
				   .Visibility((Config != nullptr ? Config->bCanShowAssetSearch : true) ? EVisibility::Visible : EVisibility::Collapsed)
				   .AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserSearchAssets")));

	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		LegacyContentSourceWidgets->NavigationBar = SNew(SNavigationBar)
						.BreadcrumbButtonContentPadding(FMargin(2, 0, 2, -1)) // This ensures proper vertical alignment of the text to fit the 24px height of the toolbar
						.OnPathClicked(this, &SContentBrowser::OnPathClicked)
						.GetPathMenuContent(this, &SContentBrowser::OnGetCrumbDelimiterContent)
						.GetComboOptions(this, &SContentBrowser::GetRecentPaths)
						.OnNavigateToPath(this, &SContentBrowser::OnNavigateToPath)
						.OnCompletePrefix(this, &SContentBrowser::OnCompletePathPrefix)
						.OnGetEditPathAsText(this, &SContentBrowser::OnGetEditPathAsText)
						.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserPath")));
	}


	/* Create the Border that the Asset View will live in, the actual layout is populated in OnFilterBarLayoutChanging,
	 * which is called initially called through SAssetFilterBar::LoadSettings through SContentBrowser::LoadSettings()
	 */
	const FMargin AssetViewPadding =
		UE::Editor::ContentBrowser::IsNewStyleEnabled()
			? FMargin(2.0f, 0.0f, 2.0f, 0.0f)
			: FMargin(2.0f, 2.0f, 2.0f, 0.0f);

	LegacyContentSourceWidgets->AssetViewBorder = SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.Padding(AssetViewPadding)
			[
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
				SNew(SInvalidationPanel)
				.UseDynamicInvalidation(true)
				[
#endif
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					.Padding(0, 0)
					[
						LegacyContentSourceWidgets->AssetViewPtr.ToSharedRef()
					]
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
				]
#endif
			];

	return LegacyContentSourceWidgets->AssetViewBorder.ToSharedRef();
}

SContentBrowser::FCollectionSource& SContentBrowser::AddSlotForCollectionContainer(int32 Index, const TSharedRef<ICollectionContainer>& CollectionContainer)
{
	if (Index == INDEX_NONE)
	{
		Index = CollectionSources.Num();
	}

	const TUniquePtr<FCollectionSource>& CollectionSource = CollectionSources.EmplaceAt_GetRef(Index, new FCollectionSource(*this, CollectionContainer));

	const int32 SlotIndex = LegacyContentSourceWidgets->SourceTreeSplitterNumFixedSlots + Index;
	SSplitter* Splitter = nullptr;
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		LegacyContentSourceWidgets->SourceTreePtr->AddSlot(SlotIndex)
			.AreaWidget(CreateCollectionsView(*CollectionSource))
			.Size(0.4f);

		Splitter = LegacyContentSourceWidgets->SourceTreePtr->GetSplitter().Get();
	}
	else
	{
		const float SourceTreeHeaderHeightMin = 26.0f + 3.0f;

		const TSharedRef<UE::Editor::ContentBrowser::Private::SContentBrowserSourceTreeArea> CollectionArea = CreateCollectionsView(*CollectionSource);

		LegacyContentSourceWidgets->PathFavoriteSplitterPtr->AddSlot(SlotIndex)
			.SizeRule_Lambda([CollectionArea]()
			{
				// Make sure the area is expanded
				return CollectionArea->IsExpanded() ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent;
			})
			.MinSize(SourceTreeHeaderHeightMin)
			.Value(0.4f)
			[
				SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
					.Padding(0.0f, 2.0f, 0.0f, 0.0f)
					[
						CollectionArea
					]
			];

		Splitter = LegacyContentSourceWidgets->PathFavoriteSplitterPtr.Get();
	}

	FString Key, Filename;
	GetSourceTreeSplitterSlotSizeSettingKeyAndFilename(SlotIndex, Key, Filename);

	float SplitterSize = Splitter->SlotAt(SlotIndex).GetSizeValue();
	GConfig->GetFloat(*SettingsIniSection, *Key, SplitterSize, Filename);
	Splitter->SlotAt(SlotIndex).SetSizeValue(SplitterSize);

	CollectionSource->LoadSettings(InstanceName);

	return *CollectionSource;
}

void SContentBrowser::RemoveSlotForCollectionContainer(const TSharedRef<ICollectionContainer>& CollectionContainer)
{
	int32 Index = CollectionSources.IndexOfByPredicate([&CollectionContainer](const TUniquePtr<FCollectionSource>& CollectionSource)
		{
			return CollectionSource->GetCollectionContainer() == CollectionContainer;
		});
	if (ensure(Index != INDEX_NONE))
	{
		CollectionSources[Index]->SaveSettings(InstanceName);

		const int32 SlotIndex = LegacyContentSourceWidgets->SourceTreeSplitterNumFixedSlots + Index;
		SSplitter& Splitter = UE::Editor::ContentBrowser::IsNewStyleEnabled() ?
			*LegacyContentSourceWidgets->SourceTreePtr->GetSplitter() : *LegacyContentSourceWidgets->PathFavoriteSplitterPtr;

		FString Key, Filename;
		GetSourceTreeSplitterSlotSizeSettingKeyAndFilename(SlotIndex, Key, Filename);

		float SplitterSize = Splitter.SlotAt(SlotIndex).GetSizeValue();
		GConfig->SetFloat(*SettingsIniSection, *Key, SplitterSize, Filename);

		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			LegacyContentSourceWidgets->SourceTreePtr->RemoveSlotAt(SlotIndex);
		}
		else
		{
			Splitter.RemoveAt(SlotIndex);
		}

		CollectionSources.RemoveAt(Index);
	}
}

void SContentBrowser::SetFavoritesExpanded(bool bExpanded)
{
	if (FContentBrowserInstanceConfig* EditorConfig = GetMutableInstanceConfig())
	{
		EditorConfig->bFavoritesExpanded = bExpanded;
		UContentBrowserConfig::Get()->SaveEditorConfig();
	}
}

TSharedRef<SWidget> SContentBrowser::CreateFavoritesView(const FContentBrowserConfig* Config)
{
	SAssignNew(LegacyContentSourceWidgets->FavoritePathViewPtr, SFavoritePathView)
		.OnItemSelectionChanged(this, &SContentBrowser::OnItemSelectionChanged, EContentBrowserViewContext::FavoriteView)
		.OnGetItemContextMenu(this, &SContentBrowser::GetItemContextMenu, EContentBrowserViewContext::FavoriteView)
		.FocusSearchBoxWhenOpened(false)
		.ShowTreeTitle(false)
		.ShowSeparator(false)
		.AllowClassesFolder(true)
		.CanShowDevelopersFolder(true)
		.OwningContentBrowserName(InstanceName)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserFavorites")))
		.ExternalSearch(FavoritesSearch);

	TSharedRef<UE::Editor::ContentBrowser::Private::SContentBrowserSourceTreeArea> ViewWidget =
		SAssignNew(
			FavoritesArea,
			UE::Editor::ContentBrowser::Private::SContentBrowserSourceTreeArea,
			"Favorites",
			FavoritesSearch,
			LegacyContentSourceWidgets->FavoritePathViewPtr.ToSharedRef())
			.Label(LOCTEXT("Favorites", "Favorites"))
			.Visibility(this, &SContentBrowser::GetFavoriteFolderVisibility)
			.OnExpansionChanged(this, &SContentBrowser::SetFavoritesExpanded)
			.IsEmpty_Lambda([PathView = LegacyContentSourceWidgets->FavoritePathViewPtr]
			{
				return PathView->IsEmpty();
			})
			.EmptyBodyLabel(LOCTEXT("FavoritesEmpty", "Right click a folder to add it to your favorites."));

	LegacyContentSourceWidgets->FavoritePathViewPtr->SetOnFolderFavoriteUpdated(SFavoritePathView::FOnFolderFavoriteUpdated::CreateSP(this, &SContentBrowser::UpdateFolderFavorite));
	return ViewWidget;
}

void SContentBrowser::SetPathViewExpanded(bool bExpanded)
{
	if (FContentBrowserInstanceConfig* EditorConfig = GetMutableInstanceConfig())
	{
		EditorConfig->PathView.bExpanded = bExpanded;
		UContentBrowserConfig::Get()->SaveEditorConfig();
	}
}

TSharedRef<SWidget> SContentBrowser::CreatePathView(const FContentBrowserConfig* Config)
{
	SAssignNew(LegacyContentSourceWidgets->PathViewPtr, SPathView)
		.OnItemSelectionChanged(this, &SContentBrowser::OnItemSelectionChanged, EContentBrowserViewContext::PathView)
		.OnGetItemContextMenu(this, &SContentBrowser::GetItemContextMenu, EContentBrowserViewContext::PathView)
		.FocusSearchBoxWhenOpened(false)
		.ShowTreeTitle(false)
		.ShowSeparator(false)
		.ShowRedirectors(MakeAttributeSPLambda(this, [this]() { return ContentBrowserUtils::ShouldShowRedirectors(LegacyContentSourceWidgets->FilterListPtr); }))
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserSources")))
		.ExternalSearch(SourcesSearch)
		.PluginPathFilters(PluginPathFilters)
		.OwningContentBrowserName(InstanceName)
		.AllowClassesFolder(Config != nullptr ? Config->bCanShowClasses : true)
		.CanShowDevelopersFolder(Config != nullptr ? Config->bCanShowDevelopersFolder : true)
		.ShadowBoxStyle(&FAppStyle::Get().GetWidgetStyle<FScrollBoxStyle>("ScrollBoxNoShadow"))
		.bEnableShadowBoxStyle(true);

	TSharedRef<UE::Editor::ContentBrowser::Private::SContentBrowserSourceTreeArea> ViewWidget =
		SAssignNew(
			PathArea,
			UE::Editor::ContentBrowser::Private::SContentBrowserSourceTreeArea,
			"Path",
			SourcesSearch,
			LegacyContentSourceWidgets->PathViewPtr.ToSharedRef())
			.Label(FText::FromString(FApp::GetProjectName()))
			.ExpandedByDefault(true) // The path area, unlike the favorites and collection areas, are expanded by default (unless overridden)
			.OnExpansionChanged(this, &SContentBrowser::SetPathViewExpanded)
			.ShadowBoxStyle(&FAppStyle::Get().GetWidgetStyle<FScrollBorderStyle>("ScrollBorderNoShadow"))
			.bEnableShadowBoxStyle(true);

	return ViewWidget;
}

TSharedRef<UE::Editor::ContentBrowser::Private::SContentBrowserSourceTreeArea> SContentBrowser::CreateCollectionsView(FCollectionSource& CollectionSource)
{
	return SAssignNew(
		CollectionSource.CollectionArea,
		UE::Editor::ContentBrowser::Private::SContentBrowserSourceTreeArea,
		"Collection",
		CollectionSource.CollectionSearch,
		CollectionSource.CollectionViewPtr.ToSharedRef())
		.Label(CollectionSource.GetCollectionContainer()->GetCollectionSource()->GetTitle())
		.HeaderContent()
		[
			SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ToolTipText(LOCTEXT("AddCollectionButtonTooltip", "Add a collection."))
						.OnClicked(FOnClicked::CreateSPLambda(this, [this, CollectionContainer = CollectionSource.GetCollectionContainer()]()
						{
							for (const TUniquePtr<FCollectionSource>& CollectionSource : CollectionSources)
							{
								if (CollectionSource->GetCollectionContainer() == CollectionContainer)
								{
									CollectionSource->CollectionArea->SetExpanded(true);
									CollectionSource->CollectionViewPtr->MakeAddCollectionMenu(SharedThis(this));
									break;
								}
							}

							return FReply::Handled();
						}))
						.ContentPadding(FMargin(1, 0))
						.Visibility_Lambda([CollectionContainer = CollectionSource.GetCollectionContainer()]() { return !CollectionContainer->IsReadOnly(ECollectionShareType::CST_All) ? EVisibility::Visible : EVisibility::Collapsed; })
						[
							SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
								.ColorAndOpacity(FSlateColor::UseForeground())
						]
				]
		]
		.IsEmpty_Lambda([CollectionView = CollectionSource.CollectionViewPtr]
		{
			return CollectionView->IsEmpty();
		})
		.EmptyBodyLabel_Lambda([CollectionContainer = CollectionSource.GetCollectionContainer()]()
		{
			if (CollectionContainer->IsReadOnly(ECollectionShareType::CST_All))
			{
				return LOCTEXT("CollectionsEmptyAndReadOnly", "No collections found.");
			}
			else
			{
				return LOCTEXT("CollectionsEmpty", "Click the <img src=\"Icons.PlusCircle\"/> in the section header to create a collection.");
			}
		});
}

TSharedRef<SWidget> SContentBrowser::CreateDrawerDockButton(const FContentBrowserConfig* Config)
{
	if(bIsDrawer)
	{
		return 
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("DockInLayout_Tooltip", "Docks this content browser in the current layout, copying all settings from the drawer.\nThe drawer will still be usable as a temporary browser."))
			.OnClicked(this, &SContentBrowser::DockInLayoutClicked)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0, 0.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.Layout"))
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4.0, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DockInLayout", "Dock in Layout"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}

	return SNullWidget::NullWidget;
}

void SContentBrowser::ExtendViewOptionsMenu(const FContentBrowserConfig* Config)
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetViewOptions");

	const bool bShowLockButton = Config == nullptr || Config->bCanShowLockButton;
	const bool bShowSourcesView = Config == nullptr || Config->bUseSourcesView;

	if (!bShowLockButton && !bShowSourcesView)
	{
		return;
	}

	Menu->AddDynamicSection("ContentBrowserViewOptionsSection",
		FNewToolMenuDelegate::CreateLambda(
			[bShowLockButton, bShowSourcesView](UToolMenu* InMenu)
		{
			if (UContentBrowserAssetViewContextMenuContext* Context = InMenu->FindContext<UContentBrowserAssetViewContextMenuContext>())
			{
				if (TSharedPtr<SContentBrowser> ContentBrowser = Context->OwningContentBrowser.Pin())
				{
					if (bShowLockButton)
					{
						if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
						{
							FToolMenuSection& Section = InMenu->FindOrAddSection("Manage");
							Section.AddMenuEntry(
								"ToggleLock",
								TAttribute<FText>(ContentBrowser.ToSharedRef(), &SContentBrowser::GetLockMenuText),
								LOCTEXT("LockToggleTooltip", "Toggle lock. If locked, this browser will ignore Find in Content Browser requests."),
								TAttribute<FSlateIcon>(ContentBrowser.ToSharedRef(), &SContentBrowser::GetLockIcon),
								FUIAction(
									FExecuteAction::CreateLambda([ContentBrowser = Context->OwningContentBrowser]() { ContentBrowser.Pin()->ToggleLockClicked(); })
								)
							);
						}
						else
						{
							FToolMenuSection& Section = InMenu->AddSection("Locking", LOCTEXT("LockingMenuHeader", "Locking"), FToolMenuInsert("AssetViewType", EToolMenuInsertType::After));
							Section.AddMenuEntry(
								"ToggleLock",
								TAttribute<FText>(ContentBrowser.ToSharedRef(), &SContentBrowser::GetLockMenuText),
								LOCTEXT("LockToggleTooltip", "Toggle lock. If locked, this browser will ignore Find in Content Browser requests."),
								TAttribute<FSlateIcon>(ContentBrowser.ToSharedRef(), &SContentBrowser::GetLockIcon),
								FUIAction(
									FExecuteAction::CreateLambda([ContentBrowser = Context->OwningContentBrowser]() { ContentBrowser.Pin()->ToggleLockClicked(); })
								)
							);
						}
					}

					if (bShowSourcesView)
					{
						FToolMenuSection& Section = InMenu->FindOrAddSection(UE::Editor::ContentBrowser::IsNewStyleEnabled() ? "Show" : "View");
						Section.AddMenuEntry(
							"ToggleSources",
							UE::Editor::ContentBrowser::IsNewStyleEnabled()
							? LOCTEXT("ToggleSourcesView_NewStyle", "Sources Panel")
							: LOCTEXT("ToggleSourcesView", "Show Sources Panel"),
							LOCTEXT("ToggleSourcesView_Tooltip", "Show or hide the sources panel"),
							TAttribute<FSlateIcon>(),
							FUIAction(
								FExecuteAction::CreateLambda([ContentBrowser = Context->OwningContentBrowser]() { ContentBrowser.Pin()->SourcesViewExpandClicked(); }),
								FCanExecuteAction(),
								FIsActionChecked::CreateLambda([ContentBrowser = Context->OwningContentBrowser]() { return ContentBrowser.Pin()->bSourcesViewExpanded; })),
							EUserInterfaceActionType::ToggleButton
						)
						.InsertPosition =
							UE::Editor::ContentBrowser::IsNewStyleEnabled()
							? FToolMenuInsert(NAME_None, EToolMenuInsertType::First)
							: FToolMenuInsert();
					}
				}
			}
		}
	));
}

SSplitter::ESizeRule SContentBrowser::GetFavoritesAreaSizeRule() const
{
	// Make sure the area is expanded and visible
	return FavoritesArea->IsExpanded() && GetFavoriteFolderVisibility() == EVisibility::Visible ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent;
}

SSplitter::ESizeRule SContentBrowser::GetPathAreaSizeRule() const
{
	return PathArea->IsExpanded() ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent;
}

void SContentBrowser::OnPathViewBoxColumnResized(float InSize)
{
	PathViewBoxWidth = InSize;
}

FOptionalSize SContentBrowser::GetPathViewBoxWidthOverride() const
{
	return FOptionalSize(PathViewBoxWidth);
}

void SContentBrowser::OnFilterBoxColumnResized(float InSize)
{
	FilterBoxWidth = InSize;
}

FOptionalSize SContentBrowser::GetFilterViewBoxWidthOverride() const
{
	return FOptionalSize(FilterBoxWidth);
}

float SContentBrowser::GetFavoritesAreaMinSize() const
{
	static const float SourceTreeHeaderHeightMin =
		UE::Editor::ContentBrowser::IsNewStyleEnabled()
		? 36.0f
		: 26.0f + 3.0f;

	return GetFavoriteFolderVisibility() == EVisibility::Visible ? SourceTreeHeaderHeightMin : 0.0f;
}

FText SContentBrowser::GetHighlightedText() const
{
	return TextFilter->GetRawFilterText();
}

void SContentBrowser::CreateNewAsset(const FString& DefaultAssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory)
{
	// For now we just forcefully enable the legacy content source when a new asset creation is requested
	ContentSourcesContainer->ActivateLegacyContentSource();
	
	LegacyContentSourceWidgets->AssetViewPtr->CreateNewAsset(DefaultAssetName, PackagePath, AssetClass, Factory);
}

void SContentBrowser::PrepareToSyncItems(TArrayView<const FContentBrowserItem> ItemsToSync, const bool bDisableFiltersThatHideAssets)
{
	// For now we just forcefully enable the legacy content source when a sync is requested so it succeeds
	ContentSourcesContainer->ActivateLegacyContentSource();
	
	bool bRepopulate = false;

	// Check to see if any of the assets require certain folders to be visible
	const UContentBrowserSettings* ContentBrowserSettings = GetDefault<UContentBrowserSettings>();
	bool bDisplayDev = ContentBrowserSettings->GetDisplayDevelopersFolder();
	bool bDisplayEngine = ContentBrowserSettings->GetDisplayEngineFolder();
	bool bDisplayPlugins = ContentBrowserSettings->GetDisplayPluginFolders();
	bool bDisplayLocalized = ContentBrowserSettings->GetDisplayL10NFolder();

	// check to see if we have an instance config that overrides the default in UContentBrowserSettings
	if (const FContentBrowserInstanceConfig* EditorConfig = GetConstInstanceConfig())
	{
		bDisplayDev = EditorConfig->bShowDeveloperContent;
		bDisplayEngine = EditorConfig->bShowEngineContent;
		bDisplayPlugins = EditorConfig->bShowPluginContent;
		bDisplayLocalized = EditorConfig->bShowLocalizedContent;
	}

	// Keep track of any of the settings changing so we can let the user know
	bool bDisplayDevChanged = false;
	bool bDisplayEngineChanged = false;
	bool bDisplayPluginsChanged = false;
	bool bDisplayLocalizedChanged = false;

	if ( !bDisplayDev || !bDisplayEngine || !bDisplayPlugins || !bDisplayLocalized )
	{
		for (const FContentBrowserItem& ItemToSync : ItemsToSync)
		{
			if (!bDisplayDev && ContentBrowserUtils::IsItemDeveloperContent(ItemToSync))
			{
				bDisplayDev = true;
				LegacyContentSourceWidgets->AssetViewPtr->OverrideShowDeveloperContent();
				bRepopulate = true;
				bDisplayDevChanged = true;
			}

			if (!bDisplayEngine && ContentBrowserUtils::IsItemEngineContent(ItemToSync))
			{
				bDisplayEngine = true;
				LegacyContentSourceWidgets->AssetViewPtr->OverrideShowEngineContent();
				bRepopulate = true;
				bDisplayEngineChanged = true;
			}

			if (!bDisplayPlugins && ContentBrowserUtils::IsItemPluginContent(ItemToSync))
			{
				bDisplayPlugins = true;
				LegacyContentSourceWidgets->AssetViewPtr->OverrideShowPluginContent();
				bRepopulate = true;
				bDisplayPluginsChanged = true;
			}

			if (!bDisplayLocalized && ContentBrowserUtils::IsItemLocalizedContent(ItemToSync))
			{
				bDisplayLocalized = true;
				LegacyContentSourceWidgets->AssetViewPtr->OverrideShowLocalizedContent();
				bRepopulate = true;
				bDisplayLocalizedChanged = true;
			}

			if (bDisplayDev && bDisplayEngine && bDisplayPlugins && bDisplayLocalized)
			{
				break;
			}
		}
	}

	// Disable any plugin filters which hide the path we're navigating too in the path tree
	bool bSomePluginPathFiltersChanged = false;
	if (LegacyContentSourceWidgets->PathViewPtr->DisablePluginPathFiltersThatHideItems(ItemsToSync))
	{
		bSomePluginPathFiltersChanged = true;
		bRepopulate = true;
	}
	
	if (bDisableFiltersThatHideAssets)
	{
		// Disable the filter categories
		// Do this before repopulate because the redirectors filter can hide folders
		LegacyContentSourceWidgets->FilterListPtr->DisableFiltersThatHideItems(ItemsToSync);
	}

	// If we have auto-enabled any flags or found a non-existant path, force a refresh
	if (bRepopulate)
	{
		// let the user know if one of their settings is being changed to be able to show the sync targets
		if (bDisplayDevChanged || bDisplayEngineChanged || bDisplayPluginsChanged || bDisplayLocalizedChanged || bSomePluginPathFiltersChanged)
		{
			TArray<FText> SettingsText;
			if (bDisplayDevChanged)
			{
				SettingsText.Add(LOCTEXT("ShowDeveloperContent", "Show Developer Content"));
			}
			if (bDisplayEngineChanged)
			{
				SettingsText.Add(LOCTEXT("ShowEngineContent", "Show Engine Content"));
			}
			if (bDisplayPluginsChanged)
			{
				SettingsText.Add(LOCTEXT("ShowPluginContent", "Show Plugin Content"));
			}
			if (bDisplayLocalizedChanged)
			{
				SettingsText.Add(LOCTEXT("ShowLocalizedContent", "Show Localized Content"));
			}
			if (bSomePluginPathFiltersChanged)
			{
				SettingsText.Add(LOCTEXT("SomePluginPathFilters", "Some Plugin Filters"));
			}
			FTextBuilder NotificationBuilder;
			const FText NotificationPrefix = FText::Format(
				LOCTEXT("AssetRequiresFilterChanges", "To show {0}|plural(one=this asset,other=these assets), the following {1}|plural(one=setting has,other=settings have) been changed for the active Content Browser:\n"),
				ItemsToSync.Num(),
				SettingsText.Num());
			NotificationBuilder.AppendLine(NotificationPrefix);
			NotificationBuilder.Indent();
			for ( const FText& SettingsTextEntry : SettingsText)
			{
				NotificationBuilder.AppendLine(SettingsTextEntry);
			}
	
			FNotificationInfo NotificationInfo(NotificationBuilder.ToText());
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
		LegacyContentSourceWidgets->PathViewPtr->Populate();
		LegacyContentSourceWidgets->FavoritePathViewPtr->Populate();
	}

	// Disable the filter search (reset the filter, then clear the search text)
	// Note: we have to remove the filter immediately, we can't wait for OnSearchBoxChanged to hit
	SetSearchBoxText(FText::GetEmpty());
	LegacyContentSourceWidgets->SearchBoxPtr->SetText(FText::GetEmpty());
	LegacyContentSourceWidgets->SearchBoxPtr->SetError(FText::GetEmpty());
}

void SContentBrowser::SyncToAssets(TArrayView<const FAssetData> AssetDataList, const bool bAllowImplicitSync, const bool bDisableFiltersThatHideAssets)
{
	// For now we just forcefully enable the legacy content source when a sync is requested so it succeeds
	ContentSourcesContainer->ActivateLegacyContentSource();
	
	SyncToLegacy(AssetDataList, TArrayView<const FString>(), bAllowImplicitSync, bDisableFiltersThatHideAssets);
}

void SContentBrowser::SyncToFolders(TArrayView<const FString> FolderList, const bool bAllowImplicitSync)
{
	// For now we just forcefully enable the legacy content source when a sync is requested so it succeeds
	ContentSourcesContainer->ActivateLegacyContentSource();
	
	SyncToLegacy(TArrayView<const FAssetData>(), FolderList, bAllowImplicitSync, /*bDisableFiltersThatHideAssets*/false);
}

void SContentBrowser::SyncToCollections(TArrayView<const FCollectionRef> CollectionsToSync)
{
	// For now we just forcefully enable the legacy content source when a sync is requested so it succeeds
	ContentSourcesContainer->ActivateLegacyContentSource();

	// You may not select both collections and paths
	LegacyContentSourceWidgets->PathViewPtr->ClearSelection();
	LegacyContentSourceWidgets->FavoritePathViewPtr->ClearSelection();

	TArray<FCollectionRef> SelectedCollections;
	SelectedCollections.Reserve(CollectionsToSync.Num());
	{
		TArray<FCollectionNameType> CollectionsToSelect;
		for (const TUniquePtr<FCollectionSource>& CollectionSource : CollectionSources)
		{
			CollectionsToSelect.Reset(CollectionsToSync.Num());
			for (const FCollectionRef& CollectionToSync : CollectionsToSync)
			{
				if (CollectionSource->GetCollectionContainer() == CollectionToSync.Container)
				{
					CollectionsToSelect.Emplace(CollectionToSync.Name, CollectionToSync.Type);
				}
			}
			CollectionSource->CollectionViewPtr->SetSelectedCollections(CollectionsToSelect),

			Algo::Transform(
				CollectionSource->CollectionViewPtr->GetSelectedCollections(),
				SelectedCollections,
				[&CollectionSource](const FCollectionNameType& Collection) { return FCollectionRef(CollectionSource->GetCollectionContainer(), Collection); });
		}
	}

	if (SelectedCollections.Num() == 0 && !CollectionSources.IsEmpty())
	{
		// Select a dummy "None" collection to avoid the sources view switching to the paths view
		SelectedCollections.Emplace(CollectionSources[0]->GetCollectionContainer(), FCollectionNameType(NAME_None, ECollectionShareType::CST_System));
	}

	SourcesChanged({}, SelectedCollections);
}

void SContentBrowser::SyncToItems(TArrayView<const FContentBrowserItem> ItemsToSync, const bool bAllowImplicitSync, const bool bDisableFiltersThatHideAssets)
{
	// For now we just forcefully enable the legacy content source when a sync is requested so it succeeds
	ContentSourcesContainer->ActivateLegacyContentSource();

	TArray<FContentBrowserItem> NewItemsToSync = ContentBrowserUtils::FilterOrAliasItems(ItemsToSync, *LegacyContentSourceWidgets->PathViewPtr);
	ItemsToSync = NewItemsToSync;
	PrepareToSyncItems(ItemsToSync, bDisableFiltersThatHideAssets);

	// Tell the sources view first so the asset view will be up to date by the time we request the sync
	LegacyContentSourceWidgets->PathViewPtr->SyncToItems(ItemsToSync, bAllowImplicitSync);
	LegacyContentSourceWidgets->FavoritePathViewPtr->SyncToItems(ItemsToSync, bAllowImplicitSync);
	LegacyContentSourceWidgets->AssetViewPtr->SyncToItems(ItemsToSync);
}

void SContentBrowser::SyncToLegacy(TArrayView<const FAssetData> AssetDataList, TArrayView<const FString> FolderList, const bool bAllowImplicitSync, const bool bDisableFiltersThatHideAssets)
{
	// For now we just forcefully enable the legacy content source when a sync is requested so it succeeds
	ContentSourcesContainer->ActivateLegacyContentSource();
	
	TArray<FContentBrowserItem> ItemsToSync = ContentBrowserUtils::ConvertLegacySelectionToItems(AssetDataList, FolderList);

	PrepareToSyncItems(ItemsToSync, bDisableFiltersThatHideAssets);

	// Tell the sources view first so the asset view will be up to date by the time we request the sync
	LegacyContentSourceWidgets->PathViewPtr->SyncToItems(ItemsToSync, bAllowImplicitSync);
	LegacyContentSourceWidgets->FavoritePathViewPtr->SyncToItems(ItemsToSync, bAllowImplicitSync);
	LegacyContentSourceWidgets->AssetViewPtr->SyncToItems(ItemsToSync);
}

void SContentBrowser::SyncTo( const FContentBrowserSelection& ItemSelection, const bool bAllowImplicitSync, const bool bDisableFiltersThatHideAssets )
{
	// For now we just forcefully enable the legacy content source when a sync is requested so it succeeds
	ContentSourcesContainer->ActivateLegacyContentSource();
	
	if (ItemSelection.IsLegacy())
	{
		SyncToLegacy(ItemSelection.SelectedAssets, ItemSelection.SelectedFolders, bAllowImplicitSync, bDisableFiltersThatHideAssets);
	}
	else
	{
		SyncToItems(ItemSelection.SelectedItems, bAllowImplicitSync, bDisableFiltersThatHideAssets);
	}
}

void SContentBrowser::SetIsPrimaryContentBrowser(bool NewIsPrimary)
{
	if (!CanSetAsPrimaryContentBrowser()) 
	{
		return;
	}

	bIsPrimaryBrowser = NewIsPrimary;

	if ( bIsPrimaryBrowser )
	{
		SyncGlobalSelectionSet();
	}
	else
	{
		USelection* EditorSelection = GEditor->GetSelectedObjects();
		if ( ensure( EditorSelection != NULL ) )
		{
			EditorSelection->DeselectAll();
		}
	}
}

bool SContentBrowser::CanSetAsPrimaryContentBrowser() const
{
	return bCanSetAsPrimaryBrowser;
}

TSharedPtr<FTabManager> SContentBrowser::GetTabManager() const
{
	if (TSharedPtr<SDockTab> Tab = ContainingTab.Pin())
	{
		return Tab->GetTabManagerPtr();
	}

	return TSharedPtr<FTabManager>();
}

void SContentBrowser::LoadSelectedObjectsIfNeeded()
{
	// Get the selected assets in the asset view
	const TArray<FAssetData>& SelectedAssets = LegacyContentSourceWidgets->AssetViewPtr->GetSelectedAssets();

	// Load every asset that isn't already in memory
	for ( auto AssetIt = SelectedAssets.CreateConstIterator(); AssetIt; ++AssetIt )
	{
		const FAssetData& AssetData = *AssetIt;
		const bool bShowProgressDialog = (!AssetData.IsAssetLoaded() && FEditorFileUtils::IsMapPackageAsset(AssetData.GetObjectPathString()));
		GWarn->BeginSlowTask(LOCTEXT("LoadingObjects", "Loading Objects..."), bShowProgressDialog);

		(*AssetIt).GetAsset();

		GWarn->EndSlowTask();
	}

	// Sync the global selection set if we are the primary browser
	if ( bIsPrimaryBrowser )
	{
		SyncGlobalSelectionSet();
	}
}

bool SContentBrowser::IsLegacyContentSourceActive() const
{
	return ContentSourcesContainer.IsValid() && ContentSourcesContainer->IsLegacyContentSourceActive();
}

void SContentBrowser::GetSelectedAssets(TArray<FAssetData>& SelectedAssets)
{
	// For now we just forcefully enable the legacy content source when this public function is called so it succeeds
	ContentSourcesContainer->ActivateLegacyContentSource();

	SelectedAssets = LegacyContentSourceWidgets->AssetViewPtr->GetSelectedAssets();
}

void SContentBrowser::GetSelectedFolders(TArray<FString>& SelectedFolders)
{
	// For now we just forcefully enable the legacy content source when this public function is called so it succeeds
	ContentSourcesContainer->ActivateLegacyContentSource();
	
	SelectedFolders = LegacyContentSourceWidgets->AssetViewPtr->GetSelectedFolders();
}

TArray<FString> SContentBrowser::GetSelectedPathViewFolders()
{
	// For now we just forcefully enable the legacy content source when this public function is called so it succeeds
	ContentSourcesContainer->ActivateLegacyContentSource();
	
	check(LegacyContentSourceWidgets->PathViewPtr.IsValid());
	return LegacyContentSourceWidgets->PathViewPtr->GetSelectedPaths();
}

void SContentBrowser::SaveSettings() const
{
	// Individual content sources will handle saving their own settings. If the legacy content source is active we save its settings, otherwise
	// the settings were saved when the legacy content source was disabled
	if (!ContentSourcesContainer->IsLegacyContentSourceActive())
	{
		return;
	}
	
	const FString& SettingsString = InstanceName.ToString();

	GConfig->SetBool(*SettingsIniSection, *(SettingsString + TEXT(".SourcesExpanded")), bSourcesViewExpanded, GEditorPerProjectIni);
	GConfig->SetBool(*SettingsIniSection, *(SettingsString + TEXT(".IsLocked")), bIsLocked, GEditorPerProjectIni);

	FavoritesArea->SaveSettings(GEditorPerProjectIni, SettingsIniSection, SettingsString);
	PathArea->SaveSettings(GEditorPerProjectIni, SettingsIniSection, SettingsString);

	for(int32 SlotIndex = 0; SlotIndex < LegacyContentSourceWidgets->PathAssetSplitterPtr->GetChildren()->Num(); SlotIndex++)
	{
		// First Slot containing the PathView is using SizeToContent so the SizeValue is not updated
		// Adding another config line for older projects otherwise when they open for the first time after this update the SplitterSlot size will be based on the older one
		// The older one is a normalized value, so it is too small and will make the SplitterSlot for the PathView seems like if it was collapsed the very first time you re-open it
		const bool bIsFirstSlot = SlotIndex == 0;
		float SplitterSize = bIsFirstSlot ? PathViewBoxWidth : LegacyContentSourceWidgets->PathAssetSplitterPtr->SlotAt(SlotIndex).GetSizeValue();
		if (bIsFirstSlot)
		{
			GConfig->SetFloat(*SettingsIniSection, *(SettingsString + FString::Printf(TEXT(".VerticalSplitter.FixedSlotSize%d"), SlotIndex)), SplitterSize, GEditorPerProjectIni);
		}
		else
		{
			GConfig->SetFloat(*SettingsIniSection, *(SettingsString + FString::Printf(TEXT(".VerticalSplitter.SlotSize%d"), SlotIndex)), SplitterSize, GEditorPerProjectIni);
		}
	}

	{
		SSplitter& Splitter = UE::Editor::ContentBrowser::IsNewStyleEnabled() ?
			*LegacyContentSourceWidgets->SourceTreePtr->GetSplitter() : *LegacyContentSourceWidgets->PathFavoriteSplitterPtr;
		for (int32 SlotIndex = 0; SlotIndex < Splitter.GetChildren()->Num(); SlotIndex++)
		{
			FString Key, Filename;
			GetSourceTreeSplitterSlotSizeSettingKeyAndFilename(SlotIndex, Key, Filename);

			float SplitterSize = Splitter.SlotAt(SlotIndex).GetSizeValue();
			GConfig->SetFloat(*SettingsIniSection, *Key, SplitterSize, Filename);
		}
	}

	// Save all our data using the settings string as a key in the user settings ini
	LegacyContentSourceWidgets->FilterListPtr->SaveSettings(GEditorPerProjectIni, SettingsIniSection, SettingsString);
	LegacyContentSourceWidgets->PathViewPtr->SaveSettings(GEditorPerProjectIni, SettingsIniSection, SettingsString);
	LegacyContentSourceWidgets->FavoritePathViewPtr->SaveSettings(GEditorPerProjectIni, SettingsIniSection, SettingsString + TEXT(".Favorites"));
	for (const TUniquePtr<FCollectionSource>& CollectionSource : CollectionSources)
	{
		CollectionSource->SaveSettings(InstanceName);
	}
	LegacyContentSourceWidgets->AssetViewPtr->SaveSettings(GEditorPerProjectIni, SettingsIniSection, SettingsString);
	
	GConfig->SetArray(*SettingsIniSection, *(SettingsString + TEXT(".JumpMRU")), JumpMRU, GEditorPerProjectIni);
}

const FName SContentBrowser::GetInstanceName() const
{
	return InstanceName;
}

bool SContentBrowser::IsLocked() const
{
	return bIsLocked;
}

void SContentBrowser::SetKeyboardFocusOnSearch() const
{
	if (ContentSourcesContainer->IsLegacyContentSourceActive())
	{
		// Focus on the search box
		FSlateApplication::Get().SetKeyboardFocus( LegacyContentSourceWidgets->SearchBoxPtr, EFocusCause::SetDirectly );
	}
}

void SContentBrowser::CopySettingsFromBrowser(TSharedPtr<SContentBrowser> OtherBrowser)
{
	if (ContentSourcesContainer->IsLegacyContentSourceActive())
	{
		FName InstanceNameToCopyFrom = OtherBrowser->GetInstanceName();

		// Clear out any existing settings that dont get reset on load
		LegacyContentSourceWidgets->FilterListPtr->RemoveAllFilters();

		LoadSettings(InstanceNameToCopyFrom);
	}
	
}

FReply SContentBrowser::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if (ContentSourcesContainer->IsLegacyContentSourceActive())
	{
		bool bIsRenamingAsset = LegacyContentSourceWidgets->AssetViewPtr && LegacyContentSourceWidgets->AssetViewPtr->IsRenamingAsset();
		if(bIsRenamingAsset || Commands->ProcessCommandBindings( InKeyEvent ) )
		{
			return FReply::Handled();
		}
	}
	
	return FReply::Unhandled();
}

FReply SContentBrowser::OnPreviewMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	// Clicking in a content browser will shift it to be the primary browser
	FContentBrowserSingleton::Get().SetPrimaryContentBrowser(SharedThis(this));

	return FReply::Unhandled();
}

FReply SContentBrowser::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	// Mouse back and forward buttons traverse history
	if ( MouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton)
	{
		HistoryManager.GoBack();
		return FReply::Handled();
	}
	else if ( MouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton2)
	{
		HistoryManager.GoForward();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SContentBrowser::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	// Mouse back and forward buttons traverse history
	if ( InMouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton)
	{
		HistoryManager.GoBack();
		return FReply::Handled();
	}
	else if ( InMouseEvent.GetEffectingButton() == EKeys::ThumbMouseButton2)
	{
		HistoryManager.GoForward();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SContentBrowser::OnContainingTabSavingVisualState() const
{
	SaveSettings();
}

void SContentBrowser::OnContainingTabClosed(TSharedRef<SDockTab> DockTab)
{
	FContentBrowserSingleton::Get().ContentBrowserClosed( SharedThis(this) );
}

void SContentBrowser::OnContainingTabActivated(TSharedRef<SDockTab> DockTab, ETabActivationCause InActivationCause)
{
	if(InActivationCause == ETabActivationCause::UserClickedOnTab)
	{
		FContentBrowserSingleton::Get().SetPrimaryContentBrowser(SharedThis(this));
	}
}

void SContentBrowser::GetSourceTreeSplitterSlotSizeSettingKeyAndFilename(int32 SlotIndex, FString& OutKey, FString& OutFilename) const
{
	OutKey = InstanceName.ToString();

	if (SlotIndex < LegacyContentSourceWidgets->SourceTreeSplitterNumFixedSlots)
	{
		OutKey += FString::Printf(TEXT(".FavoriteSplitter.SlotSize%d"), SlotIndex);
		OutFilename = GEditorPerProjectIni;
	}
	else
	{
		const TUniquePtr<FCollectionSource>& CollectionSource = CollectionSources[SlotIndex - LegacyContentSourceWidgets->SourceTreeSplitterNumFixedSlots];
		if (CollectionSource->IsProjectCollectionContainer())
		{
			// Reconsider the .FavoriteSplitter.SlotSize key naming scheme if you hit this check.
			check(LegacyContentSourceWidgets->SourceTreeSplitterNumFixedSlots == 2);

			// Maintain backwards compatibility with previous version of the SContentBrowser which had a single collection view.
			OutKey += FString::Printf(TEXT(".FavoriteSplitter.SlotSize%d"), LegacyContentSourceWidgets->SourceTreeSplitterNumFixedSlots);
		}
		else
		{
			OutKey += FString::Printf(TEXT(".%s.FavoriteSplitter.SlotSize"), *CollectionSource->GetCollectionContainer()->GetCollectionSource()->GetName().ToString());
		}
		OutFilename = CollectionSource->GetCollectionContainer()->GetCollectionSource()->GetEditorPerProjectIni();
	}
}

void SContentBrowser::LoadSettings(const FName& InInstanceName)
{
	// Individual content sources will handle saving their own settings. If the legacy content source is active we load the settings, otherwise
	// the settings will be loaded when the legacy content source is enabled
	if (!ContentSourcesContainer->IsLegacyContentSourceActive())
	{
		return;
	}
	
	FString SettingsString = InInstanceName.ToString();

	// Now that we have determined the appropriate settings string, actually load the settings
	bSourcesViewExpanded = true;
	GConfig->GetBool(*SettingsIniSection, *(SettingsString + TEXT(".SourcesExpanded")), bSourcesViewExpanded, GEditorPerProjectIni);

	bIsLocked = false;
	GConfig->GetBool(*SettingsIniSection, *(SettingsString + TEXT(".IsLocked")), bIsLocked, GEditorPerProjectIni);

	FavoritesArea->LoadSettings(GEditorPerProjectIni, SettingsIniSection, SettingsString);
	PathArea->LoadSettings(GEditorPerProjectIni, SettingsIniSection, SettingsString);

	for(int32 SlotIndex = 0; SlotIndex < LegacyContentSourceWidgets->PathAssetSplitterPtr->GetChildren()->Num(); SlotIndex++)
	{
		// First Slot containing the PathView is using SizeToContent so the SizeValue is not updated
		// Adding another config line for older projects otherwise when they open for the first time after this update the SplitterSlot size will be based on the older one
		// The older one is a normalized value, so it is too small and make will make the SplitterSlot for the PathView seems like if it was collapsed the very first time you re-open it
		const bool bIsFirstSlot = SlotIndex == 0;
		float SplitterSize = bIsFirstSlot ? PathViewBoxWidth : LegacyContentSourceWidgets->PathAssetSplitterPtr->SlotAt(SlotIndex).GetSizeValue();
		if (bIsFirstSlot)
		{
			GConfig->GetFloat(*SettingsIniSection, *(SettingsString + FString::Printf(TEXT(".VerticalSplitter.FixedSlotSize%d"), SlotIndex)), SplitterSize, GEditorPerProjectIni);
			PathViewBoxWidth = SplitterSize;
		}
		else
		{
			GConfig->GetFloat(*SettingsIniSection, *(SettingsString + FString::Printf(TEXT(".VerticalSplitter.SlotSize%d"), SlotIndex)), SplitterSize, GEditorPerProjectIni);
			LegacyContentSourceWidgets->PathAssetSplitterPtr->SlotAt(SlotIndex).SetSizeValue(SplitterSize);
		}
	}

	{
		SSplitter& Splitter = UE::Editor::ContentBrowser::IsNewStyleEnabled() ?
			*LegacyContentSourceWidgets->SourceTreePtr->GetSplitter() : *LegacyContentSourceWidgets->PathFavoriteSplitterPtr;
		for (int32 SlotIndex = 0; SlotIndex < Splitter.GetChildren()->Num(); SlotIndex++)
		{
			FString Key, Filename;
			GetSourceTreeSplitterSlotSizeSettingKeyAndFilename(SlotIndex, Key, Filename);

			float SplitterSize = Splitter.SlotAt(SlotIndex).GetSizeValue();
			GConfig->GetFloat(*SettingsIniSection, *Key, SplitterSize, Filename);
			Splitter.SlotAt(SlotIndex).SetSizeValue(SplitterSize);
		}
	}

	// Save all our data using the settings string as a key in the user settings ini
	LegacyContentSourceWidgets->FilterListPtr->LoadSettings(InInstanceName, GEditorPerProjectIni, SettingsIniSection, SettingsString);
	LegacyContentSourceWidgets->PathViewPtr->LoadSettings(GEditorPerProjectIni, SettingsIniSection, SettingsString);
	LegacyContentSourceWidgets->FavoritePathViewPtr->LoadSettings(GEditorPerProjectIni, SettingsIniSection, SettingsString + TEXT(".Favorites"));
	for (const TUniquePtr<FCollectionSource>& CollectionSource : CollectionSources)
	{
		CollectionSource->LoadSettings(InstanceName);
	}
	LegacyContentSourceWidgets->AssetViewPtr->LoadSettings(GEditorPerProjectIni, SettingsIniSection, SettingsString);

	TArray<FString> TempJumpMRU;
	GConfig->GetArray(*SettingsIniSection, *(SettingsString + TEXT(".JumpMRU")), TempJumpMRU, GEditorPerProjectIni);

	// We previously allowed some non-normalized paths, fix those up.
	JumpMRU.Reset(TempJumpMRU.Num());
	for (FString& Path : TempJumpMRU)
	{
		FPaths::NormalizeDirectoryName(Path);
		if (!JumpMRU.Contains(Path))
		{
			JumpMRU.Add(MoveTemp(Path));
		}
	}
}

void SContentBrowser::SourcesChanged(const TArray<FString>& SelectedPaths, const TArray<FCollectionRef>& SelectedCollections)
{
	FString NewSource = SelectedPaths.Num() > 0 ? SelectedPaths[0] : (SelectedCollections.Num() > 0 ? SelectedCollections[0].Name.ToString() : TEXT("None"));
	UE_LOGF(LogContentBrowser, VeryVerbose, "The content browser source was changed by the sources view to '%ls'", *NewSource);

	FAssetViewContentSources ContentSources;
	{
		TArray<FName> SelectedPathNames;
		SelectedPathNames.Reserve(SelectedPaths.Num());
		for (const FString& SelectedPath : SelectedPaths)
		{
			SelectedPathNames.Add(FName(*SelectedPath));
		}
		ContentSources = FAssetViewContentSources(MoveTemp(SelectedPathNames), SelectedCollections);
	}

	// A dynamic collection should apply its search query to the CB search, so we need to stash the current search so that we can restore it again later
	if (ContentSources.IsDynamicCollection())
	{
		// Only stash the user search term once in case we're switching between dynamic collections
		if (!StashedSearchBoxText.IsSet())
		{
			StashedSearchBoxText = TextFilter->GetRawFilterText();
		}

		const FCollectionRef& DynamicCollection = ContentSources.GetCollections()[0];

		FString DynamicQueryString;
		DynamicCollection.Container->GetDynamicQueryText(DynamicCollection.Name, DynamicCollection.Type, DynamicQueryString);

		const FText DynamicQueryText = FText::FromString(DynamicQueryString);
		SetSearchBoxText(DynamicQueryText);
		LegacyContentSourceWidgets->SearchBoxPtr->SetText(DynamicQueryText);
	}
	else if (StashedSearchBoxText.IsSet())
	{
		// Restore the stashed search term
		const FText StashedText = StashedSearchBoxText.GetValue();
		StashedSearchBoxText.Reset();

		SetSearchBoxText(StashedText);
		LegacyContentSourceWidgets->SearchBoxPtr->SetText(StashedText);
	}

	if (!LegacyContentSourceWidgets->AssetViewPtr->GetContentSources().IsEmpty())
	{
		// Update the current history data to preserve selection if there is a valid ContentSources
		HistoryManager.UpdateHistoryData();
	}

	// Change the filter for the asset view
	LegacyContentSourceWidgets->AssetViewPtr->SetContentSources(ContentSources);

	// Add a new history data now that the source has changed
	HistoryManager.AddHistoryData();

	// Update the breadcrumb trail path
	UpdatePath();
}

void SContentBrowser::FolderEntered(const FContentBrowserItem& Folder)
{
	check(Folder.IsFolder());

	// Have we entered a sub-collection folder?
	const bool bCollectionFolder = EnumHasAnyFlags(Folder.GetItemCategory(), EContentBrowserItemFlags::Category_Collection);
	if (bCollectionFolder)
	{
		TSharedPtr<ICollectionContainer> CollectionContainer;
		FName CollectionName;
		ECollectionShareType::Type CollectionFolderShareType = ECollectionShareType::CST_All;
		if (ContentBrowserUtils::IsCollectionPath(Folder.GetVirtualPath().ToString(), &CollectionContainer, &CollectionName, &CollectionFolderShareType))
		{
			for (const TUniquePtr<FCollectionSource>& CollectionSource : CollectionSources)
			{
				if (CollectionSource->GetCollectionContainer() == CollectionContainer)
				{
					const FCollectionNameType SelectedCollection(CollectionName, CollectionFolderShareType);

					TArray<FCollectionNameType> Collections;
					Collections.Add(SelectedCollection);
					CollectionSource->CollectionViewPtr->SetSelectedCollections(Collections);

					CollectionSelected(CollectionContainer, SelectedCollection);
					break;
				}
			}
		}
	}
	else
	{
		// set the path view to the incoming path
		TArray<FString> SelectedPaths;
		SelectedPaths.Add(Folder.GetVirtualPath().ToString());
		LegacyContentSourceWidgets->PathViewPtr->SetSelectedPaths(SelectedPaths);

		PathSelected(SelectedPaths[0]);
	}
}

void SContentBrowser::PathSelected(const FString& FolderPath)
{
	JumpMRU.AddUnique(FolderPath);

	// You may not select both collections and paths
	for (const TUniquePtr<FCollectionSource>& CollectionSource : CollectionSources)
	{
		CollectionSource->CollectionViewPtr->ClearSelection();
	}

	TArray<FString> SelectedPaths = LegacyContentSourceWidgets->PathViewPtr->GetSelectedPaths();
	// Selecting a folder shows it in the favorite list also
	LegacyContentSourceWidgets->FavoritePathViewPtr->SetSelectedPaths(SelectedPaths);
	TArray<FCollectionRef> SelectedCollections;
	SourcesChanged(SelectedPaths, SelectedCollections);

	LegacyContentSourceWidgets->PathViewPtr->SaveSettings(GEditorPerProjectIni, SettingsIniSection, InstanceName.ToString());

	// Notify 'asset path changed' delegate
	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>( TEXT("ContentBrowser") );
	FContentBrowserModule::FOnAssetPathChanged& PathChangedDelegate = ContentBrowserModule.GetOnAssetPathChanged();
	if(PathChangedDelegate.IsBound())
	{
		PathChangedDelegate.Broadcast(FolderPath);
	}

	// Update the context menu's selected paths list
	LegacyContentSourceWidgets->PathContextMenu->SetSelectedFolders(LegacyContentSourceWidgets->PathViewPtr->GetSelectedFolderItems());
}

void SContentBrowser::FavoritePathSelected(const FString& FolderPath)
{
	JumpMRU.AddUnique(FolderPath);
	
	// You may not select both collections and paths
	for (const TUniquePtr<FCollectionSource>& CollectionSource : CollectionSources)
	{
		CollectionSource->CollectionViewPtr->ClearSelection();
	}

	TArray<FString> SelectedPaths = LegacyContentSourceWidgets->FavoritePathViewPtr->GetSelectedPaths();
	// Selecting a favorite shows it in the main list also
	LegacyContentSourceWidgets->PathViewPtr->SetSelectedPaths(SelectedPaths);
	TArray<FCollectionRef> SelectedCollections;
	SourcesChanged(SelectedPaths, SelectedCollections);

	LegacyContentSourceWidgets->PathViewPtr->SaveSettings(GEditorPerProjectIni, SettingsIniSection, InstanceName.ToString());

	// Notify 'asset path changed' delegate
	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	FContentBrowserModule::FOnAssetPathChanged& PathChangedDelegate = ContentBrowserModule.GetOnAssetPathChanged();
	if (PathChangedDelegate.IsBound())
	{
		PathChangedDelegate.Broadcast(FolderPath);
	}

	// Update the context menu's selected paths list
	LegacyContentSourceWidgets->PathContextMenu->SetSelectedFolders(LegacyContentSourceWidgets->FavoritePathViewPtr->GetSelectedFolderItems());
}

TSharedRef<FExtender> SContentBrowser::GetPathContextMenuExtender(const TArray<FString>& InSelectedPaths) const
{
	return LegacyContentSourceWidgets->PathContextMenu->MakePathViewContextMenuExtender(InSelectedPaths);
}

void SContentBrowser::CollectionSelected(const TSharedPtr<ICollectionContainer>& CollectionContainer, const FCollectionNameType& SelectedCollection)
{
	// You may not select both collections and paths
	LegacyContentSourceWidgets->PathViewPtr->ClearSelection();
	LegacyContentSourceWidgets->FavoritePathViewPtr->ClearSelection();

	TArray<FCollectionRef> SelectedCollections;
	for (const TUniquePtr<FCollectionSource>& CollectionSource : CollectionSources)
	{
		if (CollectionSource->GetCollectionContainer() == CollectionContainer)
		{
			Algo::Transform(
				CollectionSource->CollectionViewPtr->GetSelectedCollections(),
				SelectedCollections,
				[&CollectionContainer](const FCollectionNameType& Collection) { return FCollectionRef(CollectionContainer, Collection); });
		}
		else
		{
			CollectionSource->CollectionViewPtr->ClearSelection();
		}
	}

	TArray<FString> SelectedPaths;

	if (SelectedCollections.Num() == 0)
	{
		// Select a dummy "None" collection to avoid the sources view switching to the paths view
		SelectedCollections.Emplace(CollectionContainer, FCollectionNameType(NAME_None, ECollectionShareType::CST_System));
	}
	
	SourcesChanged(SelectedPaths, SelectedCollections);
}

void SContentBrowser::SetSelectedPaths(const TArray<FString>& FolderPaths, bool bNeedsRefresh/* = false */)
{
	if (FolderPaths.Num() > 0)
	{
		TArray<FString> NormalizedFolderPaths = FolderPaths;
		for (FString& NormalizedFolderPath : NormalizedFolderPaths)
		{
			FPaths::NormalizeDirectoryName(NormalizedFolderPath);
		}

		if (bNeedsRefresh)
		{
			LegacyContentSourceWidgets->PathViewPtr->RequestPopulation();	
			LegacyContentSourceWidgets->FavoritePathViewPtr->Populate();
		}

		// Need to set selected paths on PathViewPtr here even if we just called RequestPopulation above because PathSelected below expects it to be set 
		LegacyContentSourceWidgets->PathViewPtr->SetSelectedPaths(NormalizedFolderPaths);
		LegacyContentSourceWidgets->FavoritePathViewPtr->SetSelectedPaths(NormalizedFolderPaths);
		PathSelected(NormalizedFolderPaths[0]);
	}
}

void SContentBrowser::ForceShowPluginContent(bool bEnginePlugin)
{
	// For now we just forcefully enable the legacy content source when this public function is called so it succeeds
	ContentSourcesContainer->ActivateLegacyContentSource();
	
	if (LegacyContentSourceWidgets->AssetViewPtr.IsValid())
	{
		LegacyContentSourceWidgets->AssetViewPtr->ForceShowPluginFolder(bEnginePlugin);
	}
}

void SContentBrowser::OnApplyHistoryData( const FHistoryData& History )
{
	if (!ContentSourcesContainer->IsLegacyContentSourceActive())
	{
		return;
	}
	
	LegacyContentSourceWidgets->PathViewPtr->ApplyHistoryData(History);
	LegacyContentSourceWidgets->FavoritePathViewPtr->ApplyHistoryData(History);
	for (const TUniquePtr<FCollectionSource>& CollectionSource : CollectionSources)
	{
		CollectionSource->CollectionViewPtr->ApplyHistoryData(History);
	}
	LegacyContentSourceWidgets->AssetViewPtr->ApplyHistoryData(History);

	// Update the breadcrumb trail path
	UpdatePath();

	if (History.ContentSources.HasVirtualPaths())
	{
		// Notify 'asset path changed' delegate
		FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		FContentBrowserModule::FOnAssetPathChanged& PathChangedDelegate = ContentBrowserModule.GetOnAssetPathChanged();
		if (PathChangedDelegate.IsBound())
		{
			PathChangedDelegate.Broadcast(History.ContentSources.GetVirtualPaths()[0].ToString());
		}
	}
}

void SContentBrowser::OnUpdateHistoryData(FHistoryData& HistoryData) const
{
	if (!ContentSourcesContainer->IsLegacyContentSourceActive())
	{
		return;
	}
	
	const FAssetViewContentSources& ContentSources = LegacyContentSourceWidgets->AssetViewPtr->GetContentSources();
	const TArray<FContentBrowserItem> SelectedItems = LegacyContentSourceWidgets->AssetViewPtr->GetSelectedItems();

	const FText NewSource = ContentSources.HasVirtualPaths() ?
		IContentBrowserDataModule::Get().GetSubsystem()->ConvertVirtualPathToDisplay(ContentSources.GetVirtualPaths()[0], EContentBrowserItemTypeFilter::IncludeFolders) :
		(ContentSources.HasCollections() ? FText::FromName(ContentSources.GetCollections()[0].Name) : LOCTEXT("AllAssets", "All Assets"));

	HistoryData.HistoryDesc = NewSource;
	HistoryData.ContentSources = ContentSources;

	HistoryData.SelectionData.Reset();
	for (const FContentBrowserItem& SelectedItem : SelectedItems)
	{
		HistoryData.SelectionData.SelectedVirtualPaths.Add(SelectedItem.GetVirtualPath());
	}
}

void SContentBrowser::NewFolderRequested(const FString& SelectedPath)
{
	if( ensure(SelectedPath.Len() > 0) && LegacyContentSourceWidgets->AssetViewPtr.IsValid() )
	{
		CreateNewFolder(SelectedPath, FOnCreateNewFolder::CreateSP(LegacyContentSourceWidgets->AssetViewPtr.Get(), &SAssetView::NewFolderItemRequested));
	}
}

void SContentBrowser::NewFileItemRequested(const FContentBrowserItemDataTemporaryContext& NewItemContext)
{
	if (LegacyContentSourceWidgets && LegacyContentSourceWidgets->AssetViewPtr)
	{
		LegacyContentSourceWidgets->AssetViewPtr->NewFileItemRequested(NewItemContext);
	}
}

void SContentBrowser::SetSearchText(const FText& InSearchText)
{
	if (LegacyContentSourceWidgets && LegacyContentSourceWidgets->SearchBoxPtr.IsValid())
	{
		LegacyContentSourceWidgets->SearchBoxPtr->SetText(InSearchText);
	}
}

void SContentBrowser::SetAssetViewUserSearching(bool bIsSearching)
{
	LegacyContentSourceWidgets->AssetViewPtr->SetUserSearching(bIsSearching);
}

void SContentBrowser::SetSearchBoxText(const FText& InSearchText)
{
	// Has anything changed? (need to test case as the operators are case-sensitive)
	if (!InSearchText.ToString().Equals(TextFilter->GetRawFilterText().ToString(), ESearchCase::CaseSensitive))
	{
		TextFilter->SetRawFilterText(InSearchText);
		LegacyContentSourceWidgets->SearchBoxPtr->SetError(TextFilter->GetFilterErrorText());
		if (InSearchText.IsEmpty())
		{
			LegacyContentSourceWidgets->AssetViewPtr->SetUserSearching(false);
		}
		else
		{
			LegacyContentSourceWidgets->AssetViewPtr->SetUserSearching(true);
		}
	}
}

void SContentBrowser::OnSearchBoxChanged(const FText& InSearchText)
{
	bool bOverrideHandled = false;

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	if (LegacyContentSourceWidgets->AssetSearchOverrideInstance.IsValid() &&
		LegacyContentSourceWidgets->AssetSearchOverrideInstance->IsOverrideActive())
	{
		LegacyContentSourceWidgets->AssetSearchOverrideInstance->OnSearchTextChanged(InSearchText);
		bOverrideHandled = true;
	}
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	if (!bOverrideHandled)
	{
		SetSearchBoxText(InSearchText);
	}

	// Broadcast 'search box changed' delegate
	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>( TEXT("ContentBrowser") );
	ContentBrowserModule.GetOnSearchBoxChanged().Broadcast(InSearchText, bIsPrimaryBrowser);
}

void SContentBrowser::OnSearchBoxCommitted(const FText& InSearchText, ETextCommit::Type CommitInfo)
{
	bool bOverrideHandled = false;

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	if (LegacyContentSourceWidgets->AssetSearchOverrideInstance.IsValid() &&
		LegacyContentSourceWidgets->AssetSearchOverrideInstance->IsOverrideActive())
	{
		LegacyContentSourceWidgets->AssetSearchOverrideInstance->OnSearchTextCommitted(InSearchText, CommitInfo);
		bOverrideHandled = true;
	}
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	if (!bOverrideHandled)
	{
		SetSearchBoxText(InSearchText);
	}
}

FReply SContentBrowser::OnSearchKeyDown(const FGeometry& Geometry, const FKeyEvent& InKeyEvent)
{
	FInputChord CheckChord(InKeyEvent.GetKey(), EModifierKey::FromBools(InKeyEvent.IsControlDown(), InKeyEvent.IsAltDown(), InKeyEvent.IsShiftDown(), InKeyEvent.IsCommandDown()));

	// Clear focus if the content browser drawer key is clicked so it will close the opened content browser
	if (FGlobalEditorCommonCommands::Get().OpenContentBrowserDrawer->HasActiveChord(CheckChord))
	{
		FReply Reply = FReply::Handled().ClearUserFocus(EFocusCause::SetDirectly);
		
		if (bIsDrawer)
		{
			GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->DismissContentBrowserDrawer();
		}
		return Reply;
	}

	return FReply::Unhandled();
}

bool SContentBrowser::IsSaveSearchButtonEnabled() const
{
	return !TextFilter->GetRawFilterText().IsEmptyOrWhitespace();
}

void SContentBrowser::OnSaveSearchButtonClicked(const FText& InSearchText)
{
	// Need to make sure we can see the collections view
	if (!bSourcesViewExpanded)
	{
		SourcesViewExpandClicked();
	}

	// We want to add any currently selected paths to the final saved query so that you get back roughly the same list of objects as what you're currently seeing
	FString SelectedPathsQuery;
	{
		const TArray<FName>& VirtualPaths = LegacyContentSourceWidgets->AssetViewPtr->GetContentSources().GetVirtualPaths();
		for (int32 SelectedPathIndex = 0; SelectedPathIndex < VirtualPaths.Num(); ++SelectedPathIndex)
		{
			SelectedPathsQuery.Append(TEXT("Path:'"));
			SelectedPathsQuery.Append(VirtualPaths[SelectedPathIndex].ToString());
			SelectedPathsQuery.Append(TEXT("'..."));

			if (SelectedPathIndex + 1 < VirtualPaths.Num())
			{
				SelectedPathsQuery.Append(TEXT(" OR "));
			}
		}
	}

	// todo: should we automatically append any type filters too?

	// Produce the final query
	FText FinalQueryText;
	if (SelectedPathsQuery.IsEmpty())
	{
		FinalQueryText = TextFilter->GetRawFilterText();
	}
	else
	{
		FinalQueryText = FText::FromString(FString::Printf(TEXT("(%s) AND (%s)"), *TextFilter->GetRawFilterText().ToString(), *SelectedPathsQuery));
	}

	// Get all menu extenders for this context menu from the content browser module
	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	TArray<FContentBrowserMenuExtender> MenuExtenderDelegates = ContentBrowserModule.GetAllCollectionViewContextMenuExtenders();

	TArray<TSharedPtr<FExtender>> Extenders;
	for (int32 i = 0; i < MenuExtenderDelegates.Num(); ++i)
	{
		if (MenuExtenderDelegates[i].IsBound())
		{
			Extenders.Add(MenuExtenderDelegates[i].Execute());
		}
	}
	TSharedPtr<FExtender> MenuExtender = FExtender::Combine(Extenders);

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, NULL, MenuExtender, true);

	/** Make the menu to save a search */
	MenuBuilder.BeginSection("ContentBrowserSaveSearch", LOCTEXT("ContentBrowserCreateFilterMenuHeading", "Create Filter"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ContentBrowserSaveAsCustomFilter", "Save as Custom Filter"),
		LOCTEXT("ContentBrowserSaveAsCustomFilterTooltip", "Save the current search text as a custom filter in the filter bar"),
		FSlateIcon(),
		FUIAction(FSimpleDelegate::CreateSP(this, &SContentBrowser::SaveSearchAsFilter))
	);

	MenuBuilder.EndSection();

	if (!CollectionSources.IsEmpty())
	{
		if (CollectionSources.Num() == 1)
		{
			CollectionSources[0]->CollectionViewPtr->MakeSaveDynamicCollectionMenu(MenuBuilder, FinalQueryText);
		}
		else
		{
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("ContentBrowserCollectionContainersMenuHeading", "Collection Containers"));

			for (const TUniquePtr<FCollectionSource>& CollectionSource : CollectionSources)
			{
				const TSharedPtr<ICollectionContainer>& CollectionContainer = CollectionSource->GetCollectionContainer();
				MenuBuilder.AddSubMenu(
					CollectionContainer->GetCollectionSource()->GetTitle(),
					TAttribute<FText>(),
					FNewMenuDelegate::CreateSP(CollectionSource->CollectionViewPtr.ToSharedRef(), &SCollectionView::MakeSaveDynamicCollectionMenu, FinalQueryText));
			}

			MenuBuilder.EndSection();
		}
	}

	FWidgetPath WidgetPath;
	if (FSlateApplication::Get().GeneratePathToWidgetUnchecked(AsShared(), WidgetPath, EVisibility::All)) // since the collection window can be hidden, we need to manually search the path with a EVisibility::All instead of the default EVisibility::Visible
	{
		FSlateApplication::Get().PushMenu(
			AsShared(),
			WidgetPath,
			MenuBuilder.MakeWidget(),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::TopMenu)
		);
	}
}

void SContentBrowser::SaveSearchAsFilter()
{
	LegacyContentSourceWidgets->FilterListPtr->CreateCustomFilterDialog(TextFilter->GetRawFilterText());
}

void SContentBrowser::EditPathCommand()
{
	LegacyContentSourceWidgets->NavigationBar->StartEditingPath();
}

void SContentBrowser::OnNavigateToPath(const FString& NewPath)
{
	FContentBrowserItem Item = ContentBrowserUtils::TryGetItemFromUserProvidedPath(NewPath);
	if (Item.IsValid())
	{
		SyncToItems(MakeArrayView(&Item, 1));
	}
}

void SContentBrowser::OnPathClicked( const FString& CrumbData )
{
	FAssetViewContentSources ContentSources = LegacyContentSourceWidgets->AssetViewPtr->GetContentSources();

	if (ContentSources.HasCollections() )
	{
		// Collection crumb was clicked. See if we've clicked on a different collection in the hierarchy, and change the path if required.
		FCollectionSource* CollectionSource = nullptr;
		FCollectionNameType CollectionClicked(NAME_None, ECollectionShareType::CST_System);
		if (ParseCollectionCrumbData(CrumbData, CollectionSource, CollectionClicked) &&
			(ContentSources.GetCollections()[0].Container != CollectionSource->GetCollectionContainer() ||
				ContentSources.GetCollections()[0].Name != CollectionClicked.Name ||
				ContentSources.GetCollections()[0].Type != CollectionClicked.Type))
		{
			TArray<FCollectionNameType> Collections;
			Collections.Add(CollectionClicked);
			CollectionSource->CollectionViewPtr->SetSelectedCollections(Collections);

			CollectionSelected(CollectionSource->GetCollectionContainer(), CollectionClicked);
		}
	}
	else if ( !ContentSources.HasVirtualPaths() )
	{
		// No collections or paths are selected. This is "All Assets". Don't change the path when this is clicked.
	}
	else if (ContentSources.GetVirtualPaths().Num() > 1 || ContentSources.GetVirtualPaths()[0].ToString() != CrumbData )
	{
		// More than one path is selected or the crumb that was clicked is not the same path as the current one. Change the path.
		TArray<FString> SelectedPaths;
		SelectedPaths.Add(CrumbData);
		LegacyContentSourceWidgets->PathViewPtr->SetSelectedPaths(SelectedPaths);
		LegacyContentSourceWidgets->FavoritePathViewPtr->SetSelectedPaths(SelectedPaths);
		PathSelected(SelectedPaths[0]);
	}
}

TArray<FNavigationBarComboOption> SContentBrowser::GetRecentPaths() const
{
	TArray<FNavigationBarComboOption> RecentPaths;
	RecentPaths.Reserve(JumpMRU.Num());
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	for (const FString& RecentPath : JumpMRU)
	{
		FContentBrowserItem Item = ContentBrowserData->GetItemAtPath(FName(RecentPath), EContentBrowserItemTypeFilter::IncludeFolders);
		if (Item.IsValid())
		{
			RecentPaths.Emplace(ContentBrowserData->ConvertVirtualPathToDisplay(Item), RecentPath);
		}
	}
	return RecentPaths;
}

void SContentBrowser::OnPathMenuItemClicked(FString ClickedPath)
{
	OnPathClicked(ClickedPath);
}

FText SContentBrowser::OnGetEditPathAsText(const FString& Path) const
{
	const FAssetViewContentSources& ContentSources = LegacyContentSourceWidgets->AssetViewPtr->GetContentSources();
	if (ContentSources.HasCollections())
	{
		// Do not present collections as text because their names are not very user friendly right now.
		return FText::GetEmpty();
	}
	else if (ContentSources.HasVirtualPaths())
	{
		return ContentBrowserUtils::GetUserFacingPathFromVirtualPath(FName(Path));
	}
	return FText::GetEmpty();
}

TArray<FNavigationBarComboOption> SContentBrowser::OnCompletePathPrefix(const FString& Prefix) const
{
	FStringView PrefixView = Prefix;
	FStringView SuffixView;
	
	// Strip to last path separator, but keep the very first path separator
	if (int32 Index = UE::String::FindLastChar(PrefixView, '/'); Index != INDEX_NONE)
	{	
		PrefixView.LeftInline(FMath::Max(1, Index));
		SuffixView = Prefix;
		SuffixView.RightChopInline(PrefixView.Len() + 1);
	}

	// Find PrefixView in the available tree of data sources, get its direct children, and filter them by SuffixView 
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	TArray<FString> ChildPaths = ContentBrowserUtils::GetChildPathsFromUserProvidedPath(
		PrefixView,
		LegacyContentSourceWidgets->PathViewPtr->GetContentBrowserItemCategoryFilter(),
		LegacyContentSourceWidgets->PathViewPtr->GetContentBrowserItemAttributeFilter(),
		InstanceName,
		*LegacyContentSourceWidgets->PathViewPtr); 
	TArray<FNavigationBarComboOption> Results;
	for (const FString& ChildPath : ChildPaths)
	{
		FStringView ChildSuffix = ChildPath;
		ChildSuffix.RightChopInline(PrefixView.Len() + 1);
		if (ChildSuffix.StartsWith(SuffixView))
		{
			Results.Emplace(FText::AsCultureInvariant(ChildPath), ChildPath);
		}
	}
	return Results;
}

TSharedRef<SWidget> SContentBrowser::OnGetCrumbDelimiterContent(const FString& CrumbData) const
{
	const FAssetViewContentSources& ContentSources = LegacyContentSourceWidgets->AssetViewPtr->GetContentSources();

	TSharedPtr<SWidget> Widget = SNullWidget::NullWidget;
	TSharedPtr<SWidget> MenuWidget;

	if( ContentSources.HasCollections() )
	{
		FCollectionSource* CollectionSource = nullptr;
		FCollectionNameType CollectionClicked(NAME_None, ECollectionShareType::CST_System);
		if (ParseCollectionCrumbData(CrumbData, CollectionSource, CollectionClicked))
		{
			TArray<FCollectionNameType> ChildCollections;
			CollectionSource->GetCollectionContainer()->GetChildCollections(CollectionClicked.Name, CollectionClicked.Type, ChildCollections);

			if( ChildCollections.Num() > 0 )
			{
				FMenuBuilder MenuBuilder( true, nullptr );

				for( const FCollectionNameType& ChildCollection : ChildCollections )
				{
					const FString ChildCollectionCrumbData = ContentBrowserUtils::FormatCollectionCrumbData(*CollectionSource->GetCollectionContainer(), ChildCollection);

					MenuBuilder.AddMenuEntry(
						FText::FromName(ChildCollection.Name),
						FText::GetEmpty(),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ChildCollection.Type)),
						FUIAction(FExecuteAction::CreateSP(const_cast<SContentBrowser*>(this), &SContentBrowser::OnPathMenuItemClicked, ChildCollectionCrumbData))
						);
				}

				MenuWidget = MenuBuilder.MakeWidget();
			}
		}
	}
	else if( ContentSources.HasVirtualPaths() )
	{
		TArray<FContentBrowserItem> SubItems = ContentBrowserUtils::GetChildItemsFromVirtualPath(
			*CrumbData,
			LegacyContentSourceWidgets->PathViewPtr->GetContentBrowserItemCategoryFilter(),
			LegacyContentSourceWidgets->PathViewPtr->GetContentBrowserItemAttributeFilter(),
			InstanceName,
			*LegacyContentSourceWidgets->PathViewPtr); 
		SubItems.Sort([](const FContentBrowserItem& ItemOne, const FContentBrowserItem& ItemTwo)
		{
			return ItemOne.GetDisplayName().CompareTo(ItemTwo.GetDisplayName()) < 0;
		});

		if(SubItems.Num() > 0)
		{
			FMenuBuilder MenuBuilder( true, nullptr );

			for (const FContentBrowserItem& SubItem : SubItems)
			{
				FName FolderBrushName = NAME_None;
				FName FolderShadowBrushName = NAME_None;
				ContentBrowserUtils::TryGetFolderBrushAndShadowNameSmall(SubItem, FolderBrushName, FolderShadowBrushName);
				
				FText EntryName = SubItem.GetDisplayName();
				FUIAction EntryAction = FUIAction(FExecuteAction::CreateSP(const_cast<SContentBrowser*>(this), &SContentBrowser::OnPathMenuItemClicked, SubItem.GetVirtualPath().ToString()));

				if (FolderBrushName != NAME_None)
				{
					FLinearColor FolderColor = UE::Editor::ContentBrowser::ExtensionUtils::GetFolderColor(SubItem).Get(ContentBrowserUtils::GetDefaultColor());
						
					FMenuEntryParams Params;
					Params.EntryWidget = ContentBrowserUtils::GetFolderWidgetForNavigationBar(EntryName, FolderBrushName, FolderColor);
					Params.DirectActions = EntryAction;
					MenuBuilder.AddMenuEntry(Params);
				}
				else
				{
					MenuBuilder.AddMenuEntry(
						EntryName,
						FText::GetEmpty(),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), FolderBrushName),
						EntryAction
					);
				}

			}

			MenuWidget = MenuBuilder.MakeWidget();
		}
	}

	if( MenuWidget.IsValid() )
	{
		// Do not allow the menu to become too large if there are many directories
		Widget =
			SNew( SVerticalBox )
			+SVerticalBox::Slot()
			.MaxHeight( 400.0f )
			[
				MenuWidget.ToSharedRef()
			];
	}

	return Widget.ToSharedRef();
}

bool SContentBrowser::ParseCollectionCrumbData(const FString& CrumbData, FCollectionSource*& OutCollectionSource, FCollectionNameType& OutCollection) const
{
	OutCollectionSource = nullptr;
	OutCollection = FCollectionNameType(NAME_None, ECollectionShareType::CST_System);

	TOptional<FCollectionNameType> Collection;
	{
		FString CollectionContainerName;
		FString Temp;
		if (CrumbData.Split(TEXT("?"), &CollectionContainerName, &Temp))
		{
			for (const TUniquePtr<FCollectionSource>& CollectionSource : CollectionSources)
			{
				if (CollectionSource->GetCollectionContainer()->GetCollectionSource()->GetName() == CollectionContainerName)
				{
					FString CollectionName;
					FString CollectionTypeString;
					if (Temp.Split(TEXT("?"), &CollectionName, &CollectionTypeString))
					{
						const int32 CollectionType = FCString::Atoi(*CollectionTypeString);
						if (CollectionType >= 0 && CollectionType < ECollectionShareType::CST_All)
						{
							OutCollectionSource = CollectionSource.Get();
							OutCollection = FCollectionNameType(FName(*CollectionName), ECollectionShareType::Type(CollectionType));
							return true;
						}
					}
					break;
				}
			}
		}
	}
	return false;
}

FString SContentBrowser::GetCurrentPath(const EContentBrowserPathType PathType) const
{
	FString CurrentPath;
	const FAssetViewContentSources& ContentSources = LegacyContentSourceWidgets->AssetViewPtr->GetContentSources();
	if ( ContentSources.HasVirtualPaths() && ContentSources.GetVirtualPaths()[0] != NAME_None )
	{
		if (PathType == EContentBrowserPathType::Virtual)
		{
			ContentSources.GetVirtualPaths()[0].ToString(CurrentPath);
		}
		else if (IContentBrowserDataModule::Get().GetSubsystem()->TryConvertVirtualPath(FNameBuilder(ContentSources.GetVirtualPaths()[0]), CurrentPath) != PathType)
		{
			const EContentBrowserPathType ConvertedPathType = IContentBrowserDataModule::Get().GetSubsystem()->TryConvertVirtualPath(FNameBuilder(ContentSources.GetVirtualPaths()[0]), CurrentPath);
			if (ConvertedPathType != PathType)
			{
				CurrentPath.Reset();
			}
		}
	}

	return CurrentPath;
}

void SContentBrowser::AppendNewMenuContextObjects(const EContentBrowserDataMenuContext_AddNewMenuDomain InDomain, const TArray<FName>& InSelectedPaths, FToolMenuContext& InOutMenuContext, UContentBrowserToolbarMenuContext* CommonContext, bool bCanBeModified)
{
	if(!CommonContext)
	{
		UContentBrowserMenuContext* CommonContextObject = NewObject<UContentBrowserMenuContext>();
		CommonContextObject->ContentBrowser = SharedThis(this);
		InOutMenuContext.AddObject(CommonContextObject);
	}
	else
	{
		InOutMenuContext.AddObject(CommonContext);
	}

	{
		bool bContainsValidPackagePath = false;
		for (const FName SelectedPath : InSelectedPaths)
		{
			FString ConvertedPath;
			if (IContentBrowserDataModule::Get().GetSubsystem()->TryConvertVirtualPath(FNameBuilder(SelectedPath), ConvertedPath) == EContentBrowserPathType::Internal)
			{
				if (FPackageName::IsValidPath(ConvertedPath))
				{
					bContainsValidPackagePath = true;
					break;
				}
			}
		}

		UContentBrowserDataMenuContext_AddNewMenu* DataContextObject = NewObject<UContentBrowserDataMenuContext_AddNewMenu>();
		DataContextObject->SelectedPaths = InSelectedPaths;
		DataContextObject->OwnerDomain = InDomain;
		DataContextObject->OnBeginItemCreation = UContentBrowserDataMenuContext_AddNewMenu::FOnBeginItemCreation::CreateSP(this, &SContentBrowser::NewFileItemRequested);
		DataContextObject->bCanBeModified = bCanBeModified;
		DataContextObject->bContainsValidPackagePath = bContainsValidPackagePath;
		DataContextObject->OwningInstanceConfig = GetConstInstanceConfig();
		InOutMenuContext.AddObject(DataContextObject);
	}
}

TSharedRef<SWidget> SContentBrowser::MakeAddNewContextMenu(const EContentBrowserDataMenuContext_AddNewMenuDomain InDomain, UContentBrowserToolbarMenuContext* CommonContext)
{
	const FAssetViewContentSources& ContentSources = LegacyContentSourceWidgets->AssetViewPtr->GetContentSources();

	bool bCanBeModified = false;

	// Get all menu extenders for this context menu from the content browser module
	TSharedPtr<FExtender> MenuExtender;
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		TArray<FContentBrowserMenuExtender_SelectedPaths> MenuExtenderDelegates = ContentBrowserModule.GetAllAssetContextMenuExtenders();

		// Delegate wants paths as FStrings
		TArray<FString> SelectedPackagePaths;
		{
			// We need to try and resolve these paths back to items in order to query their attributes
			// This will only work for items that have already been discovered
			UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

			for (const FName& VirtualPathToSync : ContentSources.GetVirtualPaths())
			{
				const FContentBrowserItem ItemToSync = ContentBrowserData->GetItemAtPath(VirtualPathToSync, EContentBrowserItemTypeFilter::IncludeFolders);
				if (ItemToSync.IsValid())
				{
					FName PackagePath;
					if (ItemToSync.Legacy_TryGetPackagePath(PackagePath))
					{
						SelectedPackagePaths.Add(PackagePath.ToString());
					}
				}
			}
		}

		if (SelectedPackagePaths.Num() > 0)
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
			bCanBeModified = AssetToolsModule.Get().AllPassWritableFolderFilter(SelectedPackagePaths);

			TArray<TSharedPtr<FExtender>> Extenders;
			for (int32 i = 0; i < MenuExtenderDelegates.Num(); ++i)
			{
				if (MenuExtenderDelegates[i].IsBound())
				{
					Extenders.Add(MenuExtenderDelegates[i].Execute(SelectedPackagePaths));
				}
			}
			MenuExtender = FExtender::Combine(Extenders);
		}
	}

	FToolMenuContext ToolMenuContext(nullptr, MenuExtender, nullptr);
	AppendNewMenuContextObjects(InDomain, ContentSources.GetVirtualPaths(), ToolMenuContext, CommonContext, bCanBeModified);

	TSharedRef<SWidget> GeneratedWidget = UToolMenus::Get()->GenerateWidget("ContentBrowser.AddNewContextMenu", ToolMenuContext);
	GeneratedWidget->AddMetadata<FTagMetaData>(MakeShared<FTagMetaData>(TEXT("ContentBrowser.AddNewContextMenu")));
	return GeneratedWidget;
}

void SContentBrowser::PopulateAddNewContextMenu(class UToolMenu* Menu)
{
	const UContentBrowserDataMenuContext_AddNewMenu* ContextObject = Menu->FindContext<UContentBrowserDataMenuContext_AddNewMenu>();
	checkf(ContextObject, TEXT("Required context UContentBrowserDataMenuContext_AddNewMenu was missing!"));

	// Only add "New Folder" item if we do not have a collection selected
	FNewAssetOrClassContextMenu::FOnNewFolderRequested OnNewFolderRequested;
	if (ContextObject->OwnerDomain != EContentBrowserDataMenuContext_AddNewMenuDomain::PathView &&
		Algo::AllOf(CollectionSources, [](const TUniquePtr<FCollectionSource>& CollectionSource) { return CollectionSource->CollectionViewPtr->GetSelectedCollections().Num() == 0; }))
	{
		OnNewFolderRequested = FNewAssetOrClassContextMenu::FOnNewFolderRequested::CreateSP(this, &SContentBrowser::NewFolderRequested);
	}


	// New feature packs don't depend on the current paths, so we always add this item if it was requested
	FNewAssetOrClassContextMenu::FOnGetContentRequested OnGetContentRequested;
	
	OnGetContentRequested = FNewAssetOrClassContextMenu::FOnGetContentRequested::CreateSP(this, &SContentBrowser::OnAddContentRequested);

	FNewAssetOrClassContextMenu::MakeContextMenu(
		Menu,
		ContextObject->SelectedPaths,
		OnNewFolderRequested,
		OnGetContentRequested
		);
}

bool SContentBrowser::CanWriteToCurrentPath() const
{
	if (LegacyContentSourceWidgets && LegacyContentSourceWidgets->AssetViewPtr.IsValid())
	{
		const FAssetViewContentSources& ContentSources = LegacyContentSourceWidgets->AssetViewPtr->GetContentSources();
		if (ContentSources.GetVirtualPaths().Num() == 1)
		{
			FName CurrentPath = ContentSources.GetVirtualPaths()[0];
			if (!CachedCanWriteToCurrentPath.IsSet() || CachedCanWriteToCurrentPath.GetValue() != CurrentPath)
			{
				CachedCanWriteToCurrentPath = CurrentPath;
				bCachedCanWriteToCurrentPath = CanWriteToPath(FContentBrowserItemPath(CurrentPath, EContentBrowserPathType::Virtual));
			}

			return bCachedCanWriteToCurrentPath;
		}
		else
		{
			CachedCanWriteToCurrentPath.Reset();
			bCachedCanWriteToCurrentPath = false;
		}
	}

	return false;
}

bool SContentBrowser::CanWriteToPath(const FContentBrowserItemPath InPath) const
{
	// Reject if only virtual
	if (!InPath.HasInternalPath())
	{
		return false;
	}

	// Reject if path not inside a mount point
	if (!FPackageName::IsValidPath(InPath.GetInternalPathString()))
	{
		return false;
	}

	// Reject if folder writes blocked to path
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	const TSharedRef<FPathPermissionList>& WritableFolderFilter = AssetToolsModule.Get().GetWritableFolderPermissionList();
	if (!WritableFolderFilter->PassesStartsWithFilter(InPath.GetInternalPathName()))
	{
		return false;
	}

	return true;
}

void SContentBrowser::AddCustomTextFilter(const FCustomTextFilterData& FilterData, bool bApplyFilter)
{
	if (LegacyContentSourceWidgets && LegacyContentSourceWidgets->FilterListPtr.IsValid())
	{
		LegacyContentSourceWidgets->FilterListPtr->OnCreateCustomTextFilter(FilterData, bApplyFilter);
	}
}

void SContentBrowser::DeleteCustomTextFilterByLabel(const FText& FilterLabel)
{
	if (LegacyContentSourceWidgets && LegacyContentSourceWidgets->FilterListPtr.IsValid())
	{
		LegacyContentSourceWidgets->FilterListPtr->DeleteCustomTextFilterByLabel(FilterLabel);
	}
}

void SContentBrowser::ModifyCustomTextFilterByLabel(const FCustomTextFilterData& NewFilterData, const FText& FilterLabel)
{
	if (LegacyContentSourceWidgets && LegacyContentSourceWidgets->FilterListPtr.IsValid())
	{
		LegacyContentSourceWidgets->FilterListPtr->ModifyCustomTextFilterByLabel(NewFilterData, FilterLabel);
	}
}

bool SContentBrowser::IsAssetViewDoneFiltering()
{
	bool isDoneFiltering = false;

	if (LegacyContentSourceWidgets && LegacyContentSourceWidgets->AssetViewPtr.IsValid())
	{
		isDoneFiltering = !LegacyContentSourceWidgets->AssetViewPtr->HasItemsPendingFilter()
			&& !LegacyContentSourceWidgets->AssetViewPtr->HasThumbnailsPendingUpdate();
	}

	return isDoneFiltering;
}

bool SContentBrowser::IsAddNewEnabled() const
{
	return CanWriteToCurrentPath();
}

FText SContentBrowser::GetAddNewToolTipText() const
{
	const FAssetViewContentSources& ContentSources = LegacyContentSourceWidgets->AssetViewPtr->GetContentSources();

	if ( ContentSources.GetVirtualPaths().Num() == 1 )
	{
		const FString CurrentPath = ContentSources.GetVirtualPaths()[0].ToString();

		if (!CanWriteToCurrentPath())
		{
			return FText::Format(LOCTEXT("AddNewToolTip_CannotWrite", "Cannot write to path {0}..."), FText::FromString(CurrentPath));
		}

		return FText::Format( LOCTEXT("AddNewToolTip_AddNewContent", "Create new content in {0}...\nShortcut: Ctrl + RMB anywhere in the asset view"), FText::FromString(CurrentPath) );
	}
	else if ( ContentSources.GetVirtualPaths().Num() > 1 )
	{
		return LOCTEXT( "AddNewToolTip_MultiplePaths", "Cannot add content to multiple paths." );
	}
	
	return LOCTEXT( "AddNewToolTip_NoPath", "No path is selected as an add target." );
}

void SContentBrowser::PopulatePathViewFiltersMenu(UToolMenu* Menu)
{
	if (LegacyContentSourceWidgets && LegacyContentSourceWidgets->PathViewPtr.IsValid())
	{
		LegacyContentSourceWidgets->PathViewPtr->PopulatePathViewFiltersMenu(Menu);
	}
}

void SContentBrowser::ExtendAssetViewButtonMenuContext(FToolMenuContext& InMenuContext)
{
	UContentBrowserMenuContext* ContextObject = NewObject<UContentBrowserMenuContext>();
	ContextObject->ContentBrowser = SharedThis(this);
	InMenuContext.AddObject(ContextObject);
}

FReply SContentBrowser::OnSaveClicked()
{
	ContentBrowserUtils::SaveDirtyPackages();
	return FReply::Handled();
}

void SContentBrowser::OnAddContentRequested()
{
	IAddContentDialogModule& AddContentDialogModule = FModuleManager::LoadModuleChecked<IAddContentDialogModule>("AddContentDialog");
	FWidgetPath WidgetPath;
	FSlateApplication::Get().GeneratePathToWidgetChecked(AsShared(), WidgetPath);
	AddContentDialogModule.ShowDialog(WidgetPath.GetWindow());
}

void SContentBrowser::OnNewItemRequested(const FContentBrowserItem& NewItem)
{
	// Make sure we are showing the location of the new file (we may have created it in a folder)
	TArray<FString> SelectedPaths;
	SelectedPaths.Add(FPaths::GetPath(NewItem.GetVirtualPath().ToString()));

	const TArray<FString> CurrentlySelectedPath = LegacyContentSourceWidgets->PathViewPtr->GetSelectedPaths();

	// Only change the selected paths if needed. (To avoid adding an entry to navigation history when it is not needed)
	if (SelectedPaths != CurrentlySelectedPath)
	{
		LegacyContentSourceWidgets->PathViewPtr->SetSelectedPaths(SelectedPaths);
		PathSelected(SelectedPaths[0]);
	}
}

void SContentBrowser::OnItemSelectionChanged(const FContentBrowserItem& SelectedItem, ESelectInfo::Type SelectInfo, EContentBrowserViewContext ViewContext)
{
	if (ViewContext == EContentBrowserViewContext::AssetView)
	{		
		if (bIsPrimaryBrowser)
		{
			SyncGlobalSelectionSet();
		}

		// Notify 'asset selection changed' delegate
		FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		FContentBrowserModule::FOnAssetSelectionChanged& AssetSelectionChangedDelegate = ContentBrowserModule.GetOnAssetSelectionChanged();

		const TArray<FContentBrowserItem> SelectedItems = LegacyContentSourceWidgets->AssetViewPtr->GetSelectedItems();
		LegacyContentSourceWidgets->AssetContextMenu->SetSelectedItems(SelectedItems);

		{
			TArray<FSoftObjectPath> SelectedCollectionItems;
			for (const FContentBrowserItem& SelectedAssetItem : SelectedItems)
			{
				FSoftObjectPath CollectionItemId;
				if (SelectedAssetItem.TryGetCollectionId(CollectionItemId))
				{
					SelectedCollectionItems.Add(CollectionItemId);
				}
			}

			for (const TUniquePtr<FCollectionSource>& CollectionSource : CollectionSources)
			{
				CollectionSource->CollectionViewPtr->SetSelectedAssetPaths(SelectedCollectionItems);
			}
		}

		if (AssetSelectionChangedDelegate.IsBound())
		{
			TArray<FAssetData> SelectedAssets;
			for (const FContentBrowserItem& SelectedAssetItem : SelectedItems)
			{
				FAssetData ItemAssetData;
				if (SelectedAssetItem.Legacy_TryGetAssetData(ItemAssetData))
				{
					SelectedAssets.Add(MoveTemp(ItemAssetData));
				}
			}

			AssetSelectionChangedDelegate.Broadcast(SelectedAssets, bIsPrimaryBrowser);
		}
	}
	else if (ViewContext == EContentBrowserViewContext::FavoriteView)
	{
		checkf(!SelectedItem.IsValid() || SelectedItem.IsFolder(), TEXT("File item passed to path view selection!"));
		FavoritePathSelected(SelectedItem.IsValid() ? SelectedItem.GetVirtualPath().ToString() : FString());
	}
	else
	{
		checkf(!SelectedItem.IsValid() || SelectedItem.IsFolder(), TEXT("File item passed to path view selection!"));
		PathSelected(SelectedItem.IsValid() ? SelectedItem.GetVirtualPath().ToString() : FString());
	}
}

void SContentBrowser::OnItemsActivated(TArrayView<const FContentBrowserItem> ActivatedItems, EAssetTypeActivationMethod::Type ActivationMethod)
{
	FContentBrowserItem FirstActivatedFolder;

	// Batch these by their data sources
	TMap<UContentBrowserDataSource*, TArray<FContentBrowserItemData>> SourcesAndItems;
	for (const FContentBrowserItem& ActivatedItem : ActivatedItems)
	{
		if (ActivatedItem.IsFile())
		{
			FContentBrowserItem::FItemDataArrayView ItemDataArray = ActivatedItem.GetInternalItems();
			for (const FContentBrowserItemData& ItemData : ItemDataArray)
			{
				if (UContentBrowserDataSource* ItemDataSource = ItemData.GetOwnerDataSource())
				{
					TArray<FContentBrowserItemData>& ItemsForSource = SourcesAndItems.FindOrAdd(ItemDataSource);
					ItemsForSource.Add(ItemData);
				}
			}
		}

		if (ActivatedItem.IsFolder() && !FirstActivatedFolder.IsValid())
		{
			FirstActivatedFolder = ActivatedItem;
		}
	}

	if (SourcesAndItems.Num() == 0 && FirstActivatedFolder.IsValid())
	{
		// Activate the selected folder
		FolderEntered(FirstActivatedFolder);
		return;
	}

	// Execute the operation now
	for (const auto& SourceAndItemsPair : SourcesAndItems)
	{
		if (ActivationMethod == EAssetTypeActivationMethod::Previewed)
		{
			SourceAndItemsPair.Key->BulkPreviewItems(SourceAndItemsPair.Value);
		}
		else
		{
			for (const FContentBrowserItemData& ItemToEdit : SourceAndItemsPair.Value)
			{
				FText EditErrorMsg;
				if (!SourceAndItemsPair.Key->CanEditItem(ItemToEdit, &EditErrorMsg) && !SourceAndItemsPair.Key->CanViewItem(ItemToEdit, &EditErrorMsg))
				{
					AssetViewUtils::ShowErrorNotifcation(EditErrorMsg);
				}
			}
			
			if (!SourceAndItemsPair.Key->BulkEditItems(SourceAndItemsPair.Value))
			{
				static const FText ErrorMessage = LOCTEXT("EditItemsFailure", "Unable to edit assets");

				FNotificationInfo WarningNotification(ErrorMessage);
				WarningNotification.ExpireDuration = 5.0f;
				WarningNotification.Hyperlink = FSimpleDelegate::CreateStatic([](){ FMessageLog("LoadErrors").Open(EMessageSeverity::Info, true); });
				WarningNotification.HyperlinkText = LOCTEXT("LoadObjectHyperlink", "Show Message Log");
				WarningNotification.bFireAndForget = true;
				FSlateNotificationManager::Get().AddNotification(WarningNotification);
			}
		}
	}
}

FReply SContentBrowser::ToggleLockClicked()
{
	bIsLocked = !bIsLocked;

	return FReply::Handled();
}

FReply SContentBrowser::DockInLayoutClicked()
{
	FContentBrowserSingleton::Get().DockContentBrowserDrawer();

	return FReply::Handled();
}

FText SContentBrowser::GetLockMenuText() const
{
	return IsLocked() ? LOCTEXT("ContentBrowserLockMenu_Unlock", "Unlock Content Browser") : LOCTEXT("ContentBrowserLockMenu_Lock", "Lock Content Browser");
}

FSlateIcon SContentBrowser::GetLockIcon() const
{
	static const FSlateIcon Unlocked = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Unlock");
	static const FSlateIcon Locked = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Lock");
	return IsLocked() ? Locked : Unlocked;
}

const FSlateBrush* SContentBrowser::GetLockIconBrush() const
{
	static const FName Unlock = "Icons.Unlock";
	static const FName Lock = "Icons.Lock";

	return FAppStyle::Get().GetBrush(IsLocked() ? Lock : Unlock);
}

EVisibility SContentBrowser::GetSourcesViewVisibility() const
{
	return bSourcesViewExpanded ? EVisibility::Visible : EVisibility::Collapsed;
}

void SContentBrowser::SetSourcesViewExpanded(bool bExpanded)
{
	bSourcesViewExpanded = bExpanded;

	if (FContentBrowserInstanceConfig* EditorConfig = GetMutableInstanceConfig())
	{
		EditorConfig->bSourcesExpanded = bSourcesViewExpanded;
		UContentBrowserConfig::Get()->SaveEditorConfig();
	}


	// Notify 'Sources View Expanded' delegate
	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>( TEXT("ContentBrowser") );
	FContentBrowserModule::FOnSourcesViewChanged& SourcesViewChangedDelegate = ContentBrowserModule.GetOnSourcesViewChanged();
	if (SourcesViewChangedDelegate.IsBound())
	{
		SourcesViewChangedDelegate.Broadcast(bSourcesViewExpanded);
	}
}

FReply SContentBrowser::SourcesViewExpandClicked()
{
	SetSourcesViewExpanded(!bSourcesViewExpanded);
	return FReply::Handled();
}

void SContentBrowser::OnContentBrowserSettingsChanged(FName PropertyName)
{
	if (PropertyName.IsNone())
	{
		// Ensure the path is set to the correct view mode
		UpdatePath();
	}
}

void SContentBrowser::OnConsoleVariableChanged()
{
	UpdatePrivateContentFeatureEnabled(true /* bUpdateFilterIfChanged */);
}

void SContentBrowser::UpdatePrivateContentFeatureEnabled(bool bUpdateFilterIfChanged)
{
}

void SContentBrowser::OnLegacyContentSourceEnabled()
{
	// Re-bind our commands so they work properly
	BindCommands();

	// Create the content browser's default widgets and set them as the child widget contents
	LegacyContentSource->SetContent(CreateLegacyAssetViewWidgets());

	// Load our settings
	LoadSettings(InstanceName);

	// Sanity sync to make sure the global selection set is synced
	SyncGlobalSelectionSet();
}

void SContentBrowser::OnLegacyContentSourceDisabled()
{
	// Save our settings before destroying the widgets
	SaveSettings();

	// Unbind commands
	UnbindCommands();

	// Set the child widget contents to null and destroy all asset view widgets and they will be re-bound when the legacy content source is enabled
	LegacyContentSource->SetContent(SNullWidget::NullWidget);
	LegacyContentSourceWidgets.Reset();
	
	// Unbind all delegates since we don't need them anymore, and 
	FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();

	TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
	CollectionManagerModule.Get().GetCollectionContainers(CollectionContainers);

	for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
	{
		CollectionContainer->OnIsHiddenChanged().RemoveAll(this);
		CollectionContainer->OnCollectionRenamed().RemoveAll(this);
		CollectionContainer->OnCollectionDestroyed().RemoveAll(this);
		CollectionContainer->OnCollectionUpdated().RemoveAll(this);
	}

	CollectionManagerModule.Get().OnCollectionContainerCreated().RemoveAll(this);
	CollectionManagerModule.Get().OnCollectionContainerDestroyed().RemoveAll(this);
	
	CollectionSources.Empty();

	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	ContentBrowserData->OnItemDataUpdated().RemoveAll(this);
}

TSharedRef<SWidget> SContentBrowser::CreateLegacyAssetViewWidgets()
{
	LegacyContentSourceWidgets = MakeUnique<FLegacyContentSourceWidgets>();
	
	using namespace UE::Editor::ContentBrowser::Private;

	// The final widget that conatins all child widgets
	TSharedRef<SWidget> FinalWidget = SNullWidget::NullWidget;

	LegacyContentSourceWidgets->PathContextMenu = MakeShareable(new FPathContextMenu( AsShared() ));
	LegacyContentSourceWidgets->PathContextMenu->SetOnRenameFolderRequested(FPathContextMenu::FOnRenameFolderRequested::CreateSP(this, &SContentBrowser::OnRenameRequested));
	LegacyContentSourceWidgets->PathContextMenu->SetOnFolderDeleted(FPathContextMenu::FOnFolderDeleted::CreateSP(this, &SContentBrowser::OnOpenedFolderDeleted));
	LegacyContentSourceWidgets->PathContextMenu->SetOnFolderFavoriteToggled(FPathContextMenu::FOnFolderFavoriteToggled::CreateSP(this, &SContentBrowser::ToggleFolderFavorite));
	LegacyContentSourceWidgets->PathContextMenu->SetOnFolderFavoriteEdited(FPathContextMenu::FOnFolderFavoriteEdited::CreateSP(this, &SContentBrowser::EditFolderFavorite));
	LegacyContentSourceWidgets->PathContextMenu->SetOnPrivateContentEditToggled(FPathContextMenu::FOnPrivateContentEditToggled::CreateSP(this, &SContentBrowser::TogglePrivateContentEdit));

	// Currently this controls the asset count
	const bool bShowBottomToolbar = InitConfig.bShowBottomToolbar;

	const FContentBrowserInstanceConfig* Config = GetConstInstanceConfig();

	const bool bShowAssetAccessSpecifier = Config ? Config->bShowAssetAccessSpecifier : false;
	
	LegacyContentSourceWidgets->AssetViewPtr = SNew(SAssetView)
			.ThumbnailLabel(InitConfig.ThumbnailLabel)
			//.ThumbnailScale(Config != nullptr ? Config->ThumbnailScale : 0.18f)
			.InitialViewType(InitConfig.InitialAssetViewType)
			.OnNewItemRequested(this, &SContentBrowser::OnNewItemRequested)
			.OnItemSelectionChanged(this, &SContentBrowser::OnItemSelectionChanged, EContentBrowserViewContext::AssetView)
			.OnItemsActivated(this, &SContentBrowser::OnItemsActivated)
			.OnGetItemContextMenu(this, &SContentBrowser::GetItemContextMenu, EContentBrowserViewContext::AssetView)
			.OnItemRenameCommitted(this, &SContentBrowser::OnItemRenameCommitted)
			.FrontendFilters(FrontendFilters)
			.TextFilter(TextFilter)
			.ShowRedirectors(this, &SContentBrowser::ShouldShowRedirectors)
			.HighlightedText(this, &SContentBrowser::GetHighlightedText)
			.ShowBottomToolbar(bShowBottomToolbar)
			.ShowViewOptions(false) // We control this for the main content browser
			.AllowThumbnailEditMode(true)
			.AllowThumbnailHintLabel(false)
			.CanShowFolders(InitConfig.bCanShowFolders)
			.CanShowClasses(InitConfig.bCanShowClasses)
			.CanShowRealTimeThumbnails(InitConfig.bCanShowRealTimeThumbnails)
			.CanShowDevelopersFolder(InitConfig.bCanShowDevelopersFolder)
			.CanShowFavorites(true)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserAssets")))
			.OwningContentBrowser(SharedThis(this))
			.OnSearchOptionsChanged(this, &SContentBrowser::HandleAssetViewSearchOptionsChanged)
			.bShowPathViewFilters(true)
			.FillEmptySpaceInTileView(true)
			.ShowDisallowedAssetClassAsUnsupportedItems(true)
			.AllowCustomView(true)
			.ShowAssetAccessSpecifier(bShowAssetAccessSpecifier);
	
	TSharedRef<SWidget> ViewOptions = SNullWidget::NullWidget;

	// Note, for backwards compatibility ShowBottomToolbar controls the visibility of view options so we respect that here
	if (bShowBottomToolbar)
	{
		if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
		{
			ViewOptions =
				SNew(SActionButton)
				.ActionButtonType(EActionButtonType::Simple)
				.OnGetMenuContent(LegacyContentSourceWidgets->AssetViewPtr.ToSharedRef(), &SAssetView::GetViewButtonContent)
				.Icon(FAppStyle::Get().GetBrush("Icons.Settings"))
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("AssetViewMenuButton")));
		}
		else
		{
			ViewOptions =
				SNew(SComboButton)
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
				.OnGetMenuContent(LegacyContentSourceWidgets->AssetViewPtr.ToSharedRef(), &SAssetView::GetViewButtonContent)
				.HasDownArrow(false)
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("AssetViewMenuButton")))
				.ButtonContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4.0, 0.0f)
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
					]

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(4.0, 0.0f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Settings", "Settings"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				];
		}
	}
	
	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		constexpr float ToolBarVerticalPadding = 4.0f; // ToolBar total height should be 36.0f
		constexpr float ToolBarButtonHeight = 24.0f; // Used for buttons that should appear to be part of the ToolBar, but aren't
		constexpr float PanelInsetPadding = 2.0f;
		constexpr float SourceTreeSectionPadding = 2.0f;

		const TSharedRef<SWidget> AssetView = CreateAssetView(&InitConfig);

		CreateFavoritesView(&InitConfig);
		CreatePathView(&InitConfig);

		{
			SAssignNew(LegacyContentSourceWidgets->SourceTreePtr, SContentBrowserSourceTree)

			+ SContentBrowserSourceTree::Slot()
			.AreaWidget(FavoritesArea)
			.Size(0.2f)
			.Visibility(this, &SContentBrowser::GetFavoriteFolderVisibility)

			+ SContentBrowserSourceTree::Slot()
			.AreaWidget(PathArea)
			.Size(0.8f);
		}

		LegacyContentSourceWidgets->SearchBoxSizeSwitcher = MakeShared<TWidgetDesiredSizeSwitcher<EAxis::X>>(
			LegacyContentSourceWidgets->SearchBoxPtr,
			nullptr,
			FInt16Range(100)); // @note: this is overridden in menu registration

		LegacyContentSourceWidgets->NavigationToolBarWidget = CreateNavigationToolBar(&InitConfig);

		TSharedRef<SWidget> ToolBarWidget = CreateToolBar(&InitConfig);

		LegacyContentSourceWidgets->SearchBoxSizeSwitcher->SetMaxSizeReferenceWidget(ToolBarWidget.ToSharedPtr());

		TSharedPtr<SBox> AssetViewNavigationToolBarContainer = nullptr;
		TSharedPtr<SBox> SourceTreeAndAssetViewNavigationToolBarContainer = nullptr;

		FinalWidget = 
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
			.Thickness(2.0f)
		]

		// Source / Tree + Assets
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.0f)
		[
			SNew(SHorizontalBox)

			// Tree + Assets + Navigation Bar
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(FMargin(0.0f))
			[
				SNew(SVerticalBox)

				// Assets/tree
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.Padding(FMargin(0.0f))
				[
					// The tree/assets splitter
					SAssignNew(LegacyContentSourceWidgets->PathAssetSplitterPtr, SSplitter)
					.PhysicalSplitterHandleSize(PanelInsetPadding)

					// Sources View
					+ SSplitter::Slot()
					.Resizable(true)
					.SizeRule(SSplitter::SizeToContent)
					.OnSlotResized(this, &SContentBrowser::OnPathViewBoxColumnResized)
					[
						SNew(SBox)
						.Padding(0.0f)
						.Visibility(this, &SContentBrowser::GetSourcesViewVisibility)
						.WidthOverride(this, &SContentBrowser::GetPathViewBoxWidthOverride)
						[
							SNew(SBorder)
							.Padding(SourceTreeSectionPadding)
							.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
							[
								// Panel background, seen between items
								SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
								.BorderBackgroundColor(FStyleColors::Panel)
								.HAlign(HAlign_Fill)
								.VAlign(VAlign_Fill)
								.Padding(0.0f)
								[
									LegacyContentSourceWidgets->SourceTreePtr.ToSharedRef()
								]
							]
						]
					]

					// Asset View
					+ SSplitter::Slot()
					.Value(0.75f)
					[
						SNew(SBox)
						.Padding(0.0f)
						[
							SNew(SVerticalBox)

							+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(SBorder)
								.Padding(FMargin(3, ToolBarVerticalPadding))
								.BorderImage(bIsDrawer ? FStyleDefaults::GetNoBrush() : FAppStyle::Get().GetBrush("Brushes.Panel"))
								[
									SNew(SHorizontalBox)

									+ SHorizontalBox::Slot()
									.FillWidth(1.0f)
									.HAlign(HAlign_Fill)
									.VAlign(VAlign_Center)
									.Padding(5, 0, 0, 0)
									[
										ToolBarWidget
									]

									+ SHorizontalBox::Slot()
									.AutoWidth()
									.HAlign(HAlign_Right)
									.VAlign(VAlign_Top)
									.Padding(0.f, 2.f, 0.f, 0.f)
									[
										CreateDrawerDockButton(&InitConfig)
									]

									+ SHorizontalBox::Slot()
									.AutoWidth()
									.HAlign(HAlign_Right)
									.VAlign(VAlign_Top)
									.Padding(5.0f, 2.0f, 0.0f, 0.0f)
									[
										SNew(SBox)
										.HeightOverride(ToolBarButtonHeight)
										[
											ViewOptions
										]
									]
								]
							]

							+ SVerticalBox::Slot()
							[
								AssetView
							]

							// Navigation ToolBar
							+ SVerticalBox::Slot()
							.AutoHeight()
							.HAlign(HAlign_Fill)
							.Padding(FMargin(0, PanelInsetPadding, 0, 0))
							[
								SNew(SBorder)
								.Padding(FMargin(3))
								.BorderImage(bIsDrawer ? FStyleDefaults::GetNoBrush() : FAppStyle::Get().GetBrush("Brushes.Panel"))
								[
									SAssignNew(AssetViewNavigationToolBarContainer, SBox)
									.HAlign(HAlign_Fill)
									.Padding(0)
									[
										LegacyContentSourceWidgets->NavigationToolBarWidget.ToSharedRef()
									]
								]
							]
						]
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0)
				[
					SAssignNew(SourceTreeAndAssetViewNavigationToolBarContainer, SBox)
					.Padding(0)
				]
			]
		];

		LegacyContentSourceWidgets->SourceTreeSplitterNumFixedSlots = LegacyContentSourceWidgets->SourceTreePtr->GetSplitter()->GetChildren()->NumSlot();
	}
	else
	{
		FinalWidget = 
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding( 0, 0, 0, 0 )
		[
			SNew( SBorder )
			.Padding( FMargin( 3 ) )
			.BorderImage(bIsDrawer ? FStyleDefaults::GetNoBrush() : FAppStyle::Get().GetBrush("Brushes.Panel"))
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.Padding(5, 0, 0, 0)
				[
					CreateToolBar(&InitConfig)
				]
				// History Back Button
				+SHorizontalBox::Slot()
				.Padding(10, 0, 0, 0)
				.AutoWidth()
				[
					SNew(SButton)
					.VAlign(EVerticalAlignment::VAlign_Center)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText( this, &SContentBrowser::GetHistoryBackTooltip )
					.ContentPadding( FMargin(1, 0) )
					.OnClicked(this, &SContentBrowser::BackClicked)
					.IsEnabled(this, &SContentBrowser::IsBackEnabled)
					.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserHistoryBack")))
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.CircleArrowLeft"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
				// History Forward Button
				+ SHorizontalBox::Slot()
				.Padding(2, 0, 0, 0)
				.AutoWidth()
				[
					SNew(SButton)
					.VAlign(EVerticalAlignment::VAlign_Center)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText( this, &SContentBrowser::GetHistoryForwardTooltip )
					.ContentPadding( FMargin(1, 0) )
					.OnClicked(this, &SContentBrowser::ForwardClicked)
					.IsEnabled(this, &SContentBrowser::IsForwardEnabled)
					.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserHistoryForward")))
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.CircleArrowRight"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]

				// Path
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				.FillWidth(1.0f)
				.Padding(2, 0, 0, 0)
				[
					SAssignNew(LegacyContentSourceWidgets->NavigationBar, SNavigationBar)
					.OnPathClicked(this, &SContentBrowser::OnPathClicked)
					.GetPathMenuContent(this, &SContentBrowser::OnGetCrumbDelimiterContent)
					.GetComboOptions(this, &SContentBrowser::GetRecentPaths)
					.OnNavigateToPath(this, &SContentBrowser::OnNavigateToPath)
					.OnCompletePrefix(this, &SContentBrowser::OnCompletePathPrefix)
					.OnGetEditPathAsText(this, &SContentBrowser::OnGetEditPathAsText)
					.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserPath")))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					CreateLockButton(&InitConfig)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					CreateDrawerDockButton(&InitConfig)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5.0f, 0.0f, 0.0f, 0.0f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					ViewOptions
				]
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SSeparator)
			.Thickness(2.0f)
		]

		// Assets/tree
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0.0f)
		[
			// The tree/assets splitter
			SAssignNew(LegacyContentSourceWidgets->PathAssetSplitterPtr, SSplitter)
			.PhysicalSplitterHandleSize(2.0f)

			// Sources View
			+ SSplitter::Slot()
			.Resizable(true)
			.SizeRule(SSplitter::SizeToContent)
			.OnSlotResized(this, &SContentBrowser::OnPathViewBoxColumnResized)
			[
				SNew(SBox)
				.Padding(FMargin(4.f))
				.Visibility(this, &SContentBrowser::GetSourcesViewVisibility)
				.WidthOverride(this, &SContentBrowser::GetPathViewBoxWidthOverride)
				[
					SNew(SBorder)
					.Padding(FMargin(0))
					.BorderImage(FAppStyle::GetBrush("Brushes.Recessed"))

					[
						SAssignNew(LegacyContentSourceWidgets->PathFavoriteSplitterPtr, SSplitter)
						.Clipping(EWidgetClipping::ClipToBounds)
						.PhysicalSplitterHandleSize(2.0f)
						.HitDetectionSplitterHandleSize(8.0f)
						.Orientation(EOrientation::Orient_Vertical)
						.MinimumSlotHeight(26.0f)
						.Visibility( this, &SContentBrowser::GetSourcesViewVisibility )
						+SSplitter::Slot()
						.SizeRule(TAttribute<SSplitter::ESizeRule>(this, &SContentBrowser::GetFavoritesAreaSizeRule))
						.MinSize(TAttribute<float>(this, &SContentBrowser::GetFavoritesAreaMinSize))
						.Value(0.2f)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
							.Padding(0.0f, 2.0f, 0.0f, 0.0f)
							[
								CreateFavoritesView(&InitConfig)
							]
						]

						+SSplitter::Slot()
						.SizeRule(TAttribute<SSplitter::ESizeRule>(this, &SContentBrowser::GetPathAreaSizeRule))
						.MinSize(29.0f)
						.Value(0.8f)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
							.Padding(0.0f, 2.0f, 0.0f, 0.0f)
							[
								CreatePathView(&InitConfig)
							]
						]
					]
				]
			]

			// Asset View
			+ SSplitter::Slot()
			.Value(0.75f)
			[
				CreateAssetView(&InitConfig)
			]
		];

		LegacyContentSourceWidgets->SourceTreeSplitterNumFixedSlots = LegacyContentSourceWidgets->PathFavoriteSplitterPtr->GetChildren()->NumSlot();
	}

	LegacyContentSourceWidgets->AssetContextMenu = MakeShared<FAssetContextMenu>(LegacyContentSourceWidgets->AssetViewPtr);
	LegacyContentSourceWidgets->AssetContextMenu->BindCommands(Commands);
	LegacyContentSourceWidgets->AssetContextMenu->SetOnShowInPathsViewRequested( FAssetContextMenu::FOnShowInPathsViewRequested::CreateSP(this, &SContentBrowser::OnShowInPathsViewRequested) );
	LegacyContentSourceWidgets->AssetContextMenu->SetOnRenameRequested( FAssetContextMenu::FOnRenameRequested::CreateSP(this, &SContentBrowser::OnRenameRequested) );
	LegacyContentSourceWidgets->AssetContextMenu->SetOnDuplicateRequested( FAssetContextMenu::FOnDuplicateRequested::CreateSP(this, &SContentBrowser::OnDuplicateRequested) );
	LegacyContentSourceWidgets->AssetContextMenu->SetOnAssetViewRefreshRequested( FAssetContextMenu::FOnAssetViewRefreshRequested::CreateSP( this, &SContentBrowser::OnAssetViewRefreshRequested) );

	TOptional<FCollectionRef> SelectedCollection;

	SelectedCollection = InitConfig.SelectedCollection;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Check for someone using the deprecated SelectedCollectionName instead of SelectedCollection.
	if (!SelectedCollection.GetValue().IsValid() && InitConfig.SelectedCollectionName.Name != NAME_None)
	{
		SelectedCollection = FCollectionRef(FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(), InitConfig.SelectedCollectionName);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (SelectedCollection.IsSet() && SelectedCollection.GetValue().IsValid())
	{
		// Select the specified collection by default
		FAssetViewContentSources DefaultContentSources(SelectedCollection.GetValue());
		LegacyContentSourceWidgets->AssetViewPtr->SetContentSources(DefaultContentSources);
	}
	else
	{
		// Select /Game by default
		const FName DefaultInvariantPath(TEXT("/Game"));
		FName DefaultVirtualPath;
		IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual(DefaultInvariantPath, DefaultVirtualPath);

		FAssetViewContentSources DefaultContentSources(DefaultVirtualPath);
		LegacyContentSourceWidgets->AssetViewPtr->SetContentSources(DefaultContentSources);
	}

	if (bHasInitConfig)
	{
		// Make sure the sources view is initially visible if we were asked to show it
		SetSourcesViewExpanded(InitConfig.bExpandSourcesView && InitConfig.bUseSourcesView);
	}
	// else
	{
		// in case we do not have a config, see what the global default settings are for the Sources Panel
		const bool bSourcesExpanded = Config ? Config->bSourcesExpanded : true;
		SetSourcesViewExpanded(bSourcesExpanded);
	}
	
	// Bindings to manage history when items are deleted
	FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();

	TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
	CollectionManagerModule.Get().GetCollectionContainers(CollectionContainers);

	int32 InsertIndex = 0;
	for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
	{
		if (!CollectionContainer->IsHidden())
		{
			AddSlotForCollectionContainer(InsertIndex++, CollectionContainer.ToSharedRef());
		}

		CollectionContainer->OnIsHiddenChanged().AddSP(this, &SContentBrowser::HandleIsHiddenChanged);
		CollectionContainer->OnCollectionRenamed().AddSP(this, &SContentBrowser::HandleCollectionRenamed);
		CollectionContainer->OnCollectionDestroyed().AddSP(this, &SContentBrowser::HandleCollectionRemoved);
		CollectionContainer->OnCollectionUpdated().AddSP(this, &SContentBrowser::HandleCollectionUpdated);
	}

	CollectionManagerModule.Get().OnCollectionContainerCreated().AddSP(this, &SContentBrowser::HandleCollectionContainerAdded);
	CollectionManagerModule.Get().OnCollectionContainerDestroyed().AddSP(this, &SContentBrowser::HandleCollectionContainerRemoved);

	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	ContentBrowserData->OnItemDataUpdated().AddSP(this, &SContentBrowser::HandleItemDataUpdated);

	LegacyContentSourceWidgets->FavoritePathViewPtr->SetTreeTitle(LOCTEXT("Favorites", "Favorites"));

	// Initialize the search options
	HandleAssetViewSearchOptionsChanged();
	
	return FinalWidget;
}

void SContentBrowser::OnSearchOverrideToggled()
{
	if (LegacyContentSourceWidgets->AssetSearchOverrideInstance && LegacyContentSourceWidgets->AssetSearchOverrideInstance->IsOverrideActive())
	{
		// Use enter since this was based on a user action
		LegacyContentSourceWidgets->AssetSearchOverrideInstance->OnSearchTextCommitted(LegacyContentSourceWidgets->SearchBoxPtr->GetText(), ETextCommit::OnEnter);
		SetSearchBoxText(FText::GetEmpty());
	}
	else
	{
		SetSearchBoxText(LegacyContentSourceWidgets->SearchBoxPtr->GetText());
	}
	
	LegacyContentSourceWidgets->AssetViewPtr->RequestQuickFrontendListRefresh();
}

FReply SContentBrowser::BackClicked()
{
	HistoryManager.GoBack();

	return FReply::Handled();
}

FReply SContentBrowser::ForwardClicked()
{
	HistoryManager.GoForward();

	return FReply::Handled();
}

bool SContentBrowser::HandleFilterBySelectionCommandCanExecute() const
{
	if (!LegacyContentSourceWidgets.IsValid() || !LegacyContentSourceWidgets->FilterListPtr.IsValid())
	{
		return false;
	}

	const TArray<FAssetData> SelectedAssets = LegacyContentSourceWidgets->AssetViewPtr->GetSelectedAssets();
	return !SelectedAssets.IsEmpty();
}

void SContentBrowser::HandleFilterBySelectionCommandExecute()
{
	if (!LegacyContentSourceWidgets.IsValid() || !LegacyContentSourceWidgets->FilterListPtr.IsValid())
	{
		return;
	}

	const TArray<FAssetData> SelectedAssets = LegacyContentSourceWidgets->AssetViewPtr->GetSelectedAssets();
	if (SelectedAssets.IsEmpty())
	{
		return;
	}

	TArray<FTopLevelAssetPath> SelectedAssetPaths;
	SelectedAssetPaths.Reserve(SelectedAssets.Num());
	for (const FAssetData& SelectedAsset : SelectedAssets)
	{
		SelectedAssetPaths.Add(SelectedAsset.AssetClassPath);
	}

	LegacyContentSourceWidgets->FilterListPtr->SetActiveFilters(SelectedAssetPaths);
}

void SContentBrowser::HandleFavoriteShortcutCommandExecute(FName InCommandName)
{
	if (InCommandName.IsNone())
	{
		return;
	}

	const UContentBrowserFavoriteProjectSettings* FavoriteSettings = UContentBrowserFavoriteProjectSettings::Get();
	if (!FavoriteSettings)
	{
		return;
	}

	// Look for the item with this command assigned
	for (const FContentBrowserFavoriteItem& FavoriteItem : FavoriteSettings->FavoriteItems)
	{
		if (FavoriteItem.ShortcutCommandName == InCommandName)
		{
			const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
			ContentBrowserModule.Get().SyncBrowserToFolders({ FavoriteItem.Path });
			break;
		}
	}
}

bool SContentBrowser::HandleRenameCommandCanExecute() const
{
	// The order of these conditions are carefully crafted to match the logic of the context menu summoning, as this callback 
	// is shared between the path and asset views, and is given zero context as to which one is making the request
	// Change this logic at your peril, lest the the dominoes fall like a house of cards (checkmate)
	if (LegacyContentSourceWidgets->PathViewPtr->HasFocusedDescendants())
	{
		// Prefer the path view if it has focus, which may be the case when using the keyboard to invoke the action,
		// but will be false when using the context menu (which isn't an issue, as the path view clears the asset view 
		// selection when invoking its context menu to avoid the selection ambiguity present when using the keyboard)
		if (LegacyContentSourceWidgets->PathViewPtr->GetSelectedFolderItems().Num() > 0)
		{
			return LegacyContentSourceWidgets->PathContextMenu->CanExecuteRename();
		}
	}
	else if (LegacyContentSourceWidgets->AssetViewPtr->HasFocusedDescendants())
	{
		// Prefer the asset menu if the asset view has focus (which may be the case when using the keyboard to invoke
		// the action), as it is the only thing that is updated with the correct selection context when no context menu 
		// has been invoked, and can work for both folders and files
		if (LegacyContentSourceWidgets->AssetViewPtr->GetSelectedItems().Num() > 0)
		{
			return LegacyContentSourceWidgets->AssetContextMenu->CanExecuteRename();
		}
	}
	else if (LegacyContentSourceWidgets->AssetViewPtr->GetSelectedFolderItems().Num() > 0)
	{
		// Folder selection takes precedence over file selection for the context menu used...
		return LegacyContentSourceWidgets->PathContextMenu->CanExecuteRename();
	}
	else if (LegacyContentSourceWidgets->AssetViewPtr->GetSelectedFileItems().Num() > 0)
	{
		// ... but the asset view still takes precedence over an unfocused path view unless it has no selection
		return LegacyContentSourceWidgets->AssetContextMenu->CanExecuteRename();
	}
	else if (LegacyContentSourceWidgets->PathViewPtr->GetSelectedFolderItems().Num() > 0)
	{
		return LegacyContentSourceWidgets->PathContextMenu->CanExecuteRename();
	}

	return false;
}

void SContentBrowser::HandleRenameCommand()
{
	// The order of these conditions are carefully crafted to match the logic of the context menu summoning, as this callback 
	// is shared between the path and asset views, and is given zero context as to which one is making the request
	// Change this logic at your peril, lest the the dominoes fall like a house of cards (checkmate)
	if (LegacyContentSourceWidgets->PathViewPtr->HasFocusedDescendants())
	{
		// Prefer the path view if it has focus, which may be the case when using the keyboard to invoke the action,
		// but will be false when using the context menu (which isn't an issue, as the path view clears the asset view 
		// selection when invoking its context menu to avoid the selection ambiguity present when using the keyboard)
		if (LegacyContentSourceWidgets->PathViewPtr->GetSelectedFolderItems().Num() > 0)
		{
			LegacyContentSourceWidgets->PathContextMenu->ExecuteRename(EContentBrowserViewContext::PathView);
		}
	}
	else if (LegacyContentSourceWidgets->AssetViewPtr->HasFocusedDescendants())
	{
		// Prefer the asset menu if the asset view has focus (which may be the case when using the keyboard to invoke
		// the action), as it is the only thing that is updated with the correct selection context when no context menu 
		// has been invoked, and can work for both folders and files
		if (LegacyContentSourceWidgets->AssetViewPtr->GetSelectedItems().Num() > 0)
		{
			LegacyContentSourceWidgets->AssetContextMenu->ExecuteRename(EContentBrowserViewContext::AssetView);
		}
	}
	else if (LegacyContentSourceWidgets->AssetViewPtr->GetSelectedFolderItems().Num() > 0)
	{
		// Folder selection takes precedence over file selection for the context menu used...
		LegacyContentSourceWidgets->PathContextMenu->ExecuteRename(EContentBrowserViewContext::AssetView);
	}
	else if (LegacyContentSourceWidgets->AssetViewPtr->GetSelectedFileItems().Num() > 0)
	{
		// ... but the asset view still takes precedence over an unfocused path view unless it has no selection
		LegacyContentSourceWidgets->AssetContextMenu->ExecuteRename(EContentBrowserViewContext::AssetView);
	}
	else if (LegacyContentSourceWidgets->PathViewPtr->GetSelectedFolderItems().Num() > 0)
	{
		LegacyContentSourceWidgets->PathContextMenu->ExecuteRename(EContentBrowserViewContext::PathView);
	}
}

bool SContentBrowser::HandleSaveAssetCommandCanExecute() const
{
	if (LegacyContentSourceWidgets->AssetViewPtr->GetSelectedFileItems().Num() > 0 && !LegacyContentSourceWidgets->AssetViewPtr->IsRenamingAsset())
	{
		return LegacyContentSourceWidgets->AssetContextMenu->CanExecuteSaveAsset();
	}

	return false;
}

void SContentBrowser::HandleSaveAssetCommand()
{
	if (LegacyContentSourceWidgets->AssetViewPtr->GetSelectedFileItems().Num() > 0)
	{
		LegacyContentSourceWidgets->AssetContextMenu->ExecuteSaveAsset();
	}
}

void SContentBrowser::HandleSaveAllCurrentFolderCommand() const
{
	LegacyContentSourceWidgets->PathContextMenu->ExecuteSaveFolder();
}

void SContentBrowser::HandleResaveAllCurrentFolderCommand() const
{
	LegacyContentSourceWidgets->PathContextMenu->ExecuteResaveFolder();
}

void SContentBrowser::CopySelectedAssetPathCommand() const
{
	LegacyContentSourceWidgets->PathContextMenu->CopySelectedFolder();
}

bool SContentBrowser::HandleDeleteCommandCanExecute() const
{
	// The order of these conditions are carefully crafted to match the logic of the context menu summoning, as this callback 
	// is shared between the path and asset views, and is given zero context as to which one is making the request
	// Change this logic at your peril, lest the the dominoes fall like a house of cards (checkmate)
	if (LegacyContentSourceWidgets->PathViewPtr->HasFocusedDescendants())
	{
		// Prefer the path view if it has focus, which may be the case when using the keyboard to invoke the action,
		// but will be false when using the context menu (which isn't an issue, as the path view clears the asset view 
		// selection when invoking its context menu to avoid the selection ambiguity present when using the keyboard)
		if (LegacyContentSourceWidgets->PathViewPtr->GetSelectedFolderItems().Num() > 0)
		{
			return LegacyContentSourceWidgets->PathContextMenu->CanExecuteDelete();
		}
	}
	else if (LegacyContentSourceWidgets->AssetViewPtr->HasFocusedDescendants())
	{
		// Prefer the asset menu if the asset view has focus (which may be the case when using the keyboard to invoke
		// the action), as it is the only thing that is updated with the correct selection context when no context menu 
		// has been invoked, and can work for both folders and files
		if (LegacyContentSourceWidgets->AssetViewPtr->GetSelectedItems().Num() > 0)
		{
			return LegacyContentSourceWidgets->AssetContextMenu->CanExecuteDelete();
		}
	}
	else if (LegacyContentSourceWidgets->AssetViewPtr->GetSelectedFolderItems().Num() > 0)
	{
		// Folder selection takes precedence over file selection for the context menu used...
		return LegacyContentSourceWidgets->PathContextMenu->CanExecuteDelete();
	}
	else if (LegacyContentSourceWidgets->AssetViewPtr->GetSelectedFileItems().Num() > 0)
	{
		// ... but the asset view still takes precedence over an unfocused path view unless it has no selection
		return LegacyContentSourceWidgets->AssetContextMenu->CanExecuteDelete();
	}
	else if (LegacyContentSourceWidgets->FavoritePathViewPtr->GetSelectedFolderItems().Num() > 0)
	{
		return true;
	}
	else if (LegacyContentSourceWidgets->PathViewPtr->GetSelectedFolderItems().Num() > 0)
	{
		return LegacyContentSourceWidgets->PathContextMenu->CanExecuteDelete();
	}

	// To avoid processing a delete operation for viewport/outliner when content browser is focus but nothing selected
	if (LegacyContentSourceWidgets->PathViewPtr->HasFocusedDescendants() || LegacyContentSourceWidgets->AssetViewPtr->HasFocusedDescendants())
	{
		return true;
	}

	return false;
}

void SContentBrowser::HandleDeleteCommandExecute()
{
	// The order of these conditions are carefully crafted to match the logic of the context menu summoning, as this callback 
	// is shared between the path and asset views, and is given zero context as to which one is making the request
	// Change this logic at your peril, lest the the dominoes fall like a house of cards (checkmate)
	if (LegacyContentSourceWidgets->PathViewPtr->HasFocusedDescendants())
	{
		// Prefer the path view if it has focus, which may be the case when using the keyboard to invoke the action,
		// but will be false when using the context menu (which isn't an issue, as the path view clears the asset view 
		// selection when invoking its context menu to avoid the selection ambiguity present when using the keyboard)
		if (LegacyContentSourceWidgets->PathViewPtr->GetSelectedFolderItems().Num() > 0)
		{
			LegacyContentSourceWidgets->PathContextMenu->ExecuteDelete();
		}
	}
	else if (LegacyContentSourceWidgets->AssetViewPtr->HasFocusedDescendants())
	{
		// Prefer the asset menu if the asset view has focus (which may be the case when using the keyboard to invoke
		// the action), as it is the only thing that is updated with the correct selection context when no context menu 
		// has been invoked, and can work for both folders and files
		if (LegacyContentSourceWidgets->AssetViewPtr->GetSelectedItems().Num() > 0)
		{
			LegacyContentSourceWidgets->AssetContextMenu->ExecuteDelete();
		}
	}
	else if (LegacyContentSourceWidgets->AssetViewPtr->GetSelectedFolderItems().Num() > 0)
	{
		// Folder selection takes precedence over file selection for the context menu used...
		LegacyContentSourceWidgets->PathContextMenu->ExecuteDelete();
	}
	else if (LegacyContentSourceWidgets->AssetViewPtr->GetSelectedFileItems().Num() > 0)
	{
		// ... but the asset view still takes precedence over an unfocused path view unless it has no selection
		LegacyContentSourceWidgets->AssetContextMenu->ExecuteDelete();
	}
	else if (LegacyContentSourceWidgets->FavoritePathViewPtr->GetSelectedFolderItems().Num() > 0)
	{
		HandleDeleteFavorite(LegacyContentSourceWidgets->PathContextMenu->GetParentContent());
	}
	else if (LegacyContentSourceWidgets->PathViewPtr->GetSelectedFolderItems().Num() > 0)
	{
		LegacyContentSourceWidgets->PathContextMenu->ExecuteDelete();
	}
}

void SContentBrowser::HandleDeleteFavorite(TSharedPtr<SWidget> ParentWidget)
{
	TArray<FContentBrowserItem> SelectedFolders = LegacyContentSourceWidgets->FavoritePathViewPtr->GetSelectedFolderItems();
	if (ParentWidget.IsValid() && SelectedFolders.Num() > 0)
	{
		FText Prompt;
		if (SelectedFolders.Num() == 1)
		{
			Prompt = FText::Format(LOCTEXT("FavoriteDeleteConfirm_Single", "Remove favorite '{0}'?"), SelectedFolders[0].GetDisplayName());
		}
		else
		{
			Prompt = FText::Format(LOCTEXT("FavoriteDeleteConfirm_Multiple", "Remove {0} favorites?"), SelectedFolders.Num());
		}

		// Spawn a confirmation dialog since this is potentially a highly destructive operation
		if (ContentBrowserUtils::DisplayConfirmationDialog(LOCTEXT("FavoriteRemoveConfirm_Yes", "Remove"), Prompt))
		{
			for (const FContentBrowserItem& Folder : SelectedFolders)
			{
				ContentBrowserUtils::RemoveFavoriteFolder(FContentBrowserItemPath(Folder.GetVirtualPath(), EContentBrowserPathType::Virtual));
			}

			GConfig->Flush(false, GEditorPerProjectIni);
			LegacyContentSourceWidgets->FavoritePathViewPtr->Populate();
		}
	}
}

void SContentBrowser::HandleOpenAssetsOrFoldersCommandExecute()
{
	LegacyContentSourceWidgets->AssetViewPtr->OnOpenAssetsOrFolders();
}

void SContentBrowser::HandlePreviewAssetsCommandExecute()
{
	LegacyContentSourceWidgets->AssetViewPtr->OnPreviewAssets();
}

void SContentBrowser::HandleCreateNewFolderCommandExecute()
{
	TArray<FString> SelectedPaths = LegacyContentSourceWidgets->PathViewPtr->GetSelectedPaths();

	// only create folders when a single path is selected
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	const bool bCanCreateNewFolder = SelectedPaths.Num() == 1 && ContentBrowserData->CanCreateFolder(*SelectedPaths[0], nullptr);;

	if (bCanCreateNewFolder)
	{
		CreateNewFolder(
			SelectedPaths.Num() > 0
			? SelectedPaths[0]
			: FString(),
			FOnCreateNewFolder::CreateSP(LegacyContentSourceWidgets->AssetViewPtr.Get(), &SAssetView::NewFolderItemRequested));
	}
}

void SContentBrowser::HandleGoUpToParentFolder()
{
	const FString SelectedPath = LegacyContentSourceWidgets->PathViewPtr->GetSelectedPath();
	int32 LastSlashIdx = INDEX_NONE;
	if (ensure(SelectedPath.FindLastChar('/', LastSlashIdx)))
	{
		const int32 ChopCount = SelectedPath.Len() - LastSlashIdx;
		const FString NewPathSelection = SelectedPath.LeftChop(ChopCount);
		SetSelectedPaths({NewPathSelection}, true);
	}
}

bool SContentBrowser::HandleCanGoUpToParentFolder() const
{
	// Allow going up if there's one non-root folder selected
	TArray<FString> SelectedPaths = LegacyContentSourceWidgets->PathViewPtr->GetSelectedPaths();
	if (SelectedPaths.Num() == 1)
	{
		int32 LastSlashIdx = INDEX_NONE;
		if (SelectedPaths[0].FindLastChar('/', LastSlashIdx))
		{
			return LastSlashIdx > 0;
		}
	}
	return false;
}

void SContentBrowser::GetSelectionState(TArray<FAssetData>& SelectedAssets, TArray<FString>& SelectedPaths)
{
	SelectedAssets.Reset();
	SelectedPaths.Reset();
	if (LegacyContentSourceWidgets->AssetViewPtr->HasAnyUserFocusOrFocusedDescendants())
	{
		SelectedAssets = LegacyContentSourceWidgets->AssetViewPtr->GetSelectedAssets();
		SelectedPaths = LegacyContentSourceWidgets->AssetViewPtr->GetSelectedFolders();
	}
	else if (LegacyContentSourceWidgets->PathViewPtr->HasAnyUserFocusOrFocusedDescendants())
	{
		SelectedPaths = LegacyContentSourceWidgets->PathViewPtr->GetSelectedPaths();
	}
	// Fallback
	else if (const FContentBrowserModule* ContentBrowserModule = FModuleManager::GetModulePtr<FContentBrowserModule>("ContentBrowser"))
	{
		ContentBrowserModule->Get().GetSelectedAssets(SelectedAssets);
		ContentBrowserModule->Get().GetSelectedFolders(SelectedPaths);
	}
}

bool SContentBrowser::IsBackEnabled() const
{
	return HistoryManager.CanGoBack();
}

bool SContentBrowser::IsForwardEnabled() const
{
	return HistoryManager.CanGoForward();
}

FText SContentBrowser::GetHistoryBackTooltip() const
{
	if ( HistoryManager.CanGoBack() )
	{
		return FText::Format( LOCTEXT("HistoryBackTooltipFmt", "Back to {0}"), HistoryManager.GetBackDesc() );
	}
	return FText::GetEmpty();
}

FText SContentBrowser::GetHistoryForwardTooltip() const
{
	if ( HistoryManager.CanGoForward() )
	{
		return FText::Format( LOCTEXT("HistoryForwardTooltipFmt", "Forward to {0}"), HistoryManager.GetForwardDesc() );
	}
	return FText::GetEmpty();
}

void SContentBrowser::SyncGlobalSelectionSet()
{
	if (!ContentSourcesContainer->IsLegacyContentSourceActive())
	{
		return;
	}
	
	USelection* EditorSelection = GEditor->GetSelectedObjects();
	if ( !ensure( EditorSelection != NULL ) )
	{
		return;
	}

	// Get the selected assets in the asset view
	const TArray<FAssetData>& SelectedAssets = LegacyContentSourceWidgets->AssetViewPtr->GetSelectedAssets();

	EditorSelection->BeginBatchSelectOperation();
	{
		TSet< UObject* > SelectedObjects;
		// Lets see what the user has selected and add any new selected objects to the global selection set
		for ( auto AssetIt = SelectedAssets.CreateConstIterator(); AssetIt; ++AssetIt )
		{
			// Grab the object if it is loaded
			if ( (*AssetIt).IsAssetLoaded() )
			{
				UObject* FoundObject = (*AssetIt).GetAsset();
				if( FoundObject != NULL && FoundObject->GetClass() != UObjectRedirector::StaticClass() )
				{
					SelectedObjects.Add( FoundObject );

					// Select this object!
					EditorSelection->Select( FoundObject );
				}
			}
		}
		
		// List of objects that need to be removed from the global selection set
		TArray<UObject*> EditorSelectedObjects;
		EditorSelection->GetSelectedObjects(EditorSelectedObjects);
		for (UObject* CurEditorObject : EditorSelectedObjects)
		{
			if (CurEditorObject && !SelectedObjects.Contains(CurEditorObject))
			{
				EditorSelection->Deselect(CurEditorObject);
			}
		}
	}
	EditorSelection->EndBatchSelectOperation();
}

void SContentBrowser::UpdatePath()
{
	if (ContentSourcesContainer->IsLegacyContentSourceActive())
	{
		ContentBrowserUtils::UpdateNavigationBar(LegacyContentSourceWidgets->NavigationBar, LegacyContentSourceWidgets->AssetViewPtr, LegacyContentSourceWidgets->PathViewPtr);
	
		CachedCanWriteToCurrentPath.Reset();
	}
}

void SContentBrowser::OnFilterChanged()
{
	TArray<TSharedRef<const FPathPermissionList>> CustomPermissionLists;
	FARFilter Filter = LegacyContentSourceWidgets->FilterListPtr->GetCombinedBackendFilter(CustomPermissionLists);
	LegacyContentSourceWidgets->AssetViewPtr->SetBackendFilter(Filter, &CustomPermissionLists);

	// Notify 'filter changed' delegate
	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>( TEXT("ContentBrowser") );
	ContentBrowserModule.GetOnFilterChanged().Broadcast(Filter, bIsPrimaryBrowser);
}

FText SContentBrowser::GetPathText() const
{
	FText PathLabelText;

	if ( IsFilteredBySource() )
	{
		const FAssetViewContentSources& ContentSources = LegacyContentSourceWidgets->AssetViewPtr->GetContentSources();

		// At least one source is selected
		const int32 NumSources = ContentSources.GetVirtualPaths().Num() + ContentSources.GetCollections().Num();

		if (NumSources > 0)
		{
			PathLabelText = FText::FromName(ContentSources.HasVirtualPaths() ? ContentSources.GetVirtualPaths()[0] : ContentSources.GetCollections()[0].Name);

			if (NumSources > 1)
			{
				PathLabelText = FText::Format(LOCTEXT("PathTextFmt", "{0} and {1} {1}|plural(one=other,other=others)..."), PathLabelText, NumSources - 1);
			}
		}
	}
	else
	{
		PathLabelText = LOCTEXT("AllAssets", "All Assets");
	}

	return PathLabelText;
}

bool SContentBrowser::IsFilteredBySource() const
{
	const FAssetViewContentSources& ContentSources = LegacyContentSourceWidgets->AssetViewPtr->GetContentSources();
	return !ContentSources.IsEmpty();
}

void SContentBrowser::OnItemRenameCommitted(TArrayView<const FContentBrowserItem> Items)
{
	// After a rename is committed we allow an implicit sync so as not to
	// disorientate the user if they are looking at a parent folder

	const bool bAllowImplicitSync = true;
	const bool bDisableFiltersThatHideAssets = false;
	SyncToItems(Items, bAllowImplicitSync, bDisableFiltersThatHideAssets);
}

void SContentBrowser::OnShowInPathsViewRequested(TArrayView<const FContentBrowserItem> ItemsToFind)
{
	SyncToItems(ItemsToFind);
}

void SContentBrowser::OnRenameRequested(const FContentBrowserItem& Item, EContentBrowserViewContext ViewContext)
{
	FText RenameErrorMsg;
	if (Item.CanRename(nullptr, &RenameErrorMsg))
	{
		if (ViewContext == EContentBrowserViewContext::AssetView)
		{
			LegacyContentSourceWidgets->AssetViewPtr->RenameItem(Item);
		}
		else
		{
			LegacyContentSourceWidgets->PathViewPtr->RenameFolderItem(Item);
		}
	}
	else
	{
		AssetViewUtils::ShowErrorNotifcation(RenameErrorMsg);
	}
}

void SContentBrowser::OnOpenedFolderDeleted()
{
	// Since the contents of the asset view have just been deleted, set the selected path to the default "/Game"
	TArray<FString> DefaultSelectedPaths;
	DefaultSelectedPaths.Add(TEXT("/Game"));
	LegacyContentSourceWidgets->PathViewPtr->SetSelectedPaths(DefaultSelectedPaths);
	PathSelected(TEXT("/Game"));
}

void SContentBrowser::OnDuplicateRequested(TArrayView<const FContentBrowserItem> OriginalItems)
{
	if (OriginalItems.Num() == 1)
	{
		// Asynchronous duplication of a single item
		const FContentBrowserItem& OriginalItem = OriginalItems[0];
		if (ensureAlwaysMsgf(OriginalItem.IsFile(), TEXT("Can only duplicate files!")))
		{
			FText DuplicateErrorMsg;
			if (OriginalItem.CanDuplicate(&DuplicateErrorMsg))
			{
				const FContentBrowserItemDataTemporaryContext NewItemContext = OriginalItem.Duplicate();
				if (NewItemContext.IsValid())
				{
					LegacyContentSourceWidgets->AssetViewPtr->NewFileItemRequested(NewItemContext);
				}
			}
			else
			{
				AssetViewUtils::ShowErrorNotifcation(DuplicateErrorMsg);
			}
		}
	}
	else if (OriginalItems.Num() > 1)
	{
		// Batch these by their data sources
		TMap<UContentBrowserDataSource*, TArray<FContentBrowserItemData>> SourcesAndItems;
		for (const FContentBrowserItem& OriginalItem : OriginalItems)
		{
			FContentBrowserItem::FItemDataArrayView ItemDataArray = OriginalItem.GetInternalItems();
			for (const FContentBrowserItemData& ItemData : ItemDataArray)
			{
				if (UContentBrowserDataSource* ItemDataSource = ItemData.GetOwnerDataSource())
				{
					FText DuplicateErrorMsg;
					if (ItemDataSource->CanDuplicateItem(ItemData, &DuplicateErrorMsg))
					{
						TArray<FContentBrowserItemData>& ItemsForSource = SourcesAndItems.FindOrAdd(ItemDataSource);
						ItemsForSource.Add(ItemData);
					}
					else
					{
						AssetViewUtils::ShowErrorNotifcation(DuplicateErrorMsg);
					}
				}
			}
		}

		// Execute the operation now
		TArray<FContentBrowserItemData> NewItems;
		for (const auto& SourceAndItemsPair : SourcesAndItems)
		{
			SourceAndItemsPair.Key->BulkDuplicateItems(SourceAndItemsPair.Value, NewItems);
		}

		// Sync the view to the new items
		if (NewItems.Num() > 0)
		{
			TArray<FContentBrowserItem> ItemsToSync;
			for (const FContentBrowserItemData& NewItem : NewItems)
			{
				ItemsToSync.Emplace(NewItem);
			}

			SyncToItems(ItemsToSync);
		}
	}
}

void SContentBrowser::OnAssetViewRefreshRequested()
{
	LegacyContentSourceWidgets->AssetViewPtr->RequestSlowFullListRefresh();
}

void SContentBrowser::HandleCollectionContainerAdded(const TSharedRef<ICollectionContainer>& CollectionContainer)
{
	if (!CollectionContainer->IsHidden())
	{
		ShowCollectionContainer(CollectionContainer);
	}

	CollectionContainer->OnIsHiddenChanged().AddSP(this, &SContentBrowser::HandleIsHiddenChanged);
	CollectionContainer->OnCollectionRenamed().AddSP(this, &SContentBrowser::HandleCollectionRenamed);
	CollectionContainer->OnCollectionDestroyed().AddSP(this, &SContentBrowser::HandleCollectionRemoved);
	CollectionContainer->OnCollectionUpdated().AddSP(this, &SContentBrowser::HandleCollectionUpdated);
}

void SContentBrowser::ShowCollectionContainer(const TSharedRef<ICollectionContainer>& CollectionContainer)
{
	TArray<FSoftObjectPath> SelectedCollectionItems;
	for (const FContentBrowserItem& SelectedAssetItem : LegacyContentSourceWidgets->AssetViewPtr->GetSelectedItems())
	{
		FSoftObjectPath CollectionItemId;
		if (SelectedAssetItem.TryGetCollectionId(CollectionItemId))
		{
			SelectedCollectionItems.Add(CollectionItemId);
		}
	}

	int32 InsertIndex = INDEX_NONE;

	TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
	FCollectionManagerModule::GetModule().Get().GetVisibleCollectionContainers(CollectionContainers);

	for (int32 Index = 0; Index < CollectionContainers.Num(); ++Index)
	{
		if (CollectionContainer == CollectionContainers[Index])
		{
			InsertIndex = Index;
			break;
		}

		// Make sure FCollectionManagerModule and CollectionSources maintain the same order.
		if (!ensure(Index < CollectionSources.Num() && CollectionContainers[Index] == CollectionSources[Index]->GetCollectionContainer()))
		{
			break;
		}
	}

	AddSlotForCollectionContainer(InsertIndex, CollectionContainer).CollectionViewPtr->SetSelectedAssetPaths(SelectedCollectionItems);
}

void SContentBrowser::HandleCollectionContainerRemoved(const TSharedRef<ICollectionContainer>& CollectionContainer)
{
	CollectionContainer->OnIsHiddenChanged().RemoveAll(this);
	CollectionContainer->OnCollectionRenamed().RemoveAll(this);
	CollectionContainer->OnCollectionDestroyed().RemoveAll(this);
	CollectionContainer->OnCollectionUpdated().RemoveAll(this);

	HideCollectionContainer(CollectionContainer);
}

void SContentBrowser::HideCollectionContainer(const TSharedRef<ICollectionContainer>& CollectionContainer)
{
	RemoveSlotForCollectionContainer(CollectionContainer);

	auto UpdateContentSources = [&CollectionContainer](const FAssetViewContentSources& ContentSources, FAssetViewContentSources& OutNewContentSources)
		{
			auto Predicate = [&CollectionContainer](const FCollectionRef& Collection)
				{
					return CollectionContainer == Collection.Container;
				};

			if (ContentSources.GetCollections().ContainsByPredicate(Predicate))
			{
				TArray<FCollectionRef> NewCollections = ContentSources.GetCollections();
				NewCollections.RemoveAll(Predicate);

				OutNewContentSources = ContentSources;
				OutNewContentSources.SetCollections(NewCollections);
				return true;
			}
			return false;
		};

	{
		FAssetViewContentSources NewContentSources;
		if (UpdateContentSources(LegacyContentSourceWidgets->AssetViewPtr->GetContentSources(), NewContentSources))
		{
			LegacyContentSourceWidgets->AssetViewPtr->SetContentSources(NewContentSources);
		}
	}

	HistoryManager.RemoveHistoryData([&UpdateContentSources](FHistoryData& HistoryData)
		{
			FAssetViewContentSources NewContentSources;
			if (UpdateContentSources(HistoryData.ContentSources, NewContentSources))
			{
				if (!NewContentSources.HasCollections())
				{
					// Remove the history if we removed the last collection.
					return true;
				}

				HistoryData.ContentSources = MoveTemp(NewContentSources);
			}
			return false;
		});
}

void SContentBrowser::HandleIsHiddenChanged(ICollectionContainer& CollectionContainer, bool bIsHidden)
{
	if (bIsHidden)
	{
		HideCollectionContainer(CollectionContainer.AsShared());
	}
	else
	{
		ShowCollectionContainer(CollectionContainer.AsShared());
	}
}

void SContentBrowser::HandleCollectionRemoved(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection)
{
	// Remove Collection from ContentSources.
	auto UpdateContentSources = [&CollectionContainer, &Collection](const FAssetViewContentSources& ContentSources, FAssetViewContentSources& OutNewContentSources)
		{
			const int32 FoundIndex = ContentSources.GetCollections().IndexOfByPredicate([&CollectionContainer, &Collection](const FCollectionRef& CollectionRef)
				{
					return &CollectionContainer == CollectionRef.Container.Get() &&
						Collection.Name == CollectionRef.Name &&
						Collection.Type == CollectionRef.Type;
				});
			if (FoundIndex != INDEX_NONE)
			{
				TArray<FCollectionRef> NewCollections = ContentSources.GetCollections();
				NewCollections.RemoveAt(FoundIndex);

				OutNewContentSources = ContentSources;
				OutNewContentSources.SetCollections(NewCollections);
				return true;
			}
			return false;
		};

	{
		FAssetViewContentSources NewContentSources;
		if (UpdateContentSources(LegacyContentSourceWidgets->AssetViewPtr->GetContentSources(), NewContentSources))
		{
			LegacyContentSourceWidgets->AssetViewPtr->SetContentSources(NewContentSources);
		}
	}

	HistoryManager.RemoveHistoryData([&UpdateContentSources](FHistoryData& HistoryData)
	{
		FAssetViewContentSources NewContentSources;
		if (UpdateContentSources(HistoryData.ContentSources, NewContentSources))
		{
			if (!NewContentSources.HasCollections())
			{
				// Remove the history if we removed the last collection.
				return true;
			}

			HistoryData.ContentSources = MoveTemp(NewContentSources);
		}
		return false;
	});
}

void SContentBrowser::HandleCollectionRenamed(ICollectionContainer& CollectionContainer, const FCollectionNameType& OriginalCollection, const FCollectionNameType& NewCollection)
{
	// Replaces OriginalCollection with NewCollection in ContentSources.
	auto UpdateContentSources = [&CollectionContainer, &OriginalCollection, &NewCollection](const FAssetViewContentSources& ContentSources, FAssetViewContentSources& OutNewContentSources)
		{
			const int32 FoundIndex = ContentSources.GetCollections().IndexOfByPredicate([&CollectionContainer, &OriginalCollection](const FCollectionRef& Collection)
				{
					return &CollectionContainer == Collection.Container.Get() &&
						OriginalCollection.Name == Collection.Name &&
						OriginalCollection.Type == Collection.Type;
				});
			if (FoundIndex != INDEX_NONE)
			{
				TArray<FCollectionRef> NewCollections = ContentSources.GetCollections();
				NewCollections[FoundIndex] = FCollectionRef(CollectionContainer.AsShared(), NewCollection);

				OutNewContentSources = ContentSources;
				OutNewContentSources.SetCollections(NewCollections);
				return true;
			}
			return false;
		};

	{
		FAssetViewContentSources NewContentSources;
		if (UpdateContentSources(LegacyContentSourceWidgets->AssetViewPtr->GetContentSources(), NewContentSources))
		{
			LegacyContentSourceWidgets->AssetViewPtr->SetContentSources(NewContentSources);
		}
	}

	HistoryManager.RewriteHistoryData([&UpdateContentSources](FHistoryData& HistoryData)
		{
			FAssetViewContentSources NewContentSources;
			if (UpdateContentSources(HistoryData.ContentSources, NewContentSources))
			{
				HistoryData.ContentSources = MoveTemp(NewContentSources);
			}
		});
}

void SContentBrowser::HandleCollectionUpdated(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection)
{
	const FAssetViewContentSources& ContentSources = LegacyContentSourceWidgets->AssetViewPtr->GetContentSources();

	// If we're currently viewing the dynamic collection that was updated, make sure our active filter text is up-to-date
	if (ContentSources.IsDynamicCollection())
	{
		const FCollectionRef& DynamicCollection = ContentSources.GetCollections()[0];
		if (DynamicCollection.Container.Get() == &CollectionContainer &&
			DynamicCollection.Name == Collection.Name &&
			DynamicCollection.Type == Collection.Type)
		{
			FString DynamicQueryString;
			DynamicCollection.Container->GetDynamicQueryText(DynamicCollection.Name, DynamicCollection.Type, DynamicQueryString);

			const FText DynamicQueryText = FText::FromString(DynamicQueryString);
			SetSearchBoxText(DynamicQueryText);
			LegacyContentSourceWidgets->SearchBoxPtr->SetText(DynamicQueryText);
		}
	}
}

void SContentBrowser::HandlePathRemoved(const FName Path)
{
	HistoryManager.RemoveHistoryData([&Path](const FHistoryData& HistoryData)
	{
		return HistoryData.ContentSources.GetVirtualPaths().Num() == 1
			&& HistoryData.ContentSources.GetVirtualPaths().Contains(Path);
	});
}

void SContentBrowser::HandleItemDataUpdated(TArrayView<const FContentBrowserItemDataUpdate> InUpdatedItems)
{
	for (const FContentBrowserItemDataUpdate& ItemDataUpdate : InUpdatedItems)
	{
		if (!ItemDataUpdate.GetItemData().IsFolder())
		{
			continue;
		}

		switch (ItemDataUpdate.GetUpdateType())
		{
		case EContentBrowserItemUpdateType::Moved:
			HandlePathRemoved(ItemDataUpdate.GetPreviousVirtualPath());
			break;

		case EContentBrowserItemUpdateType::Removed:
			HandlePathRemoved(ItemDataUpdate.GetItemData().GetVirtualPath());
			break;

		case EContentBrowserItemUpdateType::Unmounted:
			HandlePathRemoved(ItemDataUpdate.GetItemData().GetVirtualPath());
			break;

		default:
			break;
		}
	}
}

FText SContentBrowser::GetSearchAssetsHintText() const
{
	if (LegacyContentSourceWidgets->PathViewPtr.IsValid())
	{
		TArray<FContentBrowserItem> Paths = LegacyContentSourceWidgets->PathViewPtr->GetSelectedFolderItems();
		if (Paths.Num() > 0)
		{
			FString SearchHint = NSLOCTEXT( "ContentBrowser", "SearchBoxPartialHint", "Search" ).ToString();
			SearchHint += TEXT(" ");
			for(int32 i = 0; i < Paths.Num(); i++)
			{
				const FString PathDisplayNameString = Paths[i].GetDisplayName().ToString();
				if (PathDisplayNameString.IsEmpty())
				{
					continue;
				}

				SearchHint += PathDisplayNameString;
				if (i + 1 < Paths.Num())
				{
					SearchHint += ", ";
				}
			}

			return FText::FromString(SearchHint);
		}
	}
	
	return NSLOCTEXT( "ContentBrowser", "SearchBoxHint", "Search Assets" );
}

void ExtractAssetSearchFilterTerms(const FText& SearchText, FString* OutFilterKey, FString* OutFilterValue, int32* OutSuggestionInsertionIndex)
{
	const FString SearchString = SearchText.ToString();

	if (OutFilterKey)
	{
		OutFilterKey->Reset();
	}
	if (OutFilterValue)
	{
		OutFilterValue->Reset();
	}
	if (OutSuggestionInsertionIndex)
	{
		*OutSuggestionInsertionIndex = SearchString.Len();
	}

	// Build the search filter terms so that we can inspect the tokens
	FTextFilterExpressionEvaluator LocalFilter(ETextFilterExpressionEvaluatorMode::Complex);
	LocalFilter.SetFilterText(SearchText);

	// Inspect the tokens to see what the last part of the search term was
	// If it was a key->value pair then we'll use that to control what kinds of results we show
	// For anything else we just use the text from the last token as our filter term to allow incremental auto-complete
	const TArray<FExpressionToken>& FilterTokens = LocalFilter.GetFilterExpressionTokens();
	if (FilterTokens.Num() > 0)
	{
		const FExpressionToken& LastToken = FilterTokens.Last();

		// If the last token is a text token, then consider it as a value and walk back to see if we also have a key
		if (LastToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
		{
			if (OutFilterValue)
			{
				*OutFilterValue = LastToken.Context.GetString();
			}
			if (OutSuggestionInsertionIndex)
			{
				*OutSuggestionInsertionIndex = FMath::Min(*OutSuggestionInsertionIndex, LastToken.Context.GetCharacterIndex());
			}

			if (FilterTokens.IsValidIndex(FilterTokens.Num() - 2))
			{
				const FExpressionToken& ComparisonToken = FilterTokens[FilterTokens.Num() - 2];
				if (ComparisonToken.Node.Cast<TextFilterExpressionParser::FEqual>())
				{
					if (FilterTokens.IsValidIndex(FilterTokens.Num() - 3))
					{
						const FExpressionToken& KeyToken = FilterTokens[FilterTokens.Num() - 3];
						if (KeyToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
						{
							if (OutFilterKey)
							{
								*OutFilterKey = KeyToken.Context.GetString();
							}
							if (OutSuggestionInsertionIndex)
							{
								*OutSuggestionInsertionIndex = FMath::Min(*OutSuggestionInsertionIndex, KeyToken.Context.GetCharacterIndex());
							}
						}
					}
				}
			}
		}

		// If the last token is a comparison operator, then walk back and see if we have a key
		else if (LastToken.Node.Cast<TextFilterExpressionParser::FEqual>())
		{
			if (FilterTokens.IsValidIndex(FilterTokens.Num() - 2))
			{
				const FExpressionToken& KeyToken = FilterTokens[FilterTokens.Num() - 2];
				if (KeyToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
				{
					if (OutFilterKey)
					{
						*OutFilterKey = KeyToken.Context.GetString();
					}
					if (OutSuggestionInsertionIndex)
					{
						*OutSuggestionInsertionIndex = FMath::Min(*OutSuggestionInsertionIndex, KeyToken.Context.GetCharacterIndex());
					}
				}
			}
		}
	}
}

void SContentBrowser::OnAssetSearchSuggestionFilter(const FText& SearchText, TArray<FAssetSearchBoxSuggestion>& PossibleSuggestions, FText& SuggestionHighlightText) const
{
	// We don't bind the suggestion list, so this list should be empty as we populate it here based on the search term
	check(PossibleSuggestions.Num() == 0);

	FString FilterKey;
	FString FilterValue;
	ExtractAssetSearchFilterTerms(SearchText, &FilterKey, &FilterValue, nullptr);

	auto PassesValueFilter = [&FilterValue](const FString& InOther)
	{
		return FilterValue.IsEmpty() || InOther.Contains(FilterValue);
	};

	auto SortPredicate = [](const FAssetSearchBoxSuggestion& A, const FAssetSearchBoxSuggestion& B)
	{
		return A.DisplayName.CompareTo(B.DisplayName) < 0;
	};

	if (FilterKey.IsEmpty() || (FilterKey == TEXT("Type") || FilterKey == TEXT("Class")))
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		TArray< TWeakPtr<IAssetTypeActions> > AssetTypeActionsList;
		AssetToolsModule.Get().GetAssetTypeActionsList(AssetTypeActionsList);

		const int32 StartIndex = PossibleSuggestions.Num();
		const FText TypesCategoryName = NSLOCTEXT("ContentBrowser", "TypesCategoryName", "Types");
		for (auto TypeActionsIt = AssetTypeActionsList.CreateConstIterator(); TypeActionsIt; ++TypeActionsIt)
		{
			if ((*TypeActionsIt).IsValid())
			{
				const TSharedPtr<IAssetTypeActions> TypeActions = (*TypeActionsIt).Pin();
				if (TypeActions->GetSupportedClass())
				{
					const FString TypeName = TypeActions->GetSupportedClass()->GetName();
					const FText TypeDisplayName = TypeActions->GetSupportedClass()->GetDisplayNameText();
					FString TypeSuggestion = FString::Printf(TEXT("Type=%s"), *TypeName);
					if (PassesValueFilter(TypeSuggestion))
					{
						PossibleSuggestions.Add(FAssetSearchBoxSuggestion{ MoveTemp(TypeSuggestion), TypeDisplayName, TypesCategoryName });
					}
				}
			}
		}

		Algo::Sort(TArrayView<FAssetSearchBoxSuggestion>(PossibleSuggestions.GetData() + StartIndex, PossibleSuggestions.Num() - StartIndex), SortPredicate);
	}

	if (FilterKey.IsEmpty() || (FilterKey == TEXT("Collection") || FilterKey == TEXT("Tag")))
	{
		ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();

		TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
		CollectionManager.GetVisibleCollectionContainers(CollectionContainers);

		const int32 StartIndex = PossibleSuggestions.Num();
		const FText CollectionsCategoryName = NSLOCTEXT("ContentBrowser", "CollectionsCategoryName", "Collections");
		TArray<FCollectionNameType> AllCollections;
		for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
		{
			AllCollections.Reset();
			CollectionContainer->GetCollections(AllCollections);

			for (const FCollectionNameType& Collection : AllCollections)
			{
				FString CollectionName = Collection.Name.ToString();
				FString CollectionSuggestion = FString::Printf(TEXT("Collection=%s"), *CollectionName);
				if (PassesValueFilter(CollectionSuggestion))
				{
					PossibleSuggestions.Add(FAssetSearchBoxSuggestion{ MoveTemp(CollectionSuggestion), FText::FromString(MoveTemp(CollectionName)), CollectionsCategoryName });
				}
			}
		}

		Algo::Sort(TArrayView<FAssetSearchBoxSuggestion>(PossibleSuggestions.GetData() + StartIndex, PossibleSuggestions.Num() - StartIndex), SortPredicate);

		// Remove duplicate collection names (either from different containers or types).
		PossibleSuggestions.SetNum(StartIndex + Algo::Unique(
			TArrayView<FAssetSearchBoxSuggestion>(PossibleSuggestions.GetData() + StartIndex, PossibleSuggestions.Num() - StartIndex),
			[](const FAssetSearchBoxSuggestion& A, const FAssetSearchBoxSuggestion& B)
			{
				return A.SuggestionString.Compare(B.SuggestionString) == 0;
			}));
	}

	if (FilterKey.IsEmpty())
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();

		const int32 StartIndex = PossibleSuggestions.Num();
		const FText MetaDataCategoryName = NSLOCTEXT("ContentBrowser", "MetaDataCategoryName", "Meta-Data");
		FString TagNameStr;
		AssetRegistry.ReadLockEnumerateAllTagToAssetDatas(
			[&PassesValueFilter, &PossibleSuggestions, &MetaDataCategoryName, &TagNameStr](FName TagName, IAssetRegistry::FEnumerateAssetDatasFunc EnumerateAssets)
			{
				TagName.ToString(TagNameStr);
				if (PassesValueFilter(TagNameStr))
				{
					PossibleSuggestions.Add(FAssetSearchBoxSuggestion{ TagNameStr, FText::FromString(TagNameStr), MetaDataCategoryName });
				}

				return true;
			});

		Algo::Sort(TArrayView<FAssetSearchBoxSuggestion>(PossibleSuggestions.GetData() + StartIndex, PossibleSuggestions.Num() - StartIndex), SortPredicate);
	}

	SuggestionHighlightText = FText::FromString(FilterValue);
}

FText SContentBrowser::OnAssetSearchSuggestionChosen(const FText& SearchText, const FString& Suggestion) const
{
	int32 SuggestionInsertionIndex = 0;
	ExtractAssetSearchFilterTerms(SearchText, nullptr, nullptr, &SuggestionInsertionIndex);

	FString SearchString = SearchText.ToString();
	SearchString.RemoveAt(SuggestionInsertionIndex, SearchString.Len() - SuggestionInsertionIndex, EAllowShrinking::No);
	SearchString.Append(Suggestion);

	return FText::FromString(SearchString);
}

TSharedPtr<SWidget> SContentBrowser::GetItemContextMenu(TArrayView<const FContentBrowserItem> SelectedItems, EContentBrowserViewContext ViewContext)
{
	// We may only open the file or folder context menu (folder takes priority), so see whether we have any folders selected
	TArray<FContentBrowserItem> SelectedFolders;
	for (const FContentBrowserItem& SelectedItem : SelectedItems)
	{
		if (SelectedItem.IsFolder())
		{
			SelectedFolders.Add(SelectedItem);
		}
	}

	const bool bIsControlDown = FSlateApplication::Get().GetModifierKeys().IsControlDown();
	const bool bIsAssetViewContext = ViewContext == EContentBrowserViewContext::AssetView;
	const bool bShouldForceAddMenu = bIsControlDown && bIsAssetViewContext;

	// This can happen with both AssetView and PathView
	if (SelectedFolders.Num() > 0 && !bShouldForceAddMenu)
	{
		// Folders selected - show the folder menu

		// Clear any selection in the asset view, as it'll conflict with other view info
		// This is important for determining which context menu may be open based on the asset selection for rename/delete operations
		if (ViewContext != EContentBrowserViewContext::AssetView)
		{
			LegacyContentSourceWidgets->AssetViewPtr->ClearSelection();
		}

		// Ensure the path context menu has the up-to-date list of paths being worked on
		LegacyContentSourceWidgets->PathContextMenu->SetSelectedFolders(SelectedFolders);

		TArray<FString> SelectedPackagePaths;
		bool bPhysicalPathExists = false;
		for (const FContentBrowserItem& SelectedFolder : SelectedFolders)
		{
			FName PackagePath;
			if (SelectedFolder.Legacy_TryGetPackagePath(PackagePath))
			{
				SelectedPackagePaths.Add(PackagePath.ToString());
			}

			if (!bPhysicalPathExists)
			{
				FString PhysicalPath;
				if (SelectedFolder.GetItemPhysicalPath(PhysicalPath) && FPaths::DirectoryExists(PhysicalPath))
				{
					bPhysicalPathExists = true;
				}
			}
		}

		TSharedPtr<FExtender> Extender;
		if (SelectedPackagePaths.Num() > 0)
		{
			Extender = GetPathContextMenuExtender(SelectedPackagePaths);
		}

		UContentBrowserFolderContext* Context = NewObject<UContentBrowserFolderContext>();
		Context->ContentBrowser = SharedThis(this);
		// Note: This always uses the path view to manage the temporary folder item, even if the context menu came from the favorites view, as the favorites view can't make folders correctly
		Context->OnCreateNewFolder = ViewContext == EContentBrowserViewContext::AssetView
			? FOnCreateNewFolder::CreateSP(LegacyContentSourceWidgets->AssetViewPtr.Get(), &SAssetView::NewFolderItemRequested)
			: FOnCreateNewFolder::CreateSP(LegacyContentSourceWidgets->PathViewPtr.Get(), &SPathView::NewFolderItemRequested);
		
		ContentBrowserUtils::CountPathTypes(SelectedPackagePaths, Context->NumAssetPaths, Context->NumClassPaths);

		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
		Context->bCanBeModified = AssetToolsModule.Get().AllPassWritableFolderFilter(SelectedPackagePaths);

		if (SelectedPackagePaths.Num() == 0)
		{
			Context->bCanBeModified = false;
		}

		if (!bPhysicalPathExists)
		{
			Context->bNoFolderOnDisk = true;
		}

		FToolMenuContext MenuContext(Commands, Extender, Context);

		{
			UContentBrowserDataMenuContext_FolderMenu* DataContextObject = NewObject<UContentBrowserDataMenuContext_FolderMenu>();
			// Include the items that are not folders to help the batch operations operate on these also.
			DataContextObject->SelectedItems = SelectedItems;
			DataContextObject->bCanBeModified = Context->bCanBeModified;
			DataContextObject->ParentWidget = ViewContext == EContentBrowserViewContext::AssetView
				? TSharedPtr<SWidget>(LegacyContentSourceWidgets->AssetViewPtr)
				: ViewContext == EContentBrowserViewContext::FavoriteView
						? TSharedPtr<SWidget>(LegacyContentSourceWidgets->FavoritePathViewPtr)
						: TSharedPtr<SWidget>(LegacyContentSourceWidgets->PathViewPtr);
			
			MenuContext.AddObject(DataContextObject);
		}

		{
			TArray<FName> SelectedVirtualPaths;
			for (const FContentBrowserItem& SelectedFolder : SelectedFolders)
			{
				SelectedVirtualPaths.Add(SelectedFolder.GetVirtualPath());
			}
			AppendNewMenuContextObjects(EContentBrowserDataMenuContext_AddNewMenuDomain::PathView, SelectedVirtualPaths, MenuContext, nullptr, Context->bCanBeModified);
		}

		Context->SelectedPackagePaths = MoveTemp(SelectedPackagePaths);
		return UToolMenus::Get()->GenerateWidget("ContentBrowser.FolderContextMenu", MenuContext);
	}
	else if (ViewContext == EContentBrowserViewContext::AssetView)
	{
		if (SelectedItems.IsEmpty() || bShouldForceAddMenu)
		{
			// Nothing selected, or forcing AddMenu: show the new asset menu
			return MakeAddNewContextMenu(EContentBrowserDataMenuContext_AddNewMenuDomain::AssetView, nullptr);
		}
		else
		{
			return LegacyContentSourceWidgets->AssetContextMenu->MakeContextMenu(SelectedItems, LegacyContentSourceWidgets->AssetViewPtr->GetContentSources(), Commands);
        }
	}

	return nullptr;
}

void SContentBrowser::PopulateFolderContextMenu(UToolMenu* Menu)
{
	UContentBrowserFolderContext* Context = Menu->FindContext<UContentBrowserFolderContext>();
	check(Context);

	const TArray<FContentBrowserItem>& SelectedFolders = LegacyContentSourceWidgets->PathContextMenu->GetSelectedFolders();

	// We can only create folders when we have a single path selected
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	const bool bCanCreateNewFolder = SelectedFolders.Num() == 1 && ContentBrowserData->CanCreateFolder(SelectedFolders[0].GetVirtualPath(), nullptr);

	FText NewFolderToolTip;
	if(SelectedFolders.Num() == 1)
	{
		if(bCanCreateNewFolder)
		{
			NewFolderToolTip = FText::Format(LOCTEXT("NewFolderTooltip_CreateIn", "Create a new folder in {0}."), ContentBrowserData->ConvertVirtualPathToDisplay(SelectedFolders[0]));
		}
		else
		{
			NewFolderToolTip = FText::Format(LOCTEXT("NewFolderTooltip_InvalidPath", "Cannot create new folders in {0}."), ContentBrowserData->ConvertVirtualPathToDisplay(SelectedFolders[0]));
		}
	}
	else
	{
		NewFolderToolTip = LOCTEXT("NewFolderTooltip_InvalidNumberOfPaths", "Can only create folders when there is a single path selected.");
	}

	{
		FToolMenuSection& Section = Menu->AddSection("Section");

		if (Context->bCanBeModified)
		{
			// New Folder
			Section.AddMenuEntry(
				"NewFolder",
				LOCTEXT("NewFolder", "New Folder"),
				NewFolderToolTip,
				FSlateIcon(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetStyleSetName(), "ContentBrowser.NewFolderIcon"),
				FUIAction(
					FExecuteAction::CreateSP(this, &SContentBrowser::CreateNewFolder, SelectedFolders.Num() > 0 ? SelectedFolders[0].GetVirtualPath().ToString() : FString(), Context->OnCreateNewFolder),
					FCanExecuteAction::CreateLambda([bCanCreateNewFolder] { return bCanCreateNewFolder; })
				)
			);
		}

		Section.AddMenuEntry(
			"FolderContext",
			LOCTEXT("ShowInNewContentBrowser", "Show in New Content Browser"),
			LOCTEXT("ShowInNewContentBrowserTooltip", "Opens a new Content Browser at this folder location (at least 1 Content Browser window needs to be locked)"),
			FSlateIcon(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetStyleSetName(), "ContentBrowser.TabIcon"),
			FUIAction(FExecuteAction::CreateSP(this, &SContentBrowser::OpenNewContentBrowser))
		);
	}

	LegacyContentSourceWidgets->PathContextMenu->MakePathViewContextMenu(Menu);
}

void SContentBrowser::CreateNewFolder(FString FolderPath, FOnCreateNewFolder InOnCreateNewFolder)
{
	const FText DefaultFolderBaseName = LOCTEXT("DefaultFolderName", "NewFolder");
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

	// Create a valid base name for this folder
	FString DefaultFolderName = DefaultFolderBaseName.ToString();
	int32 NewFolderPostfix = 0;
	FName CombinedPathName;
	for (;;)
	{
		FString CombinedPathNameStr = FolderPath / DefaultFolderName;
		if (NewFolderPostfix > 0)
		{
			CombinedPathNameStr.AppendInt(NewFolderPostfix);
		}
		++NewFolderPostfix;

		CombinedPathName = *CombinedPathNameStr;

		const FContentBrowserItem ExistingFolder = ContentBrowserData->GetItemAtPath(CombinedPathName, EContentBrowserItemTypeFilter::IncludeFolders);
		if (!ExistingFolder.IsValid())
		{
			break;
		}
	}

	const FContentBrowserItemTemporaryContext NewFolderItem = ContentBrowserData->CreateFolder(CombinedPathName);
	if (NewFolderItem.IsValid())
	{
		InOnCreateNewFolder.ExecuteIfBound(NewFolderItem);
	}
}

void SContentBrowser::OpenNewContentBrowser()
{
	const TArray<FContentBrowserItem> SelectedFolders = LegacyContentSourceWidgets->PathContextMenu->GetSelectedFolders();
	FContentBrowserSingleton::Get().SyncBrowserToItems(SelectedFolders, false, true, NAME_None, true);
}
 
const FContentBrowserInstanceConfig* SContentBrowser::GetConstInstanceConfig() const
{
	return ContentBrowserUtils::GetConstInstanceConfig(InstanceName);
}

FContentBrowserInstanceConfig* SContentBrowser::GetMutableInstanceConfig()
{
	if (InstanceName.IsNone())
	{
		return nullptr;
	}

	UContentBrowserConfig* Config = UContentBrowserConfig::Get();
	if (Config == nullptr)
	{
		return nullptr;
	}

	FContentBrowserInstanceConfig* InstanceConfig = Config->Instances.Find(InstanceName);
	return InstanceConfig;
}

FContentBrowserInstanceConfig* SContentBrowser::CreateEditorConfigIfRequired()
{
	UContentBrowserConfig* Config = UContentBrowserConfig::Get();
	if (Config == nullptr)
	{
		return nullptr;
	}

	FContentBrowserInstanceConfig* InstanceConfig = Config->Instances.Find(InstanceName);
	if (InstanceConfig != nullptr)
	{
		return InstanceConfig;
	}

	InstanceConfig = &Config->Instances.Add(InstanceName, FContentBrowserInstanceConfig());

	const UContentBrowserSettings* Settings = GetDefault<UContentBrowserSettings>();
	InstanceConfig->bShowEngineContent = Settings->GetDisplayEngineFolder();
	InstanceConfig->bShowDeveloperContent = Settings->GetDisplayDevelopersFolder();
	InstanceConfig->bShowLocalizedContent = Settings->GetDisplayL10NFolder();
	InstanceConfig->bShowPluginContent = Settings->GetDisplayPluginFolders();
	InstanceConfig->bShowFolders = Settings->DisplayFolders;
	InstanceConfig->bShowEmptyFolders = Settings->DisplayEmptyFolders;
	InstanceConfig->bShowCppFolders = Settings->GetDisplayCppFolders();
	InstanceConfig->bFavoritesExpanded = Settings->GetDisplayFavorites();
	InstanceConfig->bSearchAssetPaths = Settings->GetIncludeAssetPaths();
	InstanceConfig->bSearchClasses = Settings->GetIncludeClassNames();
	InstanceConfig->bSearchCollections = Settings->GetIncludeCollectionNames();
	InstanceConfig->bFilterRecursively = Settings->FilterRecursively;

	UContentBrowserConfig::Get()->SaveEditorConfig();

	return InstanceConfig;
}

#undef LOCTEXT_NAMESPACE
