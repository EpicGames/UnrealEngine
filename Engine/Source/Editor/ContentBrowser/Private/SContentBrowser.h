// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/MRUArray.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDataMenuContexts.h"
#include "Experimental/ContentBrowserSearchOverride.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserDelegates.h"
#include "Filters/CustomTextFilters.h"
#include "HAL/Platform.h"
#include "HistoryManager.h"
#include "IAssetTypeActions.h"
#include "IContentBrowserSingleton.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/ParentWidgetSwitcher.h"
#include "Widgets/WidgetSizeSwitcher.h"

namespace UE::Editor::ContentBrowser
{
	class SContentSourcesView;
	class SLegacyContentSource;

	namespace Private
	{
		class SContentBrowserSourceTree;
		class FContentBrowserSourceTreeEntry;
		class SContentBrowserSourceTreeArea;
	}
	
}

class FContentBrowserItemDataTemporaryContext;
class FContentBrowserItemDataUpdate;
class FExtender;
class FAssetTextFilter;
class FSourcesSearch;
class FTabManager;
class FUICommandList;
class ICollectionContainer;
class SAssetSearchBox;
class SAssetView;
class SBorder;
class SComboButton;
class SExpandableArea;
class SFavoritePathView;
class SFilterList;
class SNavigationBar;
class SPathView;
class SSearchToggleButton;
class SWidget;
class SWidgetSwitcher;
class UClass;
class UContentBrowserToolbarMenuContext;
class UFactory;
class UToolMenu;
struct FAssetData;
struct FAssetSearchBoxSuggestion;
struct FCollectionNameType;
struct FContentBrowserInstanceConfig;
struct FContentBrowserItem;
struct FContentBrowserItemPath;
struct FGeometry;
struct FKeyEvent;
struct FNavigationBarComboOption;
struct FPointerEvent;
struct FSlateBrush;
struct FToolMenuContext;

enum class EFilterBarLayout : uint8;
enum class EItemDropZone;

enum class EContentBrowserViewContext : uint8
{
	AssetView,
	PathView,
	FavoriteView,
};

/**
 * A widget to display and work with all game and engine content
 */
class SContentBrowser
	: public SCompoundWidget
{
	using Super = SCompoundWidget;
public:

	SLATE_BEGIN_ARGS( SContentBrowser )
		: _ContainingTab()
		, _InitiallyLocked(false)
		, _IsDrawer(false)
	{}
		/** The tab in which the content browser resides */
		SLATE_ARGUMENT(TSharedPtr<SDockTab>, ContainingTab)

		/** If true, this content browser will not sync from external sources. */
		SLATE_ARGUMENT(bool, InitiallyLocked )

		SLATE_ARGUMENT(bool, IsDrawer)
	SLATE_END_ARGS()

	SContentBrowser();
	~SContentBrowser();

	bool ShouldShowRedirectors() const;

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs, const FName& InstanceName, const FContentBrowserConfig* Config );

	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;

	/** Sets up an inline-name for the creation of a new asset using the specified path and the specified class and/or factory */
	void CreateNewAsset(const FString& DefaultAssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory);

	/**
	 * Changes sources to show the specified assets and selects them in the asset view
	 *
	 *	@param AssetDataList		- A list of assets to sync the view to
	 * 
	 *	@param bAllowImplicitSync	- true to allow the view to sync to parent folders if they are already selected,
	 *								  false to force the view to select the explicit Parent folders of each asset 
	 */
	void SyncToAssets( TArrayView<const FAssetData> AssetDataList, const bool bAllowImplicitSync = false, const bool bDisableFiltersThatHideAssets = true );

	/**
	 * Changes sources to show the specified folders and selects them in the asset view
	 *
	 *	@param FolderList			- A list of folders to sync the view to
	 * 
	 *	@param bAllowImplicitSync	- true to allow the view to sync to parent folders if they are already selected,
	 *								  false to force the view to select the explicit Parent folders of each asset 
	 */
	void SyncToFolders( TArrayView<const FString> FolderList, const bool bAllowImplicitSync = false );

	/**
	 * Changes sources to show the specified collections and selects them in the collection view
	 *
	 *	@param CollectionsToSync	- A list of collections to sync the view to
	 */
	void SyncToCollections(TArrayView<const FCollectionRef> CollectionsToSync);

	/**
	 * Changes sources to show the specified items and selects them in the asset view
	 *
	 *	@param ItemsToSync			- A list of items to sync the view to
	 *
	 *	@param bAllowImplicitSync	- true to allow the view to sync to parent folders if they are already selected,
	 *								  false to force the view to select the explicit Parent folders of each asset
	 */
	void SyncToItems( TArrayView<const FContentBrowserItem> ItemsToSync, const bool bAllowImplicitSync = false, const bool bDisableFiltersThatHideAssets = true );

	/**
	 * Changes sources to show the specified assets and folders and selects them in the asset view
	 *
	 *	@param AssetDataList		- A list of assets to sync the view to
	 *	@param FolderList			- A list of folders to sync the view to
	 *
	 *	@param bAllowImplicitSync	- true to allow the view to sync to parent folders if they are already selected,
	 *								  false to force the view to select the explicit Parent folders of each asset
	 */
	void SyncToLegacy( TArrayView<const FAssetData> AssetDataList, TArrayView<const FString> FolderList, const bool bAllowImplicitSync = false, const bool bDisableFiltersThatHideAssets = true );

	/**
	 * Changes sources to show the specified items and selects them in the asset view
	 *
	 *	@param AssetDataList		- A list of assets to sync the view to
	 * 
	 *	@param bAllowImplicitSync	- true to allow the view to sync to parent folders if they are already selected,
	 *								  false to force the view to select the explicit Parent folders of each asset 
	 */
	void SyncTo( const FContentBrowserSelection& ItemSelection, const bool bAllowImplicitSync = false, const bool bDisableFiltersThatHideAssets = true );

	/** Sets this content browser as the primary browser. The primary browser is the target for asset syncs and contributes to the global selection set. */
	void SetIsPrimaryContentBrowser(bool NewIsPrimary);

	/** Returns if this browser can be used as the primary browser. */
	bool CanSetAsPrimaryContentBrowser() const;

	/** Gets the tab manager for the tab containing this browser */
	TSharedPtr<FTabManager> GetTabManager() const;

	/** Loads all selected assets if unloaded */
	void LoadSelectedObjectsIfNeeded();

	/** Returns true if the legacy (asset view) content source is the active content source */
	bool IsLegacyContentSourceActive() const;

	/** Returns all the assets that are selected in the asset view */
	void GetSelectedAssets(TArray<FAssetData>& SelectedAssets);

	/** Returns all the folders that are selected in the asset view */
	void GetSelectedFolders(TArray<FString>& SelectedFolders);

	/** Returns the folders that are selected in the path view */
	TArray<FString> GetSelectedPathViewFolders();

	/** Saves all persistent settings to config and returns a string identifier */
	void SaveSettings() const;

	/** Sets the content browser to show the specified paths */
	void SetSelectedPaths(const TArray<FString>& FolderPaths, bool bNeedsRefresh = false);

	/** Gets the current path if one exists, otherwise returns empty string. */
	FString GetCurrentPath(const EContentBrowserPathType PathType) const;

	/**
	 * Forces the content browser to show plugin content
	 *
	 * @param bEnginePlugin		If true, the content browser will also be forced to show engine content
	 */
	void ForceShowPluginContent(bool bEnginePlugin);

	/** Get the unique name of this content browser's in */
	const FName GetInstanceName() const;

	/** Returns true if this content browser does not accept syncing from an external source */
	bool IsLocked() const;

	/** Gives keyboard focus to the asset search box */
	void SetKeyboardFocusOnSearch() const;

	/**
	 * Copies settings from a different browser to this browser
	 * Note this overrides any settings already saved for this browser 
	 */
	void CopySettingsFromBrowser(TSharedPtr<SContentBrowser> OtherBrowser);

	/** SWidget interface  */
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnPreviewMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;

	/** Returns true if current path can be written to */
	bool CanWriteToCurrentPath() const;

	/** Returns true if path can be written to */
	bool CanWriteToPath(const FContentBrowserItemPath InPath) const;

	/** Adds a new Custom Text Filter to the Filter List */
	void AddCustomTextFilter(const FCustomTextFilterData& FilterData, bool bApplyFilter);

	/** Deletes the specified Custom Text Filter */
	void DeleteCustomTextFilterByLabel(const FText& FilterLabel);

	/** Modifies the specified Custom Text Filter */
	void ModifyCustomTextFilterByLabel(const FCustomTextFilterData& NewFilterData, const FText& FilterLabel);

	/** Gets whether the AssetView in the Content Browser is done filtering */
	bool IsAssetViewDoneFiltering();

	/** Sets the actual text in the Search Box */
	void SetSearchText(const FText& InSearchText);

	/** Called by the editable text control when the user commits a text change */
	void OnSearchBoxCommitted(const FText& InSearchText, ETextCommit::Type CommitInfo);

	/** Toggles the favorite status of an array of folders*/
	void ToggleFolderFavorite(const TArray<FString>& FolderPaths);
	
	/** Edit favorite folder options */
	void EditFolderFavorite(const FContentBrowserItemPath& InItemPath);

	/** Sets the Content Browser filter layout to either vertical or horizontal */
	void SetFilterLayout(EFilterBarLayout InFilterBarLayout) const;

	/** Gets the current Content Browser filter layout */
	EFilterBarLayout GetFilterLayout() const;

	/** Gets a pointer to the active filter container widget (Horizontal or Vertical), nullptr if neither is active */
	TSharedPtr<SWidget> GetActiveFilterContainer() const;

private:
	class FCollectionSource;

	/** Called prior to syncing the selection in this Content Browser */
	void PrepareToSyncItems(TArrayView<const FContentBrowserItem> ItemsToSync, const bool bDisableFiltersThatHideAssets);

	/** Called to retrieve the text that should be highlighted on assets */
	FText GetHighlightedText() const;

	/** Called when a containing tab is closing, if there is one */
	void OnContainingTabSavingVisualState() const;

	/** Called when a containing tab is closed, if there is one */
	void OnContainingTabClosed(TSharedRef<SDockTab> DockTab);

	/** Called when a containing tab is activated, if there is one */
	void OnContainingTabActivated(TSharedRef<SDockTab> DockTab, ETabActivationCause InActivationCause);

	/** Gets the key and ini file used to store the settings for each splitter slot */
	void GetSourceTreeSplitterSlotSizeSettingKeyAndFilename(int32 SlotIndex, FString& OutKey, FString& OutFilename) const;

	/** Loads settings from config based on the browser's InstanceName*/
	void LoadSettings(const FName& InstanceName);

	/** Handler for when the sources were changed */
	void SourcesChanged(const TArray<FString>& SelectedPaths, const TArray<FCollectionRef>& SelectedCollections);

	/** Handler for when a folder has been entered in the asset view */
	void FolderEntered(const FContentBrowserItem& Folder);

	/** Handler for when a path has been selected in the path view */
	void PathSelected(const FString& FolderPath);

	/** Handler for when a path has been selected in the favorite path view */
	void FavoritePathSelected(const FString& FolderPath);

	/** Get the asset tree context menu */
	TSharedRef<FExtender> GetPathContextMenuExtender(const TArray<FString>& SelectedPaths) const;

	/** Handler for when a collection has been selected in the collection view */
	void CollectionSelected(const TSharedPtr<ICollectionContainer>& CollectionContainer, const FCollectionNameType& SelectedCollection);

	/** Sets the state of the browser to the one described by the supplied history data */
	void OnApplyHistoryData(const FHistoryData& History);

	/** Updates the supplied history data with current information */
	void OnUpdateHistoryData(FHistoryData& History) const;

	/** Handler for when the path context menu requests a folder creation */
	void NewFolderRequested(const FString& SelectedPath);

	/** Handler for when a data source requests file item creation */
	void NewFileItemRequested(const FContentBrowserItemDataTemporaryContext& NewItemContext);

	/** Called when the editable text needs to be set or cleared */
	void SetSearchBoxText(const FText& InSearchText);

	/** Forwards the user-searching state to the asset view */
	void SetAssetViewUserSearching(bool bIsSearching);

	/** Called by the editable text control when the search text is changed by the user */
	void OnSearchBoxChanged(const FText& InSearchText);

	/** Called by the editable text control to allow the content browser to handle specific key presses without it resulting in typing into the search box */
	FReply OnSearchKeyDown(const FGeometry& Geometry, const FKeyEvent& InKeyEvent);

	/** Should the "Save Search" button be enabled? */
	bool IsSaveSearchButtonEnabled() const;

	/** Open the menu to let you save the current search text as a filter or dynamic collection */
	void OnSaveSearchButtonClicked(const FText& InSearchText);

	/** Save the current search as a filter pill */
	void SaveSearchAsFilter();
	
	/** Binding for begining to edit text in the location bar */
	void EditPathCommand();

	/** Called when a crumb in the path breadcrumb trail or menu is clicked */
	void OnPathClicked(const FString& VirtualPath);
	
	/** Called when item in the path delimiter arrow menu is clicked */
	void OnPathMenuItemClicked(FString ClickedPath);

	/** 
	 * Populates the delimiter arrow with a menu of directories under the current directory that can be navigated to
	 * 
	 * @param CrumbData	The current directory path
	 * @return The directory menu
	 */
	TSharedRef<SWidget> OnGetCrumbDelimiterContent(const FString& CrumbData) const;

	bool ParseCollectionCrumbData(const FString& CrumbData, FCollectionSource*& OutCollectionSource, FCollectionNameType& OutCollection) const;

	/** Return a list of recently visited locations which can be used with OnNavigateToPath to return to those locations */
	TArray<FNavigationBarComboOption> GetRecentPaths() const;
	
	/** Navigate to a location by text identifier. E.g. a virtual path such as /All/Game/MyFolder */
	void OnNavigateToPath(const FString& Path);

	/** Returns how the currently shown location should be presented to the user as text when beginning to edit the path */
	FText OnGetEditPathAsText(const FString& Text) const;
	
	/** Returns a list of valid paths starting with Prefix which can be navigated to with OnNavigateToPath */
	TArray<FNavigationBarComboOption> OnCompletePathPrefix(const FString& Prefix) const;

	/** Register the context objects needed for the "Add New" menu */
	void AppendNewMenuContextObjects(const EContentBrowserDataMenuContext_AddNewMenuDomain InDomain, const TArray<FName>& InSelectedPaths, FToolMenuContext& InOutMenuContext, UContentBrowserToolbarMenuContext* CommonContext, bool bCanBeModified);

	/** Handle creating a context menu for the "Add New" button */
	TSharedRef<SWidget> MakeAddNewContextMenu(const EContentBrowserDataMenuContext_AddNewMenuDomain InDomain, UContentBrowserToolbarMenuContext* CommonContext);

	/** Handle populating a context menu for the "Add New" button */
	void PopulateAddNewContextMenu(class UToolMenu* Menu);

	/** Called to work out whether the import button should be enabled */
	bool IsAddNewEnabled() const;

	/** Gets the tool tip for the "Add New" button */
	FText GetAddNewToolTipText() const;

	/** Saves dirty content. */
	FReply OnSaveClicked();

	/** Opens the add content dialog. */
	void OnAddContentRequested();

	/** Handler for when a new item is requested in the asset view */
	void OnNewItemRequested(const FContentBrowserItem& NewItem);

	/** Handler for when the selection set in any view has changed. */
	void OnItemSelectionChanged(const FContentBrowserItem& SelectedItem, ESelectInfo::Type SelectInfo, EContentBrowserViewContext ViewContext);

	/** Handler for when the user double clicks, presses enter, or presses space on a Content Browser item */
	void OnItemsActivated(TArrayView<const FContentBrowserItem> ActivatedItems, EAssetTypeActivationMethod::Type ActivationMethod);

	/** Handler for clicking the lock button */
	FReply ToggleLockClicked();

	/** Handler for clicking the dock drawer in layout button */
	FReply DockInLayoutClicked();

	/** Gets the menu text */
	FText GetLockMenuText() const;

	/** Gets icon for the lock button */
	FSlateIcon GetLockIcon() const;

	/** Gets brush for the lock button */
	const FSlateBrush* GetLockIconBrush() const;

	/** Gets the visibility state of the asset tree */
	EVisibility GetSourcesViewVisibility() const;
	
	/** Set whether or not we the sources view is expanded. */
	void SetSourcesViewExpanded(bool bExpanded);

	/** Handler for clicking the tree expand/collapse button */
	FReply SourcesViewExpandClicked();

	/** Called to handle the Content Browser settings changing */
	void OnContentBrowserSettingsChanged(FName PropertyName);
	
	/**
	 * Called when engine Cvars change to update content browser settings
	 * e.g. private content feature
	 */
	void OnConsoleVariableChanged();
	
	/** Handler for clicking the history back button */
	FReply BackClicked();

	/** Handler for clicking the history forward button */
	FReply ForwardClicked();

	/** Handler to check if filter by selection command is allowed */
	bool HandleFilterBySelectionCommandCanExecute() const;

	/** Handler for filter by selection */
	void HandleFilterBySelectionCommandExecute();
	
	/** Handler for favorite shortcuts */
	void HandleFavoriteShortcutCommandExecute(FName InCommandName);

	/** Handler to check to see if a rename command is allowed */
	bool HandleRenameCommandCanExecute() const;

	/** Handler for Rename */
	void HandleRenameCommand();

	/** Handler to check to see if a save asset command is allowed */
	bool HandleSaveAssetCommandCanExecute() const;

	/** Handler for save asset */
	void HandleSaveAssetCommand();

	/** Handler for SaveAll in folder */
	void HandleSaveAllCurrentFolderCommand() const;

	/** Handler for Resave on a folder */
	void HandleResaveAllCurrentFolderCommand() const;

	/** Handler for Copy path on an asset*/
	void CopySelectedAssetPathCommand() const;

	/** Handler to check to see if a delete command is allowed */
	bool HandleDeleteCommandCanExecute() const;

	/** Handler for Delete */
	void HandleDeleteCommandExecute();

	/** Handler for deleting a favorite using a keybind. */
	void HandleDeleteFavorite(TSharedPtr<SWidget> ParentWidget);

	/** Handler for opening assets or folders */
	void HandleOpenAssetsOrFoldersCommandExecute();

	/** Handler for previewing assets */
	void HandlePreviewAssetsCommandExecute();

	/** Handler for creating new folder */
	void HandleCreateNewFolderCommandExecute();

	/** Handler for navigating up to the parent folder of the one currently open */
	void HandleGoUpToParentFolder();

	/** True if the user may navigate from the currently selected folder to its parent */
	bool HandleCanGoUpToParentFolder() const;
	
	/** True if the user may use the history back button */
	bool IsBackEnabled() const;

	/** True if the user may use the history forward button */
	bool IsForwardEnabled() const;

	/** Gets the tool tip text for the history back button */
	FText GetHistoryBackTooltip() const;

	/** Gets the tool tip text for the history forward button */
	FText GetHistoryForwardTooltip() const;

	/** Sets the global selection set to the asset view's selected items */
	void SyncGlobalSelectionSet();

	/** Updates the breadcrumb trail to the current path */
	void UpdatePath();

	/** Handler for when a filter in the filter list has changed */
	void OnFilterChanged();

	/** Gets the text for the path label */
	FText GetPathText() const;

	/** Returns true if currently filtering by a source */
	bool IsFilteredBySource() const;

	/** Handler for when the context menu or asset view requests to find items in the paths view */
	void OnShowInPathsViewRequested(TArrayView<const FContentBrowserItem> ItemsToFind);

	/** Handler for when the user has committed a rename of an item */
	void OnItemRenameCommitted(TArrayView<const FContentBrowserItem> Items);

	/** Handler for when the asset context menu requests to rename an item */
	void OnRenameRequested(const FContentBrowserItem& Item, EContentBrowserViewContext ViewContext);

	/** Handler for when the path context menu has successfully deleted a folder */
	void OnOpenedFolderDeleted();

	/** Handler for when the asset context menu requests to duplicate an item */
	void OnDuplicateRequested(TArrayView<const FContentBrowserItem> OriginalItems);

	/** Handler for when the asset context menu requests to refresh the asset view */
	void OnAssetViewRefreshRequested();

	/** Handles an on collection container created event */
	void HandleCollectionContainerAdded(const TSharedRef<ICollectionContainer>& CollectionContainer);

	void ShowCollectionContainer(const TSharedRef<ICollectionContainer>& CollectionContainer);

	/** Handles an on collection container destroyed event */
	void HandleCollectionContainerRemoved(const TSharedRef<ICollectionContainer>& CollectionContainer);

	void HideCollectionContainer(const TSharedRef<ICollectionContainer>& CollectionContainer);

	/** Handles an on collection container is hidden changed event */
	void HandleIsHiddenChanged(ICollectionContainer& CollectionContainer, bool bIsHidden);

	/** Handles an on collection destroyed event */
	void HandleCollectionRemoved(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection);

	/** Handles an on collection renamed event */
	void HandleCollectionRenamed(ICollectionContainer& CollectionContainer, const FCollectionNameType& OriginalCollection, const FCollectionNameType& NewCollection);

	/** Handles an on collection updated event */
	void HandleCollectionUpdated(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection);

	/** Handles a path removed event */
	void HandlePathRemoved(const FName Path);

	/** Handles content items being updated */
	void HandleItemDataUpdated(TArrayView<const FContentBrowserItemDataUpdate> InUpdatedItems);

	/** Gets all suggestions for the asset search box */
	void OnAssetSearchSuggestionFilter(const FText& SearchText, TArray<FAssetSearchBoxSuggestion>& PossibleSuggestions, FText& SuggestionHighlightText) const;

	/** Combines the chosen suggestion with the active search text */
	FText OnAssetSearchSuggestionChosen(const FText& SearchText, const FString& Suggestion) const;

	/** Gets the dynamic hint text for the "Search Assets" search text box */
	FText GetSearchAssetsHintText() const;

	/** Delegate called when generating the context menu for an item */
	TSharedPtr<SWidget> GetItemContextMenu(TArrayView<const FContentBrowserItem> SelectedItems, EContentBrowserViewContext ViewContext);

	/** Populate the context menu for a folder */
	void PopulateFolderContextMenu(UToolMenu* Menu);

	/** Delegate called to get the current selection state */
	void GetSelectionState(TArray<FAssetData>& SelectedAssets, TArray<FString>& SelectedPaths);

	/** Sets up an inline-name for the creation of a default-named folder the specified path */
	void CreateNewFolder(FString FolderPath, FOnCreateNewFolder OnCreateNewFolder);

	/** Handler for when "Open in new Content Browser" is selected */
	void OpenNewContentBrowser();

	/** Bind our UI commands */
	void BindCommands();

	/** Unbind our UI commands */
	void UnbindCommands();

	/** Gets the visibility of the favorites view */
	EVisibility GetFavoriteFolderVisibility() const;

	/** Get the visibility of the lock button */
	EVisibility GetLockButtonVisibility() const;

	/** Updates favorite status of an array of folders */
	void UpdateFolderFavorite(const TArray<FString>& FolderPaths, const FString& InFolder, EItemDropZone InPosition);

	/** Save the favorite paths to settings and optionally select and show the paths passed in */
	void SaveAndShowNewFolderFavorites(const TArray<FString>& FolderPaths);

	/* Toggles the private show private content state of an array of folders*/
	void TogglePrivateContentEdit(const TArray<FString>& FolderPaths);

	/** Called when Asset View Options "Search" options change */
	void HandleAssetViewSearchOptionsChanged();

	/** Fill menu for filtering path view with items */
	void PopulatePathViewFiltersMenu(UToolMenu* Menu);

	/** Add data so that menus can access content browser */
	void ExtendAssetViewButtonMenuContext(FToolMenuContext& InMenuContext);

	/** Creates various widgets for the content browser main view */
	TSharedRef<SWidget> CreateToolBar(const FContentBrowserConfig* Config);

	TSharedRef<SWidget> CreateNavigationToolBar(const FContentBrowserConfig* Config);

	TSharedRef<SWidget> CreateLockButton(const FContentBrowserConfig* Config);
	TSharedRef<SWidget> CreateAssetView(const FContentBrowserConfig* Config);
	TSharedRef<SWidget> CreateFavoritesView(const FContentBrowserConfig* Config);
	TSharedRef<SWidget> CreatePathView(const FContentBrowserConfig* Config);
	TSharedRef<UE::Editor::ContentBrowser::Private::SContentBrowserSourceTreeArea> CreateCollectionsView(FCollectionSource& CollectionSource);
	TSharedRef<SWidget> CreateDrawerDockButton(const FContentBrowserConfig* Config);

	FCollectionSource& AddSlotForCollectionContainer(int32 Index, const TSharedRef<ICollectionContainer>& CollectionContainer);
	void RemoveSlotForCollectionContainer(const TSharedRef<ICollectionContainer>& CollectionContainer);

	void SetFavoritesExpanded(bool bExpanded);
	void SetPathViewExpanded(bool bExpanded);

	/** Adds menu options to the view menu */
	void ExtendViewOptionsMenu(const FContentBrowserConfig* Config);

	/** Gets the size rule for various areas. When areas are a collapsed the splitter slot becomes auto sized, otherwise it is user sized */
	SSplitter::ESizeRule GetFavoritesAreaSizeRule() const;
	SSplitter::ESizeRule GetPathAreaSizeRule() const;

	/** Called when the Splitter containing the PathView Box gets resized */
	void OnPathViewBoxColumnResized(float InSize);

	/** Returns the PathView Box width */
	FOptionalSize GetPathViewBoxWidthOverride() const;

	/** Called when the Splitter containing the filters Box gets resized */
	void OnFilterBoxColumnResized(float InSize);

	/** Returns the filters Box width */
	FOptionalSize GetFilterViewBoxWidthOverride() const;

	/** Gets the min size for various areas. When areas are not visible the min size is 0, otherwise there is a minimum size to prevent overlap */
	float GetFavoritesAreaMinSize() const;

	/** Called when the layout of the SFilterList is changing */
	void OnFilterBarLayoutChanging(EFilterBarLayout NewLayout);

	/** Fetch the const config for this content browser instance. */
	const FContentBrowserInstanceConfig* GetConstInstanceConfig() const;

	/** Fetch the mutable config for this content browser instance. */
	FContentBrowserInstanceConfig* GetMutableInstanceConfig();

	/** Initialize an editor config for this instance if one does not exist. */
	FContentBrowserInstanceConfig* CreateEditorConfigIfRequired();

	void UpdatePrivateContentFeatureEnabled(bool bUpdateFilterIfChanged);

	/** Executed when the legacy content source is enabled. */
	void OnLegacyContentSourceEnabled();
	
	/** Executed when the legacy content source is enabled. */
	void OnLegacyContentSourceDisabled();
	
	/** Create the content browser widgets (SPathView, SAssetView, search and filter bars etc) */
	TSharedRef<SWidget> CreateLegacyAssetViewWidgets();
	
	void OnSearchOverrideToggled();

private:

	/**
	 * Struct that contains the widgets used by content browser's default layout (i.e Legacy Content Source)
	 * ContentBrowser 2.0 will use TEDS, this is moved here but is still in use and correct while Content Browser refactor is on hold
	 */
	struct FLegacyContentSourceWidgets
	{
		/** The context menu manager for the path view */
		TSharedPtr<class FPathContextMenu> PathContextMenu;

		/** The asset view widget */
		TSharedPtr<SAssetView> AssetViewPtr;

		/** The border that holds the content in AssetView */
		TSharedPtr<SBorder> AssetViewBorder;
		
		/** A helper class to manage asset context menu options */
		TSharedPtr<class FAssetContextMenu> AssetContextMenu;

		/** The filter list */
		TSharedPtr<SFilterList> FilterListPtr;
		
		/** The text box used to search for assets */
		TSharedPtr<SAssetSearchBox> SearchBoxPtr;

		/** Per-view instance of the active search override, if any */
		TSharedPtr<UE::Editor::ContentBrowser::Extension::ICBSearchOverrideInstance> AssetSearchOverrideInstance;

		/** Handles the search box sizing within the toolbar */
		TSharedPtr<UE::Editor::ContentBrowser::Private::TWidgetDesiredSizeSwitcher<EAxis::X>> SearchBoxSizeSwitcher;

		/** Combine breadcrumb/text-box widget for showing & changing location */
		TSharedPtr<SNavigationBar> NavigationBar;

		/** The favorites tree widget */
		TSharedPtr<SFavoritePathView> FavoritePathViewPtr;

		/** The asset tree widget */
		TSharedPtr<SPathView> PathViewPtr;

		TSharedPtr<UE::Editor::ContentBrowser::Private::SContentBrowserSourceTree> SourceTreePtr;

		/** The generated navigation toolbar widget */
		TSharedPtr<SWidget> NavigationToolBarWidget;
		
		/** The splitter between the path & asset view */
		TSharedPtr<SSplitter> PathAssetSplitterPtr;

		/** The splitter between the path, favorite and collection views */
		TSharedPtr<SSplitter> PathFavoriteSplitterPtr;

		/** The number of fixed slots in the splitter, slots after this number are collection view slots */
		int32 SourceTreeSplitterNumFixedSlots = 0;

		/** The containing widget for the vertical filter view */
		TSharedPtr<SBox> VerticalFilterViewContainer;
	};

	TUniquePtr<FLegacyContentSourceWidgets> LegacyContentSourceWidgets;

	/** The tab that contains this browser */
	TWeakPtr<SDockTab> ContainingTab;

	/** The manager that keeps track of history data for this browser */
	FHistoryManager HistoryManager;

	/** A list of locations "jumped" to for populating a dropdown of such locations.
	 *  As a general rule, simple up/down navigation should not populate this list and only direct entries such as 
	 *  "find in content browser" or typing in a path should populate it 
	 */
	TMRUArray<FString> JumpMRU;

	/** The sources search for favorites */
	TSharedPtr<FSourcesSearch> FavoritesSearch;

	/** The sources search for paths */
	TSharedPtr<FSourcesSearch> SourcesSearch;

	/** The Combo Button used to summon the filter dropdown */
	TSharedPtr<SWidget> FilterComboButton;

	/** Favorites area widget */
	TSharedPtr<UE::Editor::ContentBrowser::Private::SContentBrowserSourceTreeArea> FavoritesArea;

	/** Path area widget */
	TSharedPtr<UE::Editor::ContentBrowser::Private::SContentBrowserSourceTreeArea> PathArea;

	/** Widgets for each collection container */
	TArray<TUniquePtr<FCollectionSource>> CollectionSources;

	/** The expanded state of the asset tree */
	bool bSourcesViewExpanded = true;

	/** True if this browser is the primary content browser */
	bool bIsPrimaryBrowser = false;

	/** True if this content browser can be set to the primary browser. */
	bool bCanSetAsPrimaryBrowser = true;

	/** True if this content browser is an a drawer */
	bool bIsDrawer = false;

	/** True if source should not be changed from an outside source */
	bool bIsLocked = false;

	/** True if bShouldEnablePrivateContentFilter was true and we are filtering this->AssetView. */
	bool bPrivateContentFilterEnabled = false;

	/** Starting width of the PathView Box */
	float PathViewBoxWidth = 230.f;

	/** Starting width of the Filter Box */
	float FilterBoxWidth = 180.f;

	/** Cached result of CanWriteToPath to avoid recalculating it every frame */
	mutable bool bCachedCanWriteToCurrentPath = false;
	
	/** Unique name for this Content Browser. */
	FName InstanceName;

	/** Path that was last used to determine bCachedCanWriteToCurrentPath */
	mutable TOptional<FName> CachedCanWriteToCurrentPath;

	/** The list of FrontendFilters currently applied to the asset view */
	TSharedPtr<FAssetFilterCollectionType> FrontendFilters;

	/** The text filter to use on the assets, separate from FrontendFilters so it can be specifically optimized */
	TSharedPtr<FAssetTextFilter> TextFilter;

	/** Commands handled by this widget */
	TSharedPtr< FUICommandList > Commands;

	/** Delegate used to create a new folder */
	FOnCreateNewFolder OnCreateNewFolder;

	/** The list of plugin filters currently applied to the path view */
	TSharedPtr<FPluginFilterCollectionType> PluginPathFilters;

	/** When viewing a dynamic collection, the active search query will be stashed in this variable so that it can be restored again later */
	TOptional<FText> StashedSearchBoxText;

	/** Handle to a callback for when CVars change */
	FConsoleVariableSinkHandle CVarSinkHandle;

	/** Widget that contains the content sources selector (if enabled) and the currently enabled content source */
	TSharedPtr<UE::Editor::ContentBrowser::SContentSourcesView> ContentSourcesContainer;
	
	/** A legacy content source, which when enabled displays the default content browser widgets */
	TSharedPtr<UE::Editor::ContentBrowser::SLegacyContentSource> LegacyContentSource;

	/** A copy of the init params used by the content browser so we can re-create the widgets dynamically
	 *  If no init params were provided, the default values will be used.
	 */
	FContentBrowserConfig InitConfig;

	/** Whether this content browser was specified an init config */
	bool bHasInitConfig = false;

	/** Registers the "Add New" drop-down menu */
	static FDelayedAutoRegisterHelper AddNewContextMenuRegistration;

	/** Registers the bottom navigation bar */
	static FDelayedAutoRegisterHelper NavigationBarMenuRegistration;

	/** Registers the context menu for folders */
	static FDelayedAutoRegisterHelper FolderContextMenuRegistration;

	/** Registers a sub-menu within the settings button menu */
	static FDelayedAutoRegisterHelper PathViewFiltersMenuRegistration;

	/** Registers the top toolbar */
	static FDelayedAutoRegisterHelper ToolBarMenuRegistration;

public:
	/** The section of EditorPerProjectUserSettings in which to save content browser settings */
	static const FString SettingsIniSection;
};
