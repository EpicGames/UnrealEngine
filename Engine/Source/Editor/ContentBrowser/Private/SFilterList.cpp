// Copyright Epic Games, Inc. All Rights Reserved.


#include "SFilterList.h"

#include "Algo/AnyOf.h"
#include "AssetRegistry/ARFilter.h"
#include "ContentBrowserCommands.h"
#include "ContentBrowserDataFilter.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserFrontEndFilterExtension.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserItemData.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "ContentBrowserStyle.h"
#include "ContentBrowserUtils.h"
#include "Filters.h"
#include "Filters/FilterBarConfig.h"
#include "Filters/SAssetFilterBar.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "FrontendFilterBase.h"
#include "FrontendFilters.h"
#include "HAL/PlatformCrt.h"
#include "IContentBrowserDataModule.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "Layout/WidgetPath.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "SContentBrowser.h"
#include "SlateGlobals.h"
#include "SlotBase.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Templates/TypeHash.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenuDelegates.h"
#include "ToolMenus.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealNames.h"
#include "Widgets/InvalidateWidgetReason.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SWrapBox.h"

class SWidget;
struct FGeometry;

#define LOCTEXT_NAMESPACE "ContentBrowser"

namespace UE::Editor::ContentBrowser
{
	namespace Private
	{
		static bool bUseFavoritesFilters = false;
		static FAutoConsoleVariableRef CVarUseFavoritesFilters(
			TEXT("UnifiedFavorites.ContentBrowser.Filter"),
			Private::bUseFavoritesFilters,
			TEXT("Use unified favorites filters in the Content Browser")
		);
	}
}

/////////////////////
// SFilterList
/////////////////////

const FName SFilterList::SharedIdentifier("FilterListSharedSettings");
SFilterList::FCustomTextFilterEvent SFilterList::CustomTextFilterEvent;

void SFilterList::Construct( const FArguments& InArgs )
{
	bUseSharedSettings = InArgs._UseSharedSettings;
	this->OnFilterChanged = InArgs._OnFilterChanged;
	this->ActiveFilters = InArgs._FrontendFilters;
	InitialClassFilters = InArgs._InitialClassFilters;

	TSharedPtr<FFrontendFilterCategory> DefaultCategory = MakeShareable( new FFrontendFilterCategory(LOCTEXT("FrontendFiltersCategory", "Other Filters"), LOCTEXT("FrontendFiltersCategoryTooltip", "Filter assets by all filters in this category.")) );
	
	TSharedRef<FFilter_HideOtherDevelopers> OtherDevelopersFilter = MakeShared<FFilter_HideOtherDevelopers>(DefaultCategory, InArgs._FilterBarIdentifier);
	// This filter affecst the backend query so we must perform a full refresh when it changes
	OtherDevelopersFilter->OnChanged().Add(this->OnFilterChanged);

	// Add all built-in frontend filters here
	AllFrontendFilters_Internal.Add( MakeShareable(new FFrontendFilter_CheckedOut(DefaultCategory)) );
	AllFrontendFilters_Internal.Add( MakeShareable(new FFrontendFilter_Modified(DefaultCategory)) );
	AllFrontendFilters_Internal.Add( MakeShareable(new FFrontendFilter_Writable(DefaultCategory)) );
	AllFrontendFilters_Internal.Add(OtherDevelopersFilter);
	AllFrontendFilters_Internal.Add( MakeShareable(new FFrontendFilter_ReplicatedBlueprint(DefaultCategory)) );
	AllFrontendFilters_Internal.Add(MakeShared<FFilter_ShowRedirectors>(DefaultCategory));
	AllFrontendFilters_Internal.Add( MakeShareable(new FFrontendFilter_InUseByLoadedLevels(DefaultCategory)) );
	AllFrontendFilters_Internal.Add( MakeShareable(new FFrontendFilter_UsedInAnyLevel(DefaultCategory)) );
	AllFrontendFilters_Internal.Add( MakeShareable(new FFrontendFilter_NotUsedInAnyLevel(DefaultCategory)) );
	AllFrontendFilters_Internal.Add( MakeShareable(new FFrontendFilter_NotUsedInAnyAsset(DefaultCategory)) );
	AllFrontendFilters_Internal.Add( MakeShareable(new FFrontendFilter_ArbitraryComparisonOperation(DefaultCategory)) );
	AllFrontendFilters_Internal.Add(MakeShareable(new FFrontendFilter_Recent(DefaultCategory)));
	AllFrontendFilters_Internal.Add( MakeShareable(new FFrontendFilter_NotSourceControlled(DefaultCategory)) );
	AllFrontendFilters_Internal.Add(MakeShareable(new FFrontendFilter_VirtualizedData(DefaultCategory)));
	AllFrontendFilters_Internal.Add(MakeShared<FFrontendFilter_Unsupported>(DefaultCategory));
	AllFrontendFilters_Internal.Add(MakeShareable(new FFrontendFilter_Favorites(DefaultCategory)));

	// Add any global user-defined frontend filters
	for (TObjectIterator<UContentBrowserFrontEndFilterExtension> ExtensionIt(RF_NoFlags); ExtensionIt; ++ExtensionIt)
	{
		if (UContentBrowserFrontEndFilterExtension* PotentialExtension = *ExtensionIt)
		{
			if (PotentialExtension->HasAnyFlags(RF_ClassDefaultObject) && !PotentialExtension->GetClass()->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract))
			{
				// Grab the filters
				TArray< TSharedRef<FFrontendFilter> > ExtendedFrontendFilters;
				PotentialExtension->AddFrontEndFilterExtensions(DefaultCategory, ExtendedFrontendFilters);
				AllFrontendFilters_Internal.Append(ExtendedFrontendFilters);

				// Grab the categories
				for (const TSharedRef<FFrontendFilter>& FilterRef : ExtendedFrontendFilters)
				{
					TSharedPtr<FFilterCategory> Category = FilterRef->GetCategory();
					if (Category.IsValid())
					{
						this->AllFilterCategories.AddUnique(Category);
					}
				}
			}
		}
	}

	// Add in filters specific to this invocation
	for (const TSharedRef<FFrontendFilter>& Filter : InArgs._ExtraFrontendFilters)
	{
		if (TSharedPtr<FFilterCategory> Category = Filter->GetCategory())
		{
			this->AllFilterCategories.AddUnique(Category);
		}

		AllFrontendFilters_Internal.Add(Filter);
	}

	this->AllFilterCategories.AddUnique(DefaultCategory);

	// Add the local copy of all filters to SFilterBar's copy of all filters
	for(TSharedRef<FFrontendFilter> FrontendFilter : AllFrontendFilters_Internal)
	{
		this->AddFilter(FrontendFilter);
	}
	
	SAssetFilterBar<FAssetFilterType>::FArguments Args;

	/** Explicitly setting this to true as it should ALWAYS be true for SFilterList */
	Args._UseDefaultAssetFilters = true;
	Args._OnFilterChanged = this->OnFilterChanged;
	Args._CreateTextFilter = InArgs._CreateTextFilter;
	Args._FilterBarIdentifier = InArgs._FilterBarIdentifier;
	Args._OnFilterBarLayoutChanging = InArgs._OnFilterBarLayoutChanging;
	Args._FilterBarLayout = InArgs._FilterBarLayout;
	Args._CanChangeOrientation = InArgs._CanChangeOrientation;
	Args._OnExtendAddFilterMenu = InArgs._OnExtendAddFilterMenu;
	Args._FilterMenuName = FName("ContentBrowser.FilterMenu");
	Args._DefaultMenuExpansionCategory = InArgs._DefaultMenuExpansionCategory;
	Args._bUseSectionsForCustomCategories = InArgs._bUseSectionsForCustomCategories;
	Args._CommandList = InArgs._CommandList;

	SAssetFilterBar<FAssetFilterType>::Construct(Args);

	/* If we are using shared settings, add a default config for the shared settings in case it doesnt exist
	 * This needs to go after SAssetFilterBar<FAssetFilterType>::Construct() to ensure UFilterBarConfig is valid
	 */
	if(bUseSharedSettings)
	{
		UFilterBarConfig::Get()->FilterBars.FindOrAdd(SharedIdentifier);

		// Bind our delegate for when another SFilterList creates a custom text filter, so we can sync our list
		CustomTextFilterEvent.AddSP(this, &SFilterList::OnExternalCustomTextFilterCreated);
	}
	
}

const TArray<UClass*>& SFilterList::GetInitialClassFilters()
{
	return InitialClassFilters;
}

TSharedPtr<FFrontendFilter> SFilterList::GetFrontendFilter(const FString& InName) const
{
	for (const TSharedRef<FFrontendFilter>& Filter : AllFrontendFilters_Internal)
	{
		if (Filter->GetName() == InName)
		{
			return Filter;
		}
	}
	return TSharedPtr<FFrontendFilter>();
}

void SFilterList::ActivateClassFilterByPath(const FTopLevelAssetPath& ClassPath)
{
	if (GetAssetTypeFilterCheckState(ClassPath) == ECheckBoxState::Checked)
	{
		// Checked means there's already a filter for it (checked in filter list), enable
		ToggleAssetTypeFilterEnabled(ClassPath, true);
	}
	else
	{
		// Create and enable the filter if needed
		SetAssetTypeFilterCheckState(ClassPath, ECheckBoxState::Checked, true);
	}
}

TSharedRef<SWidget> SFilterList::ExternalMakeAddFilterMenu()
{
	return SAssetFilterBar<FAssetFilterType>::MakeAddFilterMenu();
}

FARFilter SFilterList::GetCombinedBackendFilter(TArray<TSharedRef<const FPathPermissionList>>& OutPermissionLists) const
{
	TSharedPtr<FFilter_HideOtherDevelopers> OtherDevelopersFilter = StaticCastSharedPtr<FFilter_HideOtherDevelopers>(GetFrontendFilter("HideOtherDevelopersBackend"));
	if (OtherDevelopersFilter.IsValid() && OtherDevelopersFilter->IsActive())
	{
		OutPermissionLists.Add(OtherDevelopersFilter->GetPathPermissionList());
	}

	FARFilter CombinedFilter = Super::GetCombinedBackendFilter();

	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		CombinedFilter.bRecursivePaths = this->bFilterPathsRecursively;
	}

	return CombinedFilter;
}

void SFilterList::DisableFiltersThatHideItems(TArrayView<const FContentBrowserItem> ItemList)
{
	if (HasAnyFilters() && ItemList.Num() > 0)
	{
		// Determine if we should disable backend filters. If any item fails the combined backend filter, disable them all.
		bool bDisableAllBackendFilters = false;
		{
			FContentBrowserDataCompiledFilter CompiledDataFilter;
			{
				static const FName RootPath = "/";

				UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

				FContentBrowserDataFilter DataFilter;
				DataFilter.bRecursivePaths = true;
				TArray<TSharedRef<const FPathPermissionList>> UnusedPermissionLists;
				ContentBrowserUtils::AppendAssetFilterToContentBrowserFilter(GetCombinedBackendFilter(UnusedPermissionLists), nullptr, nullptr, DataFilter);

				ContentBrowserData->CompileFilter(RootPath, DataFilter, CompiledDataFilter);
			}

			for (const FContentBrowserItem& Item : ItemList)
			{
				if (!Item.IsFile())
				{
					continue;
				}

				FContentBrowserItem::FItemDataArrayView InternalItems = Item.GetInternalItems();
				for (const FContentBrowserItemData& InternalItemRef : InternalItems)
				{
					UContentBrowserDataSource* ItemDataSource = InternalItemRef.GetOwnerDataSource();

					FContentBrowserItemData InternalItem = InternalItemRef;
					ItemDataSource->ConvertItemForFilter(InternalItem, CompiledDataFilter);

					if (!ItemDataSource->DoesItemPassFilter(InternalItem, CompiledDataFilter))
					{
						bDisableAllBackendFilters = true;
						break;
					}
				}

				if (bDisableAllBackendFilters)
				{
					break;
				}
			}
		}

		// Iterate over all enabled filters and disable any frontend filters that would hide any of the supplied assets
		bool ExecuteOnFilterChanged = false;
		for (const TSharedPtr<SFilter> Filter : Filters)
		{
			if (Filter->IsEnabled())
			{
				if (const TSharedPtr<FFilterBase<FAssetFilterType>>& FrontendFilter = Filter->GetFrontendFilter())
				{
					for (const FContentBrowserItem& Item : ItemList)
					{
						if (!FrontendFilter->IsInverseFilter() && !FrontendFilter->PassesFilter(Item))
						{
							// This is a frontend filter and at least one asset did not pass.
							Filter->SetEnabled(false, false);
							SetFrontendFilterActive(FrontendFilter.ToSharedRef(), false);
							ExecuteOnFilterChanged = true;
						}
					}
				}
			}
		}

		TSharedPtr<FFilter_HideOtherDevelopers> OtherDevelopersFilter = StaticCastSharedPtr<FFilter_HideOtherDevelopers>(GetFrontendFilter("HideOtherDevelopersBackend"));
		// Special case: if item is hidden because of "hide other developers" filter, disable it
		if (OtherDevelopersFilter.IsValid() && OtherDevelopersFilter->IsActive())
		{
			TSharedRef<const FPathPermissionList> PermissionList = OtherDevelopersFilter->GetPathPermissionList();	
			for (const FContentBrowserItem& Item : ItemList)
			{
				if (PermissionList->PassesStartsWithFilter(WriteToString<256>(Item.GetInternalPath())))
				{
					TSharedRef<FFilter_HideOtherDevelopers>	OtherDevelopersFilterAsRef = OtherDevelopersFilter.ToSharedRef();
					int32 ExistingIndex = Filters.IndexOfByPredicate([OtherDevelopersFilterAsRef](TSharedPtr<SFilter> Filter) { return Filter->GetFrontendFilter() == OtherDevelopersFilterAsRef; });
					TSharedRef<SFilter> FilterWidget = ExistingIndex == INDEX_NONE ? AddFilterToBar(OtherDevelopersFilterAsRef) : Filters[ExistingIndex];
					FilterWidget->SetEnabled(false, false);
					SetFrontendFilterActive(OtherDevelopersFilterAsRef, false);
					ExecuteOnFilterChanged = true;
					break;
				}
			}
		}

			auto AddAndActivateInverseFilter = [this, &ExecuteOnFilterChanged](const TSharedRef<FFilterBase<FAssetFilterType>>& InFilter) 
		{
			int32 ExistingIndex = Filters.IndexOfByPredicate([InFilter](TSharedPtr<SFilter> Filter) { return Filter->GetFrontendFilter() == InFilter; });
			TSharedRef<SFilter> FilterWidget = ExistingIndex == INDEX_NONE ? AddFilterToBar(InFilter) : Filters[ExistingIndex];
			FilterWidget->SetEnabled(true, false);
			SetFrontendFilterActive(InFilter, true);
			ExecuteOnFilterChanged = true;
		};

		// Special case: if the object is a redirector then enable the 'show redirectors' filter - this will also prevent
		// folders that contain only redirectors from being hidden with the "hide empty folders" setting
		FString RedirectorClassPath = UObjectRedirector::StaticClass()->GetPathName();
		const bool bAnyRedirectors = Algo::AnyOf(ItemList, [RedirectorClassPath](const FContentBrowserItem& Item) {
			FContentBrowserItemDataAttributeValue Attribute = Item.GetItemAttribute(ContentBrowserItemAttributes::ItemTypeName, false);
			return Attribute.IsValid() && Attribute.GetValueString() == RedirectorClassPath;
		});
		if (bAnyRedirectors)
		{
			TSharedPtr<FFilter_ShowRedirectors> RedirectorFilter = StaticCastSharedPtr<FFilter_ShowRedirectors>(GetFrontendFilter("ShowRedirectorsBackend"));
			if (RedirectorFilter.IsValid())
			{
				AddAndActivateInverseFilter(RedirectorFilter.ToSharedRef());
			}
		}

		// Disable all backend filters if it was determined that the combined backend filter hides any of the assets
		if (bDisableAllBackendFilters)
		{
			for(const TSharedPtr<SAssetFilter> AssetFilter : AssetFilters)
			{
				if(AssetFilter.IsValid())
				{
					FARFilter BackendFilter = AssetFilter->GetBackendFilter();
					if (!BackendFilter.IsEmpty())
					{
						AssetFilter->SetEnabled(false, false);
						ExecuteOnFilterChanged = true;
					}
				}
			}
		}

		if (ExecuteOnFilterChanged)
		{
			OnFilterChanged.ExecuteIfBound();
		}
	}
}

void SFilterList::SetFrontendFilterCheckState(const TSharedPtr<FFrontendFilter>& InFrontendFilter, ECheckBoxState CheckState)
{
	this->SetFilterCheckState(InFrontendFilter, CheckState);
}

ECheckBoxState SFilterList::GetFrontendFilterCheckState(const TSharedPtr<FFrontendFilter>& InFrontendFilter) const
{
	return this->GetFilterCheckState(InFrontendFilter);
}

bool SFilterList::IsFrontendFilterActive(const TSharedPtr<FFrontendFilter>& InFrontendFilter) const
{
	return this->IsFilterActive(InFrontendFilter);
}

bool IsFilteredByPicker(const TArray<UClass*>& FilterClassList, UClass* TestClass)
{
	if (FilterClassList.Num() == 0)
	{
		return false;
	}
	for (const UClass* Class : FilterClassList)
	{
		if (TestClass->IsChildOf(Class))
		{
			return false;
		}
	}
	return true;
}

UAssetFilterBarContext* SFilterList::CreateAssetFilterBarContext()
{
	UAssetFilterBarContext* AssetFilterBarContext = SAssetFilterBar<const FContentBrowserItem&>::CreateAssetFilterBarContext();

	// Override PopulateFilterMenu - ContentBrowser has a different menu layout to SAssetFilterBar
	AssetFilterBarContext->PopulateFilterMenu = FOnPopulateAddAssetFilterMenu::CreateSP(this, &SFilterList::PopulateAddFilterMenu);

	AssetFilterBarContext->OnFilterAssetType = FOnFilterAssetType::CreateLambda([this](UClass *TestClass)
	{
		return !IsFilteredByPicker(this->InitialClassFilters, TestClass);
	});

	return AssetFilterBarContext;
}

void SFilterList::PopulateAddFilterMenu(UToolMenu* Menu, TSharedPtr<FFilterCategory> MenuExpansion, FOnFilterAssetType OnFilterAssetType)
{
	if (!UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		Super::PopulateAddFilterMenu(Menu, MenuExpansion, OnFilterAssetType);
		return;
	}

	using namespace UE::Editor::Widgets;

	TMap<TSharedPtr<FFilterCategory>, FFilterCategoryMenu> CategoryToMenuMap = BuildCategoryToMenuMap(AssetFilterCategories, CustomClassFilters, OnFilterAssetType);

	// Populate the common filter sections (Reset Filters etc)
	{
		this->PopulateCommonFilterSections(Menu);

		// Remove legacy section
		Menu->RemoveSection("FilterBarTextFilters");

		FToolMenuSection& AdvancedSection = Menu->FindOrAddSection(
			"FilterBarAdvanced",
			LOCTEXT("FilterBarAdvancedSection", "Advanced"),
			FToolMenuInsert("BasicFilterBarFiltersMenu", EToolMenuInsertType::Last));

		// Only add the custom text filter submenu if we have a valid CreateTextFilter delegate to use
		if (this->CreateTextFilter.IsBound())
		{
			AdvancedSection.AddSubMenu(
				"CustomFiltersSubMenu",
				LOCTEXT("FilterBarTextFilters", "Custom Filters"),
				LOCTEXT("FilterBarTextFiltersTooltip", "Custom Filters"),
				FNewToolMenuDelegate::CreateSP(this, &SFilterList::CreateTextFiltersMenu),
				false,
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Filter"));
		}

		if (FToolMenuSection* ManageSection = Menu->FindSection("FilterBarResetFilters"))
		{
			ManageSection->Label = LOCTEXT("FilterBarManageSection", "Manage");

			ManageSection->AddMenuEntry(
				"CopyFilters",
				LOCTEXT("FilterListCopyFilters", "Copy Filters"),
				LOCTEXT("FilterListCopyFiltersTooltip", "Copy the current filter selection"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Copy"),
				FUIAction(
					FExecuteAction::CreateSP(this, &SFilterList::OnCopyFilters))
			);

			ManageSection->AddMenuEntry(
				"PasteFilters",
				LOCTEXT("FilterListPasteFilters", "Paste Filters"),
				LOCTEXT("FilterListPasteFiltersTooltip", "Paste to the current filter selection"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "GenericCommands.Paste"),
				FUIAction(
					FExecuteAction::CreateSP(this, &SFilterList::OnPasteFilters),
					FCanExecuteAction::CreateSP(this, &SFilterList::CanPasteFilters))
			);

			ManageSection->AddMenuEntry(
				"FilterRecursively",
				LOCTEXT("FilterListFilterRecursively", "Filter Recursively"),
				LOCTEXT("FilterListFilterRecursivelyTooltip", "Apply the current filter selection recursively, relevant to the current path"),
				{ },
				FUIAction(
					FExecuteAction::CreateSP(this, &SFilterList::ToggleFilterPathsRecursively),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SFilterList::IsFilteringPathsRecursively)),
				EUserInterfaceActionType::ToggleButton
			);

			// Add filter by selection if command is mapped in the command list
			if (const TSharedPtr<FUICommandList> CommandList = CommandListWeak.Pin())
			{
				const FContentBrowserCommands& ContentBrowserCommands = FContentBrowserCommands::Get();
				if (CommandList->IsActionMapped(ContentBrowserCommands.FilterBySelection))
				{
					ManageSection->AddMenuEntry(ContentBrowserCommands.FilterBySelection);
				}
			}

			ManageSection->AddMenuEntry(
				"FilterAlphabetically",
				LOCTEXT("FilterAlphabetically", "Sort Filters Alphabetically"),
				LOCTEXT("FilterAlphabeticallyToolTip", "Active filters will be sorted and displayed in alphabetical order in the layout instead of their insertion order"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SFilterList::ToggleSortFiltersAlphabetically),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SFilterList::GetSortFiltersAlphabetically)),
				EUserInterfaceActionType::ToggleButton
			);
		}

		FToolMenuSection& DisplaySection = Menu->AddSection(
			"FilterBarDisplay",
			LOCTEXT("FilterListDisplaySection", "Filter Display"));
		{
			DisplaySection.AddMenuEntry(
				"HorizontalLayout",
				LOCTEXT("FilterListHorizontalLayout", "Horizontal"),
				LOCTEXT("FilterListHorizontalLayoutToolTip", "Swap to a Horizontal layout for the filter bar"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSPLambda(
					this, [this]
					{
						if (this->GetFilterLayout() != EFilterBarLayout::Horizontal)
						{
							this->SetFilterLayout(EFilterBarLayout::Horizontal);
						}
					}),
					FCanExecuteAction(),
					FIsActionChecked::CreateSPLambda(this, [this]
					{
						return this->GetFilterLayout() == EFilterBarLayout::Horizontal;
					})),
				EUserInterfaceActionType::RadioButton
			);

			DisplaySection.AddMenuEntry(
				"VerticalLayout",
				LOCTEXT("FilterListVerticalLayout", "Vertical"),
				LOCTEXT("FilterListVerticalLayoutToolTip", "Swap to a vertical layout for the filter bar"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSPLambda(this, [this]
					{
						if (this->GetFilterLayout() != EFilterBarLayout::Vertical)
						{
							this->SetFilterLayout(EFilterBarLayout::Vertical);
						}
					}),
					FCanExecuteAction(),
					FIsActionChecked::CreateSPLambda(this, [this]
					{
						return this->GetFilterLayout() == EFilterBarLayout::Vertical;
					})),
				EUserInterfaceActionType::RadioButton
			);

		}

		if (UE::Editor::ContentBrowser::Private::bUseFavoritesFilters)
		{
			// Add Favorites section before the BasicFilterBarFiltersMenu filters
			FToolMenuSection& FavoritesSection = Menu->FindOrAddSection(
				"FilterBarFavorites",
				LOCTEXT("FilterBarFavoritesSection", "Favorites Filter"),
				FToolMenuInsert("FilterBarDisplay", EToolMenuInsertType::After));

			// Find the Favorites filter and add it directly to the favorites section
			for (const TSharedRef<FFilterBase<FAssetFilterType>>& Filter : AllFrontendFilters)
			{
				if (Filter->GetName() == "Favorites")
				{
					FavoritesSection.AddMenuEntry(
						"FavoritesFilter",
						Filter->GetDisplayName(),
						Filter->GetToolTipText(),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Star"),
						FUIAction(
							FExecuteAction::CreateSP(this, &SFilterList::FrontendFilterClicked, Filter),
							FCanExecuteAction(),
							FIsActionChecked::CreateSP(this, &SFilterList::IsFrontendFilterInUse, Filter)),
						EUserInterfaceActionType::ToggleButton
					);
					break;
				}
			}
		}
	}

	// Sort by category name so that we add the submenus in alphabetical order
	CategoryToMenuMap.KeySort(
		[](const TSharedPtr<FFilterCategory>& A, const TSharedPtr<FFilterCategory>& B)
		{
			return A.IsValid()
				&& B.IsValid()
				&& A->Title.CompareTo(B->Title) < 0;
		}
	);

	// Sort each submenu internals 
	for (TPair<TSharedPtr<FFilterCategory>, FFilterCategoryMenu>& CategoryToMenu : CategoryToMenuMap)
	{
		CategoryToMenu.Value.Classes.Sort(
			[](const TSharedPtr<FCustomClassFilterData>& A, const TSharedPtr<FCustomClassFilterData>& B)
			{
				return A.IsValid()
					&& B.IsValid()
					&& A->GetName().CompareTo(B->GetName()) < 0;
			}
		);
	}

	// Sort frontend filters
	AllFrontendFilters.Sort(
		[](const TSharedRef<FFilterBase<FAssetFilterType>>& A, const TSharedRef<FFilterBase<FAssetFilterType>>& B)
		{
			return A->GetDisplayName().CompareTo(B->GetDisplayName()) < 0;
		}
	);

	// Sort user filters
	CustomTextFilters.Sort(
		[](const TSharedRef<ICustomTextFilter<FAssetFilterType>>& A, const TSharedRef<ICustomTextFilter<FAssetFilterType>>& B)
		{
			return A->GetFilter().IsValid()
				&& B->GetFilter().IsValid()
				&& A->GetFilter()->GetDisplayName().CompareTo(B->GetFilter()->GetDisplayName()) < 0;
		}
	);

	// If we want to expand a category
	if (MenuExpansion)
	{
		// First add the expanded category, this appears as standard entries in the list (Note: intentionally not using FindChecked here as removing it from the map later would cause the ref to be garbage)
		if (FFilterCategoryMenu* ExpandedCategory = CategoryToMenuMap.Find( MenuExpansion ))
		{
			FToolMenuSection& Section = Menu->AddSection(ExpandedCategory->SectionExtensionHook, ExpandedCategory->SectionHeading);

			// If we are doing a full menu (i.e expanding basic) we add a menu entry which toggles all other categories
            Section.AddMenuEntry(
                FName(FText::AsCultureInvariant(ExpandedCategory->SectionHeading).ToString()),
                ExpandedCategory->SectionHeading,
                MenuExpansion->Tooltip,
                FSlateIcon(FAppStyle::Get().GetStyleSetName(), "PlacementBrowser.Icons.Basic"),
                FUIAction(
                FExecuteAction::CreateSP( this, &SFilterList::FilterByTypeCategory, MenuExpansion, ExpandedCategory->Classes ),
                FCanExecuteAction(),
                FGetActionCheckState::CreateSP(this, &SFilterList::IsTypeCategoryChecked, MenuExpansion, ExpandedCategory->Classes ) ),
                EUserInterfaceActionType::ToggleButton
                );

			Section.AddSeparator("ExpandedCategorySeparator");

			// Now populate with all the assets from the expanded category
			SFilterList::CreateFiltersMenuCategory( Section, ExpandedCategory->Classes);

			// Remove the Expanded from the map now, as this is treated differently and is no longer needed.
			CategoryToMenuMap.Remove(MenuExpansion);
		}
	}

	FToolMenuSection& AdvancedSection = Menu->FindOrAddSection("FilterBarAdvanced");

	FToolMenuEntry& AllFiltersSubMenu = AdvancedSection.AddSubMenu(
		"AllFilters",
		LOCTEXT("AllAssetsMenuHeading", "All Filters"),
		{ },
		FNewToolMenuDelegate::CreateLambda([this, CategoryToMenuMap](UToolMenu* InMenu){
			// Add all the other categories as submenus, un-labelled, acts only as a root for the submenus
			FToolMenuSection& TypeFiltersSection = InMenu->AddSection("AssetFilterBarFilterAdvancedAsset", { });

			for (const TPair<TSharedPtr<FFilterCategory>, FFilterCategoryMenu>& CategoryMenuPair : CategoryToMenuMap)
			{
				TypeFiltersSection.AddSubMenu(
					FName(FText::AsCultureInvariant(CategoryMenuPair.Key->Title).ToString()),
					CategoryMenuPair.Key->Title,
					CategoryMenuPair.Key->Tooltip,
					FNewToolMenuDelegate::CreateSP(this, &SFilterList::CreateFiltersMenuCategory, CategoryMenuPair.Value.Classes),
					FUIAction(
					FExecuteAction::CreateSP(this, &SFilterList::FilterByTypeCategory, CategoryMenuPair.Key, CategoryMenuPair.Value.Classes),
					FCanExecuteAction(),
					FGetActionCheckState::CreateSP(this, &SFilterList::IsTypeCategoryChecked, CategoryMenuPair.Key, CategoryMenuPair.Value.Classes)),
					EUserInterfaceActionType::ToggleButton
				);
			}

			// Now add all non-asset filters
			this->PopulateCustomFilters(InMenu);
		}));

	AllFiltersSubMenu.InsertPosition.Position = EToolMenuInsertType::Last;
}

void SFilterList::CreateCustomFilterDialog(const FText& InText)
{
	CreateCustomTextFilterFromSearch(InText);
}

void SFilterList::OnCreateCustomTextFilter(const FCustomTextFilterData& InFilterData, bool bApplyFilter)
{
	SAssetFilterBar<FAssetFilterType>::OnCreateCustomTextFilter(InFilterData, bApplyFilter);

	// If we are using shared settings (i.e sharing custom text filters) broadcast the event for all other instances to update
	if(bUseSharedSettings)
	{
		// First save the shared settings for other instances to use
		SaveSettings();
		CustomTextFilterEvent.Broadcast(AsShared());
	}
}

void SFilterList::OnModifyCustomTextFilter(const FCustomTextFilterData& InFilterData, TSharedPtr<ICustomTextFilter<FAssetFilterType>> InFilter)
{
	SAssetFilterBar<FAssetFilterType>::OnModifyCustomTextFilter(InFilterData, InFilter);

	// If we are using shared settings (i.e sharing custom text filters) broadcast the event for all other instances to update
	if(bUseSharedSettings)
	{
		// First save the shared settings for other instances to use
		SaveSettings();
		CustomTextFilterEvent.Broadcast(AsShared());
	}
}

void SFilterList::ModifyCustomTextFilterByLabel(const FCustomTextFilterData& NewFilterData, const FText& FilterLabel)
{
	// Find the filter with the matching label
	TSharedRef<ICustomTextFilter<FAssetFilterType>>* MatchingFilter = CustomTextFilters
		.FindByPredicate([&FilterLabel](const TSharedRef<ICustomTextFilter<FAssetFilterType>>& FilterElement)
			{
				return FilterElement->CreateCustomTextFilterData().FilterLabel.EqualTo(FilterLabel);
			});

	if (MatchingFilter)
	{
		OnModifyCustomTextFilter(NewFilterData, MatchingFilter->ToSharedPtr());
	}
}

bool SFilterList::IsFilteringPathsRecursively() const
{
	return bFilterPathsRecursively;
}

void SFilterList::SetFilterPathsRecursively(const bool bInFilterRecursively)
{
	if (bFilterPathsRecursively != bInFilterRecursively)
	{
		bFilterPathsRecursively = bInFilterRecursively;
		this->OnFilterChanged.ExecuteIfBound();
	}
}

void SFilterList::ToggleFilterPathsRecursively()
{
	SetFilterPathsRecursively(!bFilterPathsRecursively);
}

void SFilterList::ToggleSortFiltersAlphabetically()
{
	bSortFiltersAlphabetically = !bSortFiltersAlphabetically;
	RequestFiltersUpdate();
}

TSharedPtr<SWidget> SFilterList::GetActiveFilterContainer() const
{
	if (HorizontalFilterBox.IsValid() && HorizontalFilterBox->GetChildren()->Num() > 0)
	{
		return HorizontalFilterBox;
	}
	if (VerticalFilterBox.IsValid() && VerticalFilterBox->GetChildren()->Num() > 0)
	{
		return VerticalFilterBox;
	}

	// If neither has children, return nullptr.
	return nullptr;
}

void SFilterList::OnDeleteCustomTextFilter(const TSharedPtr<ICustomTextFilter<FAssetFilterType>> InFilter)
{
	SAssetFilterBar<FAssetFilterType>::OnDeleteCustomTextFilter(InFilter);

	// If we are using shared settings (i.e sharing custom text filters) broadcast the event for all other instances to update
	if(bUseSharedSettings)
	{
		// First save the shared settings for other instances to use
		SaveSettings();
		CustomTextFilterEvent.Broadcast(AsShared());
	}
}

void SFilterList::DeleteCustomTextFilterByLabel(const FText& FilterLabel)
{
	// Find the filter with the matching label
	TSharedRef<ICustomTextFilter<FAssetFilterType>>* MatchingFilter = CustomTextFilters
		.FindByPredicate([&FilterLabel](const TSharedRef<ICustomTextFilter<FAssetFilterType>>& FilterElement)
			{
				return FilterElement->CreateCustomTextFilterData().FilterLabel.EqualTo(FilterLabel);
			});

	if (MatchingFilter)
	{
		OnDeleteCustomTextFilter(MatchingFilter->ToSharedPtr());
	}
}

bool SFilterList::RestoreCustomTextFilterState(const FCustomTextFilterState& InFilterState)
{
	// Find the filter associated with the current instance data from our list of custom text filters
	TSharedRef< ICustomTextFilter<FAssetFilterType> >* Filter =
		CustomTextFilters.FindByPredicate([&InFilterState](const TSharedRef< ICustomTextFilter<FAssetFilterType> >& Element)
	{
		return Element->CreateCustomTextFilterData().FilterLabel.EqualTo(InFilterState.FilterData.FilterLabel);
	});

	// Return if we couldn't find the filter we are trying to restore
	if(!Filter)
	{
		return false;
	}

	// Get the actual FFilterBase
	TSharedRef<FFilterBase<FAssetFilterType>> ActualFilter = Filter->Get().GetFilter().ToSharedRef();

	// Add it to the filter bar, since if it exists in this list it is checked
	TSharedRef<SFilter> AddedFilter = this->AddFilterToBar(ActualFilter);

	// Set the filter as active if it was previously
	AddedFilter->SetEnabled(InFilterState.bIsActive, false);
	this->SetFrontendFilterActive(ActualFilter, InFilterState.bIsActive);

	return true;
}

void SFilterList::OnExternalCustomTextFilterCreated(TSharedPtr<SWidget> BroadcastingFilterList)
{
	// Do nothing if we aren't using shared settings or if the event was broadcasted by this filter list
	if(!bUseSharedSettings || BroadcastingFilterList == AsShared())
	{
		return;
	}

	/* We are going to remove all our custom text filters and re-load them from the shared settings, since a different
	 * instance modified them.
	 */

	// To preserve the state of any checked/active custom text filters
	TArray<FCustomTextFilterState> CurrentCustomTextFilterStates;
	
	for (const TSharedRef<ICustomTextFilter<FAssetFilterType>>& CustomTextFilter : this->CustomTextFilters)
	{
		// Get the actual FFilterBase
		TSharedRef<FFilterBase<FAssetFilterType>> CustomFilter = CustomTextFilter->GetFilter().ToSharedRef();
			
		// Is the filter "checked", i.e visible in the filter bar
		bool bIsChecked = IsFrontendFilterInUse(CustomFilter);

		// Is the filter "active", i.e visible and enabled in the filter bar
		bool bIsActive = IsFilterActive(CustomFilter);

		// Only save the state if the filter is checked so we can restore it
		if(bIsChecked)
		{
			/* Remove the filter from the list (calling SBasicFilterBar::RemoveFilter because we get a compiler error
			*  due to SAssetFilterBar overriding RemoveFilter that takes in an SFilter that hides the parent class function)
			*/
			SBasicFilterBar<FAssetFilterType>::RemoveFilter(CustomFilter, false);
			
			FCustomTextFilterState FilterState;
			FilterState.FilterData = CustomTextFilter->CreateCustomTextFilterData();
			FilterState.bIsChecked = bIsChecked;
			FilterState.bIsActive = bIsActive;
			
			CurrentCustomTextFilterStates.Add(FilterState);
		}
	}

	// Get the shared settings and reload the filters
	FFilterBarSettings* SharedSettings = &UFilterBarConfig::Get()->FilterBars.FindOrAdd(SharedIdentifier);
	LoadCustomTextFilters(SharedSettings);

	// Restore the state of any previously active ones
	for(const FCustomTextFilterState& SavedFilterState : CurrentCustomTextFilterStates)
	{
		RestoreCustomTextFilterState(SavedFilterState);
	}
}

void SFilterList::UpdateCustomTextFilterIncludes(const bool InIncludeClassName, const bool InIncludeAssetPath, const bool InIncludeCollectionNames)
{
	bIncludeClassName = InIncludeClassName;
	bIncludeAssetPath = InIncludeAssetPath;
	bIncludeCollectionNames = InIncludeCollectionNames;

	for (TSharedPtr<ICustomTextFilter<FAssetFilterType>> CustomTextFilter : CustomTextFilters)
	{
		// This is a safe cast, since SFilterList will always and only have FFilterListCustomTextFilters
		if(TSharedPtr<FFrontendFilter_CustomText> FilterListCustomTextFilter = StaticCastSharedPtr<FFrontendFilter_CustomText>(CustomTextFilter))
		{
			FilterListCustomTextFilter->UpdateCustomTextFilterIncludes(bIncludeClassName, bIncludeAssetPath, bIncludeCollectionNames);
		}
	}
}

void SFilterList::SaveSettingsInternal(FFilterBarSettings* InSettingsToSave)
{
	check(InSettingsToSave);

	// If this instance doesn't want to use the shared settings, save the settings normally
	if(!bUseSharedSettings)
	{
		// Only save the orientation if we allow dynamic modification and saving
		InSettingsToSave->bIsLayoutSaved = this->bCanChangeOrientation;
		if (this->bCanChangeOrientation)
		{
			InSettingsToSave->FilterBarLayout = this->FilterBarLayout;

			if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
			{
				InSettingsToSave->bFilterRecursively = this->bFilterPathsRecursively;
			}
		}
		InSettingsToSave->bSortFiltersAlphabetically = this->bSortFiltersAlphabetically;

		SAssetFilterBar<FAssetFilterType>::SaveSettingsInternal(InSettingsToSave);
		return;
	}

	if(FilterBarIdentifier.IsNone())
	{
		UE_LOGF(LogSlate, Error, "SFilterList Requires that you specify a FilterBarIdentifier to save settings");
		return;
	}

	// Get the settings unique to this instance and the common settings
	FFilterBarSettings* InstanceSettings = &UFilterBarConfig::Get()->FilterBars.FindOrAdd(FilterBarIdentifier);
	FFilterBarSettings* SharedSettings = &UFilterBarConfig::Get()->FilterBars.FindOrAdd(SharedIdentifier);

	// Empty both the configs, we are just going to re-save everything there
	InstanceSettings->Empty();
	SharedSettings->Empty();

	// Save all the programmatically added filters normally
	SaveFilters(InstanceSettings);

	/** For each custom text filter: Save the filterdata into the common settings, so that all instances that use it
	 *	are synced.
	 *	For each CHECKED custom text filter: Save just the filter name, and the checked and active state into the
	 *	instance settings. Those are specific to this instance (i.e we don't want a filter to be active in all
	 *	instances if activated in one)
	 */
	for (const TSharedRef<ICustomTextFilter<FAssetFilterType>>& CustomTextFilter : this->CustomTextFilters)
	{
		// Get the actual FFilterBase
		TSharedRef<FFilterBase<FAssetFilterType>> CustomFilter = CustomTextFilter->GetFilter().ToSharedRef();
			
		// Is the filter "checked", i.e visible in the filter bar
		bool bIsChecked = IsFrontendFilterInUse(CustomFilter);

		// Is the filter "active", i.e visible and enabled in the filter bar
		bool bIsActive = IsFilterActive(CustomFilter);

		// Get the data associated with this filter
		FCustomTextFilterData FilterData = CustomTextFilter->CreateCustomTextFilterData();

		// Just save the filter data into the shared settings
		FCustomTextFilterState SharedFilterState;
		SharedFilterState.FilterData = FilterData;
		SharedSettings->CustomTextFilters.Add(SharedFilterState);

		if(bIsChecked)
		{
			// Create a duplicate filter data that just contains the filter label for this instance to know
			FCustomTextFilterData InstanceFilterData;
			InstanceFilterData.FilterLabel = FilterData.FilterLabel;
			
			// Just save the filter name and enabled/active state into the shared settings
			FCustomTextFilterState InstanceFilterState;
			InstanceFilterState.bIsChecked = bIsChecked;
			InstanceFilterState.bIsActive = bIsActive;
			InstanceFilterState.FilterData = InstanceFilterData;
			
			InstanceSettings->CustomTextFilters.Add(InstanceFilterState);
		}
	}

	// Only save the orientation if we allow dynamic modification and saving
	InstanceSettings->bIsLayoutSaved = this->bCanChangeOrientation;
	if(this->bCanChangeOrientation)
	{
		InstanceSettings->FilterBarLayout = this->FilterBarLayout;
	}
	InSettingsToSave->bSortFiltersAlphabetically = this->bSortFiltersAlphabetically;

	SaveConfig();
}

void SFilterList::SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString)
{
	// Workaround for backwards compatibility with filters that save settings until they are ported to EditorConfig
	for ( const TSharedPtr<SFilter> Filter : this->Filters )
	{
		const FString FilterName = Filter->GetFilterName();

		// If it is a FrontendFilter
		if ( Filter->GetFrontendFilter().IsValid() )
		{
			const TSharedPtr<FFilterBase<FAssetFilterType>>& FrontendFilter = Filter->GetFrontendFilter();
			const FString CustomSettingsString = FString::Printf(TEXT("%s.CustomSettings.%s"), *SettingsString, *FilterName);
			FrontendFilter->SaveSettings(IniFilename, IniSection, CustomSettingsString);
		}
	}
	
	SaveSettings();
}

void SFilterList::LoadSettings(const FName& InInstanceName, const FString& IniFilename, const FString& IniSection, const FString& SettingsString)
{
	// Workaround for backwards compatibility with filters that save settings until they are ported to EditorConfig
	for ( auto FrontendFilterIt = this->AllFrontendFilters.CreateIterator(); FrontendFilterIt; ++FrontendFilterIt )
	{
		TSharedRef<FFilterBase<FAssetFilterType>>& FrontendFilter = *FrontendFilterIt;
		const FString& FilterName = FrontendFilter->GetName();

		const FString CustomSettingsString = FString::Printf(TEXT("%s.CustomSettings.%s"), *SettingsString, *FilterName);
		FrontendFilter->LoadSettings(IniFilename, IniSection, CustomSettingsString);
	}
	
	LoadSettings(InInstanceName);
}


void SFilterList::LoadSettings(const FName& InInstanceName)
{
	// If this instance doesn't want to use the shared settings, load the settings normally
	if(!bUseSharedSettings)
	{
		const FFilterBarSettings* FilterBarConfig = GetConstConfig();

		if(!FilterBarConfig)
		{
			UE_LOGF(LogSlate, Error, "SFilterBar Requires that you specify a FilterBarIdentifier to load settings");
			return;
		}

		/* Only load the setting if we saved it, aka if SaveSettings() was ever called and bCanChangeOrientation is true */
		if(FilterBarConfig->bIsLayoutSaved)
		{
			this->SetFilterLayout(FilterBarConfig->FilterBarLayout);
			if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
			{
				this->SetFilterPathsRecursively(FilterBarConfig->bFilterRecursively);
			}
			this->SetSortFiltersAlphabetically(FilterBarConfig->bSortFiltersAlphabetically);
		}

		SAssetFilterBar<FAssetFilterType>::LoadSettings();

		return;
	}

	if(InInstanceName.IsNone())
	{
		UE_LOGF(LogSlate, Error, "SFilterList Requires that you specify a FilterBarIdentifier to load settings");
		return;
	}

	// Get the settings unique to this instance and the common settings
	const FFilterBarSettings* InstanceSettings = UFilterBarConfig::Get()->FilterBars.Find(InInstanceName);
	const FFilterBarSettings* SharedSettings = UFilterBarConfig::Get()->FilterBars.Find(SharedIdentifier);

	// Load the filters specified programmatically normally
	LoadFilters(InstanceSettings);

	// Load the custom text filters from the shared settings
	LoadCustomTextFilters(SharedSettings);
	
	// From the instance settings, get each checked filter and set the checked and active state
	for(const FCustomTextFilterState& FilterState : InstanceSettings->CustomTextFilters)
	{
		if(!RestoreCustomTextFilterState(FilterState))
		{
			UE_LOGF(LogSlate, Warning, "SFilterList was unable to load the following custom text filter: %ls", *FilterState.FilterData.FilterLabel.ToString());
		}
	}

	if(InstanceSettings->bIsLayoutSaved)
	{
		FilterBarLayout = InstanceSettings->FilterBarLayout;
		SetSortFiltersAlphabetically(InstanceSettings->bSortFiltersAlphabetically);
	}

	// We want to call this even if the Layout isn't saved, to make sure OnFilterBarLayoutChanging is fired
	SetFilterLayout(FilterBarLayout);
	
	this->OnFilterChanged.ExecuteIfBound();
}

void SFilterList::LoadSettings()
{
	LoadSettings(FilterBarIdentifier);
}

void SFilterList::LoadCustomTextFilters(const FFilterBarSettings* FilterBarConfig)
{
	CustomTextFilters.Empty();
	
	// Extract just the filter data from the common settings
	for(const FCustomTextFilterState& FilterState : FilterBarConfig->CustomTextFilters)
	{
		// Create an ICustomTextFilter using the provided delegate
		TSharedRef<ICustomTextFilter<FAssetFilterType>> NewTextFilter = this->CreateTextFilter.Execute().ToSharedRef();

		// Get the actual FFilterBase
		TSharedRef<FFilterBase<FAssetFilterType>> NewFilter = NewTextFilter->GetFilter().ToSharedRef();

		// Set the internals of the custom text filter from what we have saved
		NewTextFilter->SetFromCustomTextFilterData(FilterState.FilterData);

		// Add this to our list of custom text filters
		CustomTextFilters.Add(NewTextFilter);
	}
}

void SFilterList::AddWidgetToCurrentLayout(TSharedRef<SWidget> InWidget)
{
	if(FilterBarLayout == EFilterBarLayout::Horizontal)
	{
		HorizontalFilterBox->AddSlot()
		[
			InWidget
		];
	}
	else
	{
		VerticalFilterBox->AddSlot()
		[
			InWidget
		];
	}
}

/////////////////////////////////////////
// FFilterListCustomTextFilter
/////////////////////////////////////////

FFrontendFilter_CustomText::FFrontendFilter_CustomText()
	: FFrontendFilter(nullptr)
{

}

/** Returns the system name for this filter */
FString FFrontendFilter_CustomText::GetName() const
{
	// Todo: Find some way to enforce this on all custom text filter interfaces
	return FCustomTextFilter<FAssetFilterType>::GetFilterTypeName().ToString();
}

FText FFrontendFilter_CustomText::GetDisplayName() const
{
	return DisplayName;
}
FText FFrontendFilter_CustomText::GetToolTipText() const
{
	return RawFilterText;
}

FLinearColor FFrontendFilter_CustomText::GetColor() const
{
	return Color;
}

void FFrontendFilter_CustomText::UpdateCustomTextFilterIncludes(const bool InIncludeClassName, const bool InIncludeAssetPath, const bool InIncludeCollectionNames)
{
	bIncludeClassName = InIncludeClassName;
	bIncludeAssetPath = InIncludeAssetPath;
	bIncludeCollectionNames = InIncludeCollectionNames;
}

void FFrontendFilter_CustomText::SetFromCustomTextFilterData(const FCustomTextFilterData& InFilterData)
{
	Color = InFilterData.FilterColor;
	DisplayName = InFilterData.FilterLabel;
	RawFilterText = InFilterData.FilterString;
}

FCustomTextFilterData FFrontendFilter_CustomText::CreateCustomTextFilterData() const
{
	FCustomTextFilterData CustomTextFilterData;

	CustomTextFilterData.FilterColor = Color;
	CustomTextFilterData.FilterLabel = DisplayName;
	CustomTextFilterData.FilterString = RawFilterText;

	return CustomTextFilterData;
}

TSharedPtr<FFilterBase<FAssetFilterType>> FFrontendFilter_CustomText::GetFilter()
{
	return AsShared();
}

TOptional<FText> FFrontendFilter_CustomText::GetAsCustomTextFilter()
{
	return RawFilterText;
}

#undef LOCTEXT_NAMESPACE
