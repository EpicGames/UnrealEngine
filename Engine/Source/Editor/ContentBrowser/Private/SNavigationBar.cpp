// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationBar.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SSuggestionTextBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

struct FNavigationCrumb
{
	FString Path;
	bool bHasChildren; // Whether to show the > button next to this location allowing navigation to children
};

// Category of list view item to choose an icon
enum class ELocationSource
{
	History,
	Suggestion
};

// Type for list view control
struct FLocationItem
{
	FText DisplayPath;
	FString Path;
	ELocationSource Source;
};

class SLocationListView : public SListView<TSharedPtr<FLocationItem>>
{
public:
	SLATE_BEGIN_ARGS(SLocationListView)
		{}
		SLATE_EVENT(FOnGenerateRow, OnGenerateRow)
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		SLATE_EVENT(FOnKeyDown, OnKeyDownHandler)
		SLATE_EVENT(FOnKeyChar, OnKeyCharHandler)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		OnKeyCharHandler = InArgs._OnKeyCharHandler;
		using SSuper = SListView<TSharedPtr<FLocationItem>>;
		SSuper::Construct(SSuper::FArguments()
			.ListItemsSource(&Items)
			.SelectionMode(ESelectionMode::Single)
			.OnGenerateRow(InArgs._OnGenerateRow)
			.OnSelectionChanged(InArgs._OnSelectionChanged)
			.OnKeyDownHandler(InArgs._OnKeyDownHandler)
		);
	}

	// Forward typing events to the navigation bar so that focus can be moved back to the editable text box 
	FReply OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent)
	{
		if (OnKeyCharHandler.IsBound())
		{
			return OnKeyCharHandler.Execute(MyGeometry, InCharacterEvent);
		}
		return FReply::Unhandled();
	}
	
	void SetItems(TArray<FNavigationBarComboOption> InItems, ELocationSource InSource)
	{
		RequestListRefresh();
		ClearSelection();
		Items.Reset(InItems.Num());
		for (FNavigationBarComboOption& Item : InItems)
		{
			Items.Emplace(MakeShared<FLocationItem>(FLocationItem{ MoveTemp(Item.DisplayPath), MoveTemp(Item.Path), InSource }));
		}
	}
	
	FReply NavigateToFirst()
	{
		if (Items.Num() > 0)
		{
			SetSelection(Items[0], ESelectInfo::OnNavigation);
			return FReply::Handled().SetUserFocus(AsShared(), EFocusCause::SetDirectly);
		}
		return FReply::Handled();
	}
	FReply NavigateToLast()
	{
		if (Items.Num() > 0)
		{
			SetSelection(Items.Last(), ESelectInfo::OnNavigation);
			return FReply::Handled().SetUserFocus(AsShared(), EFocusCause::SetDirectly);
		}
		return FReply::Handled();
	}
	
protected:
	FOnKeyChar OnKeyCharHandler;
	TArray<TSharedPtr<FLocationItem>> Items;
};

FNavigationBarComboOption::FNavigationBarComboOption(const FText& InDisplayPath, const FString& InPath)
	: DisplayPath(InDisplayPath)
	, Path(InPath)
{
}

void SNavigationBar::Construct(const FArguments& InArgs)
{
	OnGetComboOptions = InArgs._GetComboOptions;
	OnCompletePrefix = InArgs._OnCompletePrefix;
	OnNavigateToPath = InArgs._OnNavigateToPath;
	OnGetEditPathAsText = InArgs._OnGetEditPathAsText;
	OnPathClicked = InArgs._OnPathClicked;
	OnGetPathMenuContent = InArgs._GetPathMenuContent;

	ComboBoxStyle = InArgs._ComboBoxStyle;
	TextBoxStyle = InArgs._TextBoxStyle;
	TableRowStyle = InArgs._ItemStyle;
	bShowMenuBackground = false;
	
	const ISlateStyle& StyleSet = FAppStyle::Get();
	SuggestionIcon = StyleSet.GetBrush("Icons.Search");
	HistoryIcon = StyleSet.GetBrush("Icons.Recent");

	SComboButton::Construct(SComboButton::FArguments()
		.ComboButtonStyle(&ComboBoxStyle->ComboButtonStyle)
		.ButtonStyle(&ComboBoxStyle->ComboButtonStyle.ButtonStyle)
		.ContentPadding(ComboBoxStyle->ContentPadding)
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			[
				SNew(SOverlay)
				// Breadcrumb trail aligned to left in combo box
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					// Invisible button under visible controls to handle clicking in blank space
					SNew(SButton)
					.OnClicked(this, &SNavigationBar::HandleBlankSpaceClicked)	
					.ButtonStyle( FAppStyle::Get(), "NoBorder" )
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						.FillWidth(1.0f)
						[
							SAssignNew(BreadcrumbBar, SBreadcrumbTrail<FNavigationCrumb>)
							.Visibility(this, &SNavigationBar::GetNonEditVisibility)
							.TextStyle(InArgs._BreadcrumbTextStyle)
							.ButtonStyle(InArgs._BreadcrumbButtonStyle)
							.ButtonContentPadding(InArgs._BreadcrumbButtonContentPadding)
							.DelimiterImage(InArgs._BreadcrumbDelimiterImage)
							.OnCrumbClicked(this, &SNavigationBar::HandleCrumbClicked)
							.HasCrumbMenuContent(this, &SNavigationBar::HandleHasCrumbMenuContent)
							.GetCrumbMenuContent(this, &SNavigationBar::HandleGetCrumbMenuContent)
						]
						// Spacer to allow some blank space to enter edit mode always
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Fill)
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FAppStyle::GetNoBrush())
							.DesiredSizeOverride(FVector2D(10.0f, 1.0f))
						]
					]
				]
				// Editable text box taking up all space
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SAssignNew( EditableText, SEditableText )
					.Visibility(this, &SNavigationBar::GetEditTextVisibility)
					.OnTextCommitted(this, &SNavigationBar::HandleTextCommitted)
					.OnTextChanged(this, &SNavigationBar::HandleTextChanged)
					.SelectAllTextWhenFocused(true) 
					.ClearKeyboardFocusOnCommit(true)
					.OnKeyDownHandler(this, &SNavigationBar::HandleEditableTextKeyDown)
				]
			]
		]
		.MenuContent()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(StyleSet.GetMargin("Menu.Heading.Padding"))
			.AutoHeight()
			[
				// This matches multibox heading control, it could be refactored to use that (or a shared control)
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(this, &SNavigationBar::GetPopupHeading)
					.TextStyle(&StyleSet, "Menu.Heading")
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(14.f, 0.f, 0.f, 0.f))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
					.Thickness(1.0f)
					.SeparatorImage(StyleSet.GetBrush("Menu.Separator"))
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.MaxHeight(300.0f)
			.Padding(2.0f,2.0f)
			[
				SAssignNew(ComboListView, SLocationListView)
				.OnGenerateRow(this, &SNavigationBar::HandleGenerateComboRow)
				.OnSelectionChanged(this, &SNavigationBar::HandleComboSelectionChanged)
				.OnKeyDownHandler(this, &SNavigationBar::HandleComboKeyDown)
				.OnKeyCharHandler(this, &SNavigationBar::HandleComboKeyChar)
			]
		]
	);

	BreadcrumbBar->ScrollToEnd();
	ComboListView->SetBackgroundBrush(FStyleDefaults::GetNoBrush());
	EditableText->SetTextBlockStyle(&TextBoxStyle->TextStyle);
	SetMenuContentWidgetToFocus(ComboListView);
}

void SNavigationBar::ClearPaths()
{
	BreadcrumbBar->ClearCrumbs();
}

void SNavigationBar::PushPath(const FText& ElementText, const FString& FullPath, bool bHasChildren)
{
	BreadcrumbBar->PushCrumb(ElementText, FNavigationCrumb{FullPath, bHasChildren});	
	BreadcrumbBar->ScrollToEnd();
}

void SNavigationBar::HandleCrumbClicked(const FNavigationCrumb& Crumb)
{
	OnPathClicked.ExecuteIfBound(Crumb.Path);
}

bool SNavigationBar::HandleHasCrumbMenuContent(const FNavigationCrumb& Crumb)
{
	return Crumb.bHasChildren;
}

TSharedRef<SWidget> SNavigationBar::HandleGetCrumbMenuContent(const FNavigationCrumb& Crumb)
{
	if (OnGetPathMenuContent.IsBound())
	{
		return OnGetPathMenuContent.Execute(Crumb.Path);
	}
	return SNullWidget::NullWidget;
}

bool SNavigationBar::SupportsKeyboardFocus() const 
{
	return bIsFocusable;
}

void SNavigationBar::OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent) 
{
	// Stop editing and hide dropdown when focusing away from either the navigation bar or the popup
	TSharedPtr<SWindow> Popup = PopupWindowPtr.Pin();
	if(PreviousFocusPath.ContainsWidget(this) || PreviousFocusPath.ContainsWidget(Popup.Get()))
	{
		if (!NewWidgetPath.ContainsWidget(this) && !NewWidgetPath.ContainsWidget(Popup.Get()))
		{
			if (CompletionTimerHandle.IsValid())
			{
				UnRegisterActiveTimer(CompletionTimerHandle.ToSharedRef());	
				CompletionTimerHandle.Reset();
			}

			EditTextVisibility = EVisibility::Hidden;
			SetIsOpen(false, false);
		}
	}
}

FReply SNavigationBar::OnButtonClicked()
{
	GenerateHistoryOptions();
	return SComboButton::OnButtonClicked();
}

void SNavigationBar::GenerateHistoryOptions()
{
	PopupHeading = LOCTEXT("NavigationBar.HistoryHeader", "HISTORY");
	TArray<FNavigationBarComboOption> Options;
	if (OnGetComboOptions.IsBound())
	{
		Options = OnGetComboOptions.Execute();
	}
	ComboListView->SetItems(MoveTemp(Options), ELocationSource::History);
}

void SNavigationBar::GenerateCompletionOptions(const FString& Prefix)
{
	PopupHeading = LOCTEXT("NavigationBar.SuggestionsHeader", "SUGGESTIONS"); 
	TArray<FNavigationBarComboOption> Options;
	if (OnCompletePrefix.IsBound())
	{
		Options = OnCompletePrefix.Execute(Prefix);
	}
	ComboListView->SetItems(MoveTemp(Options), ELocationSource::Suggestion);
}

FReply SNavigationBar::HandleBlankSpaceClicked()
{
	StartEditingPath();
	return FReply::Handled();
}

FReply SNavigationBar::HandleComboKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharEvent)
{
	// Add the typed character to the currently suggested suggestion and return focus to the editable text box
	TArray<TSharedPtr<FLocationItem>> SelectedItems = ComboListView->GetSelectedItems();
	if (SelectedItems.Num() != 0)
	{
		FString NewText = SelectedItems[0]->Path;
		NewText += InCharEvent.GetCharacter();
		EditableText->SetText(FText::FromString(NewText));
		EditableText->SetSelectAllTextWhenFocused(false);

		return FReply::Handled().SetUserFocus(EditableText.ToSharedRef());
	}
	return FReply::Unhandled();	
}

FReply SNavigationBar::HandleComboKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Enter)
	{
		// Select (and navigate to) the first selected item on hitting enter
		TArray<TSharedPtr<FLocationItem>> SelectedItems = ComboListView->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			HandleComboSelectionChanged(SelectedItems[0], ESelectInfo::OnKeyPress);

			// Set focus back to navigation box 
			FSlateApplication::Get().ForEachUser([this](FSlateUser& User) 
			{
				TSharedRef<SWidget> ThisRef = this->AsShared();
				if (User.IsWidgetInFocusPath(this->ComboListView))
				{
					User.SetFocus(ThisRef);
				}
			});

			return FReply::Handled();
		}
	}
	else if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		// Close text box and popup when pressing escape
		EditTextVisibility = EVisibility::Hidden;
		SetIsOpen(false, false);
		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Tab)
	{
		// Put current selected item into editable text and move focus back there
		TArray<TSharedPtr<FLocationItem>> SelectedItems = ComboListView->GetSelectedItems();
		if (SelectedItems.Num() > 0)
		{
			FString NewText = SelectedItems[0]->Path + TEXT("/"); // TODO: Ideally only add slash if this prefix has children itself 
			EditableText->SetText(FText::FromString(NewText));
			EditableText->SetSelectAllTextWhenFocused(false);
			return FReply::Handled().SetUserFocus(EditableText.ToSharedRef(), EFocusCause::SetDirectly);
		}
		return FReply::Handled(); // Swallow event to avoid confusion with items being selected or not 
	}

	return FReply::Unhandled();
}

FReply SNavigationBar::HandleEditableTextKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent)
{
	if (EditableText->HasKeyboardFocus() && KeyEvent.GetKey() == EKeys::Escape)
	{
		// Stop editing and discard when user presses escape 
		return FReply::Handled().SetUserFocus(AsShared(), EFocusCause::Cleared);
	}

	if (EditableText->HasKeyboardFocus() && KeyEvent.GetKey() == EKeys::Tab)
	{
		// Swallow tab key to prevent moving focus away because Tab is used by the popup if it's focused
		return FReply::Handled();
	}

	bool bUp = KeyEvent.GetKey() == EKeys::Up;
	bool bDown = KeyEvent.GetKey() == EKeys::Down;
	
	if (bUp || bDown)
	{
		if (!IsOpen())
		{
			// Open popup if completions haven't been generated from timer after text edit yet
			GenerateCompletionOptions(EditableText->GetText().ToString());
			SetIsOpen(true, false);
			return FReply::Handled();
		}
		else 
		{
			// Switch focus from text box to first or last element of popup to start navigation
			if (bUp)
			{
				return ComboListView->NavigateToLast();
			}
			else 
			{
				return ComboListView->NavigateToFirst();
			}
		}
	}

	return FReply::Unhandled();
}

void SNavigationBar::StartEditingPath()
{
	EditTextVisibility = EVisibility::Visible;
	FText Text = FText::GetEmpty();
	if (BreadcrumbBar->HasCrumbs())
	{
		FNavigationCrumb Crumb = BreadcrumbBar->PeekCrumb();
		if (OnGetEditPathAsText.IsBound())
		{
			Text = OnGetEditPathAsText.Execute(Crumb.Path);
		}
	}
	EditableText->SetSelectAllTextWhenFocused(true);
	EditableText->SetText(Text);
	FSlateApplication::Get().SetKeyboardFocus(EditableText.ToSharedRef(), EFocusCause::SetDirectly);
}

void SNavigationBar::HandleTextChanged(const FText& NewText)
{
	if (EditableText->GetVisibility().IsVisible() && !bNoSuggestionsFromTextChange)
	{
		if (CompletionTimerHandle.IsValid())
		{
			UnRegisterActiveTimer(CompletionTimerHandle.ToSharedRef());	
		}
		// Generate completion suggestions shortly after user stops typing
		CompletionTimerHandle = RegisterActiveTimer(0.1f, FWidgetActiveTimerDelegate::CreateSP(this, &SNavigationBar::HandleUpdateCompletionOptions));
	}
}
	
void SNavigationBar::HandleTextCommitted(const FText& InText, ETextCommit::Type CommitType)
{
	switch (CommitType)
	{
		case ETextCommit::Default:
		case ETextCommit::OnCleared:
		case ETextCommit::OnUserMovedFocus:
			return; // Discard changes and don't navigate
	}

	// Stop editing and navigate to new path
	EditableText->SetSelectAllTextWhenFocused(true);
	EditTextVisibility = EVisibility::Hidden;
	SetIsOpen(false, false);
	OnNavigateToPath.ExecuteIfBound(InText.ToString());
}

EActiveTimerReturnType SNavigationBar::HandleUpdateCompletionOptions(double InCurrentTime, float InDeltaTime)
{
	GenerateCompletionOptions(EditableText->GetText().ToString());
	SetIsOpen(true, false);
	CompletionTimerHandle = nullptr;
	return EActiveTimerReturnType::Stop; // Never run more than once unless text changes again 
}

TSharedRef<ITableRow> SNavigationBar::HandleGenerateComboRow(TSharedPtr<FLocationItem> ForItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FLocationItem>>, OwnerTable)
		.Padding(ComboBoxStyle->MenuRowPadding)
		.Style(TableRowStyle)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SImage)
				.Image(GetImageForItem(*ForItem))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.Padding(2.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(ForItem->DisplayPath)
			]
		];
}

void SNavigationBar::HandleComboSelectionChanged(TSharedPtr<FLocationItem> SelectedItem, ESelectInfo::Type SelectInfo)
{
	if (!SelectedItem.IsValid())
	{
		return;
	}

	switch(SelectInfo)
	{
		case ESelectInfo::Direct:
		case ESelectInfo::OnKeyPress:
		case ESelectInfo::OnMouseClick:
			// Stop editing/hide popup and navigate to chosen path
			EditTextVisibility = EVisibility::Hidden;
			SetIsOpen(false);
			OnNavigateToPath.ExecuteIfBound(SelectedItem->Path);
			break;
		case ESelectInfo::OnNavigation:
			// Mirror chosen item into text box without moving focus back or triggering new suggestions
			TGuardValue<bool> BlockSuggestions{bNoSuggestionsFromTextChange, true};
			EditableText->SetText(FText::FromString(SelectedItem->Path));
			break;
	}
}

FText SNavigationBar::GetPopupHeading() const 
{
	return PopupHeading;
}

const FSlateBrush* SNavigationBar::GetImageForItem(const FLocationItem& ForItem) const
{
	switch (ForItem.Source)
	{
		case ELocationSource::History:
			return HistoryIcon;
		case ELocationSource::Suggestion:
		default:
			return SuggestionIcon;
	}
}

EVisibility SNavigationBar::GetEditTextVisibility() const
{
	return EditTextVisibility;
}

EVisibility SNavigationBar::GetNonEditVisibility() const
{
	if (EditTextVisibility.IsVisible())
	{
		return EVisibility::Hidden;
	}
	else
	{
		return EVisibility::Visible;
	}
}

#undef LOCTEXT_NAMESPACE