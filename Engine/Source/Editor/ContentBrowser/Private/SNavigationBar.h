// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Styling/CoreStyle.h"
#include "Styling/ToolBarStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"

class FText;
// Type of individual elements in breadcrumb bar, defines the current location or one of its parents.
struct FNavigationCrumb;
class ITableRow;
template<typename ItemType>
class SBreadcrumbTrail;
class SComboButton;
class SEditableText;
class SLocationListView;
class STableViewBase;
class USlateWidgetStyleAsset;

struct FNavigationBarComboOption
{
	FNavigationBarComboOption() = default;
	FNavigationBarComboOption(const FText& InDisplayPath, const FString& InPath);

	FText DisplayPath;
	FString Path;
};

using FOnPathClicked = TDelegate<void(const FString&)>;
using FGetPathMenuContent = TDelegate<TSharedRef<SWidget>(const FString&)>;
using FGetComboOptions = TDelegate<TArray<FNavigationBarComboOption>(void)>;
using FOnCompletePrefix = TDelegate<TArray<FNavigationBarComboOption>(const FString&)>;
using FOnNavigateToPath = TDelegate<void(const FString&)>;
using FGetEditPathAsText = TDelegate<FText(const FString&)>;

// Private class for internal list view, stores a full possible location to nagivate to.
struct FLocationItem;


class SNavigationBar : public SComboButton
{
public:
    SLATE_BEGIN_ARGS(SNavigationBar) 
	: _ComboBoxStyle(&FAppStyle::Get().GetWidgetStyle< FComboBoxStyle >("ComboBox"))
	, _ItemStyle(&FAppStyle::Get().GetWidgetStyle< FTableRowStyle >("ComboBox.Row"))
	, _TextBoxStyle(&FAppStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
	, _BreadcrumbButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
	, _BreadcrumbTextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
	, _BreadcrumbButtonContentPadding(FMargin(2,2))
	, _BreadcrumbDelimiterImage(FAppStyle::Get().GetBrush("Icons.ChevronRight"))
    {}
		/** The styling of the combo box which wraps the other navigation controls */
		SLATE_STYLE_ARGUMENT( FComboBoxStyle, ComboBoxStyle )

		/** Styling for items in the drop down history/suggestion box */
		SLATE_STYLE_ARGUMENT(FTableRowStyle, ItemStyle)

		/** The styling of the suggestions/history popup */
		SLATE_STYLE_ARGUMENT(FTableViewStyle, SuggestionsStyle)

		/** The styling of the textbox when in edit mode */
		SLATE_STYLE_ARGUMENT(FEditableTextBoxStyle, TextBoxStyle)
		
		/** The name of the style to use for the crumb buttons */
		SLATE_STYLE_ARGUMENT(FButtonStyle, BreadcrumbButtonStyle)

		/** The name of the style to use for the crumb button text */
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, BreadcrumbTextStyle)

		/** The padding for the content in crumb buttons */
		SLATE_ATTRIBUTE(FMargin, BreadcrumbButtonContentPadding)
	
		/** The image to use between crumb trail buttons */
		SLATE_ATTRIBUTE(const FSlateBrush*, BreadcrumbDelimiterImage)
		
		/** Callback for when a new location is committed via text entry */
        SLATE_EVENT(FOnNavigateToPath , OnNavigateToPath)

		/** Called when an invididual button is clicked, after the later crumbs were popped */
		SLATE_EVENT(FOnPathClicked , OnPathClicked)

		/** Called to get dropdown options for a crumb separator's menu */
		SLATE_EVENT(FGetPathMenuContent, GetPathMenuContent)
		
		/** Called to get a list of options when clicking the combo botton on the side of the box */
		SLATE_EVENT(FGetComboOptions, GetComboOptions)
		
		/** Called to get suggested text completion values for the currently edit text */
		SLATE_EVENT(FOnCompletePrefix, OnCompletePrefix)
		
		/** Called to get the edit location of a path (e.g. the last breadcrumb) as text */
		SLATE_EVENT(FGetEditPathAsText, OnGetEditPathAsText)

    SLATE_END_ARGS()
    
    void Construct(const FArguments& InArgs);
    
	// Remove all paths from the breadcrumb bar
	void ClearPaths();
	// Add a new segment to the breadcrumb bar
	void PushPath(const FText& SegmentDisplayText, const FString& FullLocation, bool bHasChildren);
	// Replace the breadcrumb bar with an editable text box and focus it
	void StartEditingPath();

private:
	// SWidget overrides
	virtual void OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent) override;
	// SComboButton overrides
	virtual FReply OnButtonClicked() override;

	// Handle clicking on part of the navigation bar which is not the combo button or the breadcrumb bar, to switch to edit mode
	FReply HandleBlankSpaceClicked();
	void HandleTextChanged(const FText& NewText);
	void HandleTextCommitted(const FText& InText, ETextCommit::Type CommitType);
	void HandleComboSelectionChanged(TSharedPtr<FLocationItem> SelectedItem, ESelectInfo::Type SelectInfo);
	TSharedRef<ITableRow> HandleGenerateComboRow(TSharedPtr<FLocationItem> ForItem, const TSharedRef<STableViewBase>& OwnerTable);
	FReply HandleEditableTextKeyDown(const FGeometry& MyGeometry, const FKeyEvent& KeyEvent);
	FReply HandleComboKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	FReply HandleComboKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharEvent);
	void HandleCrumbClicked(const FNavigationCrumb& Crumb);
	bool HandleHasCrumbMenuContent(const FNavigationCrumb& Crumb);
	TSharedRef<SWidget> HandleGetCrumbMenuContent(const FNavigationCrumb& Crumb);
	
	EActiveTimerReturnType HandleUpdateCompletionOptions(double InCurrentTime, float InDeltaTime);

	void GenerateHistoryOptions();
	void GenerateCompletionOptions(const FString& Prefix);
	
	EVisibility GetEditTextVisibility() const;
	EVisibility GetNonEditVisibility() const;
	FText GetPopupHeading() const;
	const FSlateBrush* GetImageForItem(const FLocationItem& ForItem) const;

	virtual bool SupportsKeyboardFocus() const override;

	const FComboBoxStyle* ComboBoxStyle;
	const FTableRowStyle* TableRowStyle;
	const FEditableTextBoxStyle* TextBoxStyle;
	const FSlateBrush* SuggestionIcon;
	const FSlateBrush* HistoryIcon;
	
	EVisibility EditTextVisibility = EVisibility::Hidden;
	TSharedPtr<FActiveTimerHandle> CompletionTimerHandle;

	TSharedPtr<SEditableText> EditableText;
    TSharedPtr<SBreadcrumbTrail<FNavigationCrumb>> BreadcrumbBar;
	TSharedPtr<SLocationListView> ComboListView;
	
	FText PopupHeading;
	
	bool bNoSuggestionsFromTextChange = false;
	
	FOnNavigateToPath OnNavigateToPath;
	FGetComboOptions OnGetComboOptions;
	FOnCompletePrefix OnCompletePrefix;	
	FGetEditPathAsText OnGetEditPathAsText;
	FOnPathClicked OnPathClicked;
	FGetPathMenuContent OnGetPathMenuContent;
};