// Copyright Epic Games, Inc. All Rights Reserved.


#include "SPathPicker.h"

#include "ContentBrowserDataFilter.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserItemData.h"
#include "ContentBrowserPluginFilters.h"
#include "ContentBrowserStyle.h"
#include "Delegates/Delegate.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IContentBrowserDataModule.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Misc/Paths.h"
#include "SourcesSearch.h"
#include "SPathView.h"
#include "SSearchToggleButton.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

void SPathPicker::Construct( const FArguments& InArgs )
{
	for (auto DelegateIt = InArgs._PathPickerConfig.SetPathsDelegates.CreateConstIterator(); DelegateIt; ++DelegateIt)
	{
		if ((*DelegateIt) != nullptr)
		{
			(**DelegateIt) = FSetPathPickerPathsDelegate::CreateSP(this, &SPathPicker::SetPaths);
		}
	}

	OnPathSelected = InArgs._PathPickerConfig.OnPathSelected;
	OnGetFolderContextMenu = InArgs._PathPickerConfig.OnGetFolderContextMenu;
	OnGetPathContextMenuExtender = InArgs._PathPickerConfig.OnGetPathContextMenuExtender;
	bOnPathSelectedPassesVirtualPaths = InArgs._PathPickerConfig.bOnPathSelectedPassesVirtualPaths;

	// clang-format off
	ChildSlot
	[
		SAssignNew(PathViewPtr, SPathView)
		.InitialCategoryFilter(EContentBrowserItemCategoryFilter::IncludeAssets) // TODO: Allow this to be wholesale overridden via the picker config
		.OnItemSelectionChanged(this, &SPathPicker::OnItemSelectionChanged) // TODO: Allow this to be wholesale overridden via the picker config
		.OnGetItemContextMenu(this, &SPathPicker::GetItemContextMenu) // TODO: Allow this to be wholesale overridden via the picker config
		.FocusSearchBoxWhenOpened(InArgs._PathPickerConfig.bFocusSearchBoxWhenOpened)
		.AllowContextMenu(InArgs._PathPickerConfig.bAllowContextMenu)
		.AllowReadOnlyFolders(InArgs._PathPickerConfig.bAllowReadOnlyFolders)
		.SelectionMode(ESelectionMode::Single)
		.CustomFolderPermissionList(InArgs._PathPickerConfig.CustomFolderPermissionList)
		.ShowFavorites(InArgs._PathPickerConfig.bShowFavorites)
		.DefaultPath(InArgs._PathPickerConfig.DefaultPath)
		.CreateDefaultPath(InArgs._PathPickerConfig.bAddDefaultPath)
		.AllowClassesFolder(InArgs._PathPickerConfig.bAllowClassesFolder)
		.CanShowDevelopersFolder(InArgs._PathPickerConfig.bCanShowDevelopersFolder)
		.ForceShowEngineContent(InArgs._PathPickerConfig.bForceShowEngineContent)
		.ForceShowPluginContent(InArgs._PathPickerConfig.bForceShowPluginContent)
		.ShowViewOptions(InArgs._PathPickerConfig.bShowViewOptions)
	];
	// clang-format on

	const FString& DefaultPath = InArgs._PathPickerConfig.DefaultPath;
	if (!DefaultPath.IsEmpty() && PathViewPtr->InternalPathPassesBlockLists(*DefaultPath))
	{
		const FName VirtualPath = IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual(*DefaultPath);
		// Path is created by SPathView::Construct if necessary
		if (bOnPathSelectedPassesVirtualPaths)
		{
			LastSelectedPath = VirtualPath.ToString();
		}
		else
		{
			LastSelectedPath = DefaultPath;
		}

		if (InArgs._PathPickerConfig.bNotifyDefaultPathSelected)
		{
			OnPathSelected.ExecuteIfBound(LastSelectedPath);
		}
	}
}

void SPathPicker::OnItemSelectionChanged(const FContentBrowserItem& SelectedItem, ESelectInfo::Type SelectInfo)
{
	FString NewSelectedPath;
	FName SelectedPackagePath;
	if (SelectedItem.IsFolder())
	{
		if (bOnPathSelectedPassesVirtualPaths)
		{
			NewSelectedPath = SelectedItem.GetVirtualPath().ToString();
		}
		else if (SelectedItem.Legacy_TryGetPackagePath(SelectedPackagePath))
		{
			NewSelectedPath = SelectedPackagePath.ToString();
		}
	}

	if (!NewSelectedPath.IsEmpty() && NewSelectedPath != LastSelectedPath)
	{
		LastSelectedPath = NewSelectedPath;
		OnPathSelected.ExecuteIfBound(NewSelectedPath);
	}
}

TSharedPtr<SWidget> SPathPicker::GetItemContextMenu(TArrayView<const FContentBrowserItem> SelectedItems)
{
	if (SelectedItems.IsEmpty())
	{
		return nullptr;
	}

	FOnCreateNewFolder OnCreateNewFolder = FOnCreateNewFolder::CreateSP(PathViewPtr.Get(), &SPathView::NewFolderItemRequested);

	if (OnGetFolderContextMenu.IsBound())
	{
		return OnGetFolderContextMenu.Execute(GetInternalPaths(SelectedItems), OnGetPathContextMenuExtender, OnCreateNewFolder);
	}

	return GetFolderContextMenu(SelectedItems, OnGetPathContextMenuExtender, OnCreateNewFolder);
}

TSharedPtr<SWidget> SPathPicker::GetFolderContextMenu(TArrayView<const FContentBrowserItem> SelectedItems, FContentBrowserMenuExtender_SelectedPaths InMenuExtender, FOnCreateNewFolder InOnCreateNewFolder)
{
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

	TSharedPtr<FExtender> Extender;
	if (InMenuExtender.IsBound())
	{
		// Code using extenders here currently expects internal paths
		Extender = InMenuExtender.Execute(GetInternalPaths(SelectedItems));
	}

	const bool bInShouldCloseWindowAfterSelection = true;
	const bool bCloseSelfOnly = true;
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterSelection, nullptr, Extender, bCloseSelfOnly);

	// We can only create folders when we have a single path selected
	const bool bCanCreateNewFolder = SelectedItems.Num() == 1 && ContentBrowserData->CanCreateFolder(SelectedItems[0].GetVirtualPath(), nullptr);

	FText NewFolderToolTip;
	if(SelectedItems.Num() == 1)
	{
		if(bCanCreateNewFolder)
		{
			NewFolderToolTip = FText::Format(
				LOCTEXT("NewFolderTooltip_CreateIn", "Create a new folder in {0}."),
				IContentBrowserDataModule::Get().GetSubsystem()->ConvertVirtualPathToDisplay(SelectedItems[0]));
		}
		else
		{
			NewFolderToolTip = FText::Format(
				LOCTEXT("NewFolderTooltip_InvalidPath", "Cannot create new folders in {0}."),
				IContentBrowserDataModule::Get().GetSubsystem()->ConvertVirtualPathToDisplay(SelectedItems[0]));
		}
	}
	else
	{
		NewFolderToolTip = LOCTEXT("NewFolderTooltip_InvalidNumberOfPaths", "Can only create folders when there is a single path selected.");
	}

	// New Folder
	MenuBuilder.AddMenuEntry(
		LOCTEXT("NewFolder", "New Folder"),
		NewFolderToolTip,
		FSlateIcon(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetStyleSetName(), "ContentBrowser.NewFolderIcon"),
		FUIAction(
			FExecuteAction::CreateSP(this, &SPathPicker::CreateNewFolder, SelectedItems.Num() > 0 ? SelectedItems[0].GetVirtualPath().ToString() : FString(), InOnCreateNewFolder),
			FCanExecuteAction::CreateLambda( [bCanCreateNewFolder] { return bCanCreateNewFolder; } )
			),
		"FolderContext"
		);

	return MenuBuilder.MakeWidget();
}

TArray<FString> SPathPicker::GetInternalPaths(TArrayView<const FContentBrowserItem> SelectedItems)
{
	TArray<FString> InternalPaths;
	InternalPaths.Reserve(SelectedItems.Num());
	for (const FContentBrowserItem& SelectedItem : SelectedItems)
	{
		const FName InternalPath = SelectedItem.GetInternalPath();
		if (!InternalPath.IsNone())
		{
			InternalPaths.Add(InternalPath.ToString());
		}
	}
	return InternalPaths;
}

void SPathPicker::ExecuteRenameFolder()
{
	if (PathViewPtr.IsValid())
	{
		const TArray<FContentBrowserItem> SelectedItems = PathViewPtr->GetSelectedFolderItems();
		if (SelectedItems.Num() == 1)
		{
			PathViewPtr->RenameFolderItem(SelectedItems[0]);
		}
	}
}

void SPathPicker::ExecuteAddFolder()
{
	if (PathViewPtr.IsValid())
	{
		const TArray<FString> SelectedItems = PathViewPtr->GetSelectedPaths();
		if (SelectedItems.Num() == 1)
		{
			FOnCreateNewFolder OnCreateNewFolder = FOnCreateNewFolder::CreateSP(PathViewPtr.Get(), &SPathView::NewFolderItemRequested);
			CreateNewFolder(SelectedItems[0], OnCreateNewFolder);
		}
	}
}

void SPathPicker::CreateNewFolder(FString FolderPath, FOnCreateNewFolder InOnCreateNewFolder)
{
	const FText DefaultFolderBaseName = LOCTEXT("DefaultFolderName", "NewFolder");
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

	// Create a valid base name for this folder
	FString DefaultFolderName = DefaultFolderBaseName.ToString();
	int32 NewFolderPostfix = 0;
	FName CombinedPathName;
	for (;;)
	{
		FString CombinedPathNameStr = FolderPath / DefaultFolderName;
		if (NewFolderPostfix > 0)
		{
			CombinedPathNameStr.AppendInt(NewFolderPostfix);
		}
		++NewFolderPostfix;

		CombinedPathName = *CombinedPathNameStr;

		const FContentBrowserItem ExistingFolder = ContentBrowserData->GetItemAtPath(CombinedPathName, EContentBrowserItemTypeFilter::IncludeFolders);
		if (!ExistingFolder.IsValid())
		{
			break;
		}
	}

	const FContentBrowserItemTemporaryContext NewFolderItem = ContentBrowserData->CreateFolder(CombinedPathName);
	if (NewFolderItem.IsValid())
	{
		InOnCreateNewFolder.ExecuteIfBound(NewFolderItem);
	}
}

void SPathPicker::RefreshPathView()
{
	PathViewPtr->Populate(true);
}

void SPathPicker::SetPaths(const TArray<FString>& NewPaths)
{
	PathViewPtr->SetSelectedPaths(NewPaths);
}

TArray<FString> SPathPicker::GetPaths() const
{
	return PathViewPtr->GetSelectedPaths();
}

const TSharedPtr<SPathView>& SPathPicker::GetPathView() const
{
	return PathViewPtr;
}

void SPathPicker::SyncToItems(TArrayView<const FContentBrowserItem> ItemsToSync, const bool bAllowImplicitSync)
{
	if (PathViewPtr->DisablePluginPathFiltersThatHideItems(ItemsToSync))
	{
		PathViewPtr->Populate();
	}
	PathViewPtr->SyncToItems(ItemsToSync, bAllowImplicitSync);
}

#undef LOCTEXT_NAMESPACE
