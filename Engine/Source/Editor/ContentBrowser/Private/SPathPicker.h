// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDelegates.h"
#include "IContentBrowserSingleton.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SPathView;
class SWidget;
struct FContentBrowserItem;

/**
 * A sources view designed for path picking
 */
class SPathPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPathPicker ){}

		/** A struct containing details about how the path picker should behave */
		SLATE_ARGUMENT(FPathPickerConfig, PathPickerConfig)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

	/** Sets the selected paths in this picker */
	void SetPaths(const TArray<FString>& NewPaths);

	/** Return the selected paths in this picker */
	TArray<FString> GetPaths() const;

	/** Return the associated SPathView */
	const TSharedPtr<SPathView>& GetPathView() const;
	
	void SyncToItems(TArrayView<const FContentBrowserItem> ItemsToSync, const bool bAllowImplicitSync);

	/** Handler for creating a new folder in the path picker */
	void CreateNewFolder(FString FolderPath, FOnCreateNewFolder InOnCreateNewFolder);

	/** Rename the selected, will just use the first folder selected*/
	void ExecuteRenameFolder();

	/** Add a folder to the first selected folder*/
	void ExecuteAddFolder();

	/** Refresh the path view*/
	void RefreshPathView();

private:

	/** Handle for when selection changes */
	void OnItemSelectionChanged(const FContentBrowserItem& SelectedItem, ESelectInfo::Type SelectInfo);

	/** Handler for the context menu for folder items */
	TSharedPtr<SWidget> GetItemContextMenu(TArrayView<const FContentBrowserItem> SelectedItems);
	TSharedPtr<SWidget> GetFolderContextMenu(TArrayView<const FContentBrowserItem> SelectedItems, FContentBrowserMenuExtender_SelectedPaths InMenuExtender, FOnCreateNewFolder InOnCreateNewFolder);
	static TArray<FString> GetInternalPaths(TArrayView<const FContentBrowserItem> SelectedItems);

private:

	/** The path view in this picker */
	TSharedPtr<SPathView> PathViewPtr;

	/** Delegate to invoke when selection changes. */
	FOnPathSelected OnPathSelected;

	/** Delegate to invoke when a context menu for a folder is opening. */
	FOnGetFolderContextMenu OnGetFolderContextMenu;

	/** The delegate that fires when a path is right clicked and a context menu is requested */
	FContentBrowserMenuExtender_SelectedPaths OnGetPathContextMenuExtender;

	/** If true, passes virtual paths to OnPathSelected instead of internal asset paths */
	bool bOnPathSelectedPassesVirtualPaths;

	/** Path currently selected in this picker. Used to suppress redundant OnPathSelected events. */
	FString LastSelectedPath;
};
