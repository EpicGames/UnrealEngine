// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPathView.h"

#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Algo/Copy.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetToolsModule.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/StringView.h"
#include "ContentBrowserCommands.h"
#include "ContentBrowserConfig.h"
#include "ContentBrowserDataDragDropOp.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserItemData.h"
#include "ContentBrowserLog.h"
#include "ContentBrowserMenuUtils.h"
#include "ContentBrowserModule.h"
#include "ContentBrowserPathViewMenuContexts.h"
#include "ContentBrowserPluginFilters.h"
#include "ContentBrowserSingleton.h"
#include "ContentBrowserStyle.h"
#include "ContentBrowserUtils.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "DragDropHandler.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "HAL/PlatformTime.h"
#include "HistoryManager.h"
#include "IAssetTools.h"
#include "IContentBrowserDataModule.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Interfaces/IPluginManager.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/SlateRect.h"
#include "Layout/WidgetPath.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/CString.h"
#include "Misc/ComparisonUtility.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FilterCollection.h"
#include "Misc/NamePermissionList.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"
#include "Modules/ModuleManager.h"
#include "PathViewTypes.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Settings/ContentBrowserSettings.h"
#include "SlotBase.h"
#include "SourcesSearch.h"
#include "SourcesViewWidgets.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "Framework/Commands/GenericCommands.h"
#include "Trace/Detail/Channel.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/STableRow.h"

class FDragDropOperation;
class SWidget;
struct FAssetData;
struct FGeometry;

DEFINE_LOG_CATEGORY_STATIC(LogPathView, Log, Log);

#define LOCTEXT_NAMESPACE "ContentBrowser"

namespace UE::PathView
{
TArray<TWeakPtr<SPathView>> AllPathViews;

FAutoConsoleCommand RepopulateAllPathViewsCommand(TEXT("PathView.Repopulate"),
	TEXT("Repopulate all path views to expose bugs with caching/data updates"),
	FConsoleCommandDelegate::CreateLambda([]() {
		for (TWeakPtr<SPathView> WeakView : AllPathViews)
		{
			if (TSharedPtr<SPathView> View = WeakView.Pin())
			{
				View->Populate();
			}
		}
	}));

TSharedRef<FTreeItem> CreateOrReuseNode(FContentBrowserItemData&& InData,
	TMap<FName, TSharedPtr<FTreeItem>>* OldItemsByInvariantPath)
{
	if (OldItemsByInvariantPath)
	{
		TSharedPtr<FTreeItem> ExistingItem;
		// Reolve old value so we don't pick it out again when looking at another item from a different source
		if (OldItemsByInvariantPath->RemoveAndCopyValue(InData.GetInvariantPath(), ExistingItem)
			&& ExistingItem.IsValid())
		{
			ExistingItem->RemoveAllChildren();
			ExistingItem->SetItemData(FContentBrowserItem(MoveTemp(InData)));
			return ExistingItem.ToSharedRef();
		}
	}
	return MakeShared<FTreeItem>(MoveTemp(InData));
}

void DefaultSort(TArray<TSharedPtr<FTreeItem>>& InChildren)
{
	if (InChildren.Num() < 2)
	{
		return;
	}

	static const FString ClassesPrefix = TEXT("Classes_");

	struct FItemSortInfo
	{
		// Name to display
		FString FolderName;
		float Priority;
		int32 SpecialDefaultFolderPriority;
		bool bIsClassesFolder;
		TSharedPtr<FTreeItem> TreeItem;
		// Name to use when comparing "MyPlugin" vs "Classes_MyPlugin", looking up a plugin by name and other situations
		FName ItemNameWithoutClassesPrefix;
	};

	TArray<FItemSortInfo> SortInfoArray;
	SortInfoArray.Reserve(InChildren.Num());

	const TArray<FName>& SpecialSortFolders =
		IContentBrowserDataModule::Get().GetSubsystem()->GetPathViewSpecialSortFolders();

	// Generate information needed to perform sort
	for (TSharedPtr<FTreeItem>& It : InChildren)
	{
		FItemSortInfo& SortInfo = SortInfoArray.AddDefaulted_GetRef();
		SortInfo.TreeItem = It;

		const FName InvariantPathFName = It->GetItem().GetInvariantPath();
		FNameBuilder InvariantPathBuilder(InvariantPathFName);
		const FStringView InvariantPath(InvariantPathBuilder);

		bool bIsRootInvariantFolder = false;
		if (InvariantPath.Len() > 1)
		{
			FStringView RootInvariantFolder(InvariantPath);
			RootInvariantFolder.RightChopInline(1);
			int32 SecondSlashIndex = INDEX_NONE;
			bIsRootInvariantFolder = !RootInvariantFolder.FindChar(TEXT('/'), SecondSlashIndex);
		}

		SortInfo.FolderName = It->GetItem().GetDisplayName().ToString();

		SortInfo.bIsClassesFolder = false;
		if (bIsRootInvariantFolder)
		{
			FNameBuilder ItemNameBuilder(It->GetItem().GetItemName());
			const FStringView ItemNameView(ItemNameBuilder);
			if (ItemNameView.StartsWith(ClassesPrefix))
			{
				SortInfo.bIsClassesFolder = true;
				SortInfo.ItemNameWithoutClassesPrefix = FName(ItemNameView.RightChop(ClassesPrefix.Len()));
			}

			if (SortInfo.FolderName.StartsWith(ClassesPrefix))
			{
				SortInfo.bIsClassesFolder = true;
				SortInfo.FolderName.RightChopInline(ClassesPrefix.Len(), EAllowShrinking::No);
			}
		}

		if (SortInfo.ItemNameWithoutClassesPrefix.IsNone())
		{
			SortInfo.ItemNameWithoutClassesPrefix = It->GetItem().GetItemName();
		}

		if (SortInfo.bIsClassesFolder)
		{
			// Sort using a path without "Classes_" prefix
			FStringView InvariantWithoutClassesPrefix(InvariantPath);
			InvariantWithoutClassesPrefix.RightChopInline(1);
			if (InvariantWithoutClassesPrefix.StartsWith(ClassesPrefix))
			{
				InvariantWithoutClassesPrefix.RightChopInline(ClassesPrefix.Len());
				FNameBuilder Builder;
				Builder.Append(TEXT("/"));
				Builder.Append(InvariantWithoutClassesPrefix);
				SortInfo.SpecialDefaultFolderPriority = SpecialSortFolders.IndexOfByKey(FName(Builder));
			}
			else
			{
				SortInfo.SpecialDefaultFolderPriority = SpecialSortFolders.IndexOfByKey(InvariantPathFName);
			}
		}
		else
		{
			SortInfo.SpecialDefaultFolderPriority = SpecialSortFolders.IndexOfByKey(InvariantPathFName);
		}

		if (bIsRootInvariantFolder)
		{
			if (SortInfo.SpecialDefaultFolderPriority == INDEX_NONE)
			{
				SortInfo.Priority = FContentBrowserSingleton::Get()
										.GetPluginSettings(SortInfo.ItemNameWithoutClassesPrefix)
										.RootFolderSortPriority;
			}
			else
			{
				SortInfo.Priority = 1.f;
			}
		}
		else
		{
			if (SortInfo.SpecialDefaultFolderPriority != INDEX_NONE)
			{
				SortInfo.Priority = 1.f;
			}
			else
			{
				SortInfo.Priority = 0.f;
			}
		}
	}

	// Perform sort
	SortInfoArray.Sort([](const FItemSortInfo& SortInfoA, const FItemSortInfo& SortInfoB) -> bool {
		if (SortInfoA.Priority != SortInfoB.Priority)
		{
			// Not the same priority, use priority to sort
			return SortInfoA.Priority > SortInfoB.Priority;
		}
		else if (SortInfoA.SpecialDefaultFolderPriority != SortInfoB.SpecialDefaultFolderPriority)
		{
			// Special folders use the index to sort. Non special folders are all set to 0.
			return SortInfoA.SpecialDefaultFolderPriority < SortInfoB.SpecialDefaultFolderPriority;
		}
		else
		{
			// If either is a class folder and names without classes prefix are same
			if ((SortInfoA.bIsClassesFolder != SortInfoB.bIsClassesFolder)
				&& (SortInfoA.ItemNameWithoutClassesPrefix == SortInfoB.ItemNameWithoutClassesPrefix))
			{
				return !SortInfoA.bIsClassesFolder;
			}

			// Two non special folders of the same priority, sort alphabetically
			const int32 CompareResult =
				UE::ComparisonUtility::CompareWithNumericSuffix(SortInfoA.FolderName, SortInfoB.FolderName);
			if (CompareResult != 0)
			{
				return CompareResult < 0;
			}
			else
			{
				// Classes folders have the same name so sort them adjacent but under non-classes
				return !SortInfoA.bIsClassesFolder;
			}
		}
	});

	// Replace with sorted array
	TArray<TSharedPtr<FTreeItem>> NewList;
	NewList.Reserve(SortInfoArray.Num());
	for (const FItemSortInfo& It : SortInfoArray)
	{
		NewList.Add(It.TreeItem);
	}
	InChildren = MoveTemp(NewList);
}

} // namespace UE::PathView

// Struct to factor out path view data fetching/filtering as a precursor to being able to bind this data to the view
// instead of fetching it internally
struct FPathViewData
{
public:
	FPathViewData(FName InContentBrowserName, bool InFlat)
		: Version(0)
		, OwningContentBrowserName(InContentBrowserName)
		, bFlat(InFlat)
		, FolderPathTextFilter(decltype(FolderPathTextFilter)::FItemToStringArray::CreateStatic(
			  [](FStringView Input, TArray<FString>& Out) { Out.Emplace(FString(Input)); }))
	{
	}

	~FPathViewData() { }

	uint64 GetVersion()
	{
		return Version;
	}

	// Return an array that can be bound to a tree view widget for the current visible set of root items
	TArray<TSharedPtr<FTreeItem>>* GetVisibleRootItems()
	{
		return &VisibleRootItems;
	}

	TTextFilter<FStringView>& GetFolderPathTextFilter()
	{
		return FolderPathTextFilter;
	}

	// Fetch all data from the content browser data backend and transform it into the tree data
	void PopulateFullFolderTree(const FContentBrowserDataCompiledFilter& InFilter);
	// Fetch favorite folders from config, filter them against the content browser data filter
	// bFlat parameter adds all items at the root of the tree and not create parents
	void PopulateWithFavorites(const FContentBrowserDataCompiledFilter& InFilter);
	// Apply the current text filter to everything in the tree
	void FilterFullFolderTree();
	// Clear the filter state of all items in the tree
	void ClearItemFilterState();
	// Sort the roots of the tree
	void SortRootItems();
	// Add an item to the tree by its virtual path, reusing an old object if possible to ensure persistence of
	// selection/expansion for the widget the items are bound to
	TSharedRef<FTreeItem> AddFolderItem(FContentBrowserItemData&& InItemData);

	// Remove the given item from the tree whether it's a root or child.
	void RemoveFolderItem(const TSharedRef<FTreeItem>& InItem);

	// Find an item with the exact virtual path
	TSharedPtr<FTreeItem> FindTreeItem(FName InVirtualPath, bool bVisibleOnly = false);
	// Search the tree for the item furthest from the root that matches the given path, if any
	// Searches all items, not just visible ones according to the current text filter
	TSharedPtr<FTreeItem> FindBestItemForPath(FStringView InVirtualPath);

	void ProcessVirtualPaths(TConstArrayView<FName> InVirtualPaths, TFunction<FContentBrowserDataCompiledFilter()> CompiledFilterGetter);
	void ProcessVirtualPaths(TConstArrayView<FName> InVirtualPaths, TFunction<FContentBrowserDataCompiledFilter()> CompiledFilterGetter, TSet <TSharedRef<FTreeItem>>& OutItems);

	// Apply new/modified/removed data notifications to the tree
	// bFlat parameter adds all items at the root of the tree as in the favorites tree
	void ProcessDataUpdates(TConstArrayView<FContentBrowserItemDataUpdate> InUpdatedItems,
		const FContentBrowserDataCompiledFilter& InFilter);

protected:
	TSharedRef<FTreeItem> AddFolderItemInternal(FContentBrowserItemData&& InItemData,
		TMap<FName, TSharedPtr<FTreeItem>>* OldItemsByInvariantPath);

	// Remove the given item data from the tree - if this results in an item having no data from any sources, the tree
	// item is removed. If the item was removed and had a parent, the parent is returned.
	TSharedPtr<FTreeItem> TryRemoveFolderItemInternal(const FContentBrowserItemData& InItem);
	TSharedPtr<FTreeItem> TryRemoveFolderItemInternal(const FContentBrowserMinimalItemData& InKey);

	bool PassesTextFilter(const TSharedPtr<FTreeItem>& InItem);

	struct FEmptyFolderFilter
	{
		FContentBrowserFolderContentsFilter FolderFilter;
		EContentBrowserIsFolderVisibleFlags FolderFlags;
	};
	FEmptyFolderFilter GetEmptyFolderFilter(const FContentBrowserDataCompiledFilter& CompiledDataFilter) const;

	// Incremented to trigger tree rebuild from changes to the tree contents
	uint64 Version;
	// Items with no parent
	TArray<TSharedPtr<FTreeItem>> RootItems;
	// Items with no parent
	TArray<TSharedPtr<FTreeItem>> VisibleRootItems;
	// Mapping of full virtual path such as '/All/Game/Maps/Arena' to items
	TMap<FName, TSharedPtr<FTreeItem>> VirtualPathToItem;
	// Mapping of path that doesn't change based on display settings (e.g. '/MyPlugin/MyAsset') to item
	// Used to reuse node objects when changing path view settings and rebuilding the tree 
	TMap<FName, TSharedPtr<FTreeItem>> InvariantPathToItem;

	// Used for retrieving saved settings per content browser instance
	FName OwningContentBrowserName;

	// If true, parent items are not created and all items are added as roots.
	bool bFlat;

	TTextFilter<FStringView> FolderPathTextFilter;
};

FPathViewData::FEmptyFolderFilter FPathViewData::GetEmptyFolderFilter(const FContentBrowserDataCompiledFilter& CompiledDataFilter) const
{
	const UContentBrowserSettings* ContentBrowserSettings = GetDefault<UContentBrowserSettings>();
	bool bDisplayEmpty = ContentBrowserSettings->DisplayEmptyFolders;
	// check to see if we have an instance config that overrides the default in UContentBrowserSettings
	if (FContentBrowserInstanceConfig* EditorConfig =
			ContentBrowserUtils::GetContentBrowserConfig(OwningContentBrowserName))
	{
		bDisplayEmpty = EditorConfig->bShowEmptyFolders;
	}

	FContentBrowserFolderContentsFilter FolderFilter;
	if (bDisplayEmpty)
	{
		UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
		FolderFilter.HideFolderIfEmptyFilter = ContentBrowserData->CreateHideFolderIfEmptyFilter();
	}
	else
	{
		FolderFilter.ItemCategoryFilter = CompiledDataFilter.ItemCategoryFilter;
	}
	EContentBrowserIsFolderVisibleFlags FolderFlags = ContentBrowserUtils::GetIsFolderVisibleFlags(bDisplayEmpty);
	return { MoveTemp(FolderFilter), FolderFlags };
}

void FPathViewData::PopulateFullFolderTree(const FContentBrowserDataCompiledFilter& CompiledDataFilter)
{
	TMap<FName, TSharedPtr<FTreeItem>> OldItemsByInvariantPath = MoveTemp(InvariantPathToItem);
	RootItems.Reset();
	VisibleRootItems.Reset();
	InvariantPathToItem.Reset();
	VirtualPathToItem.Reset();

	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	FEmptyFolderFilter EmptyFilter = GetEmptyFolderFilter(CompiledDataFilter);
	TArray<TSharedPtr<FTreeItem>> ItemsCreated;
	ContentBrowserData->EnumerateItemsMatchingFilter(CompiledDataFilter,
		[this,
			CompiledDataFilter,
			EmptyFilter,
			ContentBrowserData,
			&OldItemsByInvariantPath](FContentBrowserItemData&& InItemData) {
			UContentBrowserDataSource* Source = InItemData.GetOwnerDataSource();
			if (Source && !Source->IsFolderVisible(InItemData.GetVirtualPath(), EmptyFilter.FolderFlags, EmptyFilter.FolderFilter))
			{
				UE_LOGF(LogPathView, VeryVerbose, "[%ls] Populate: skipping folder %ls:%ls that fails current pre-text filtering",
				 	*WriteToString<256>(OwningContentBrowserName),
					*WriteToString<256>(Source->GetFName()),
					*WriteToString<256>(InItemData.GetVirtualPath()));
				return true; // continue enumerating
			}

			UE_LOGF(LogPathView, VeryVerbose, "[%ls] Populate: adding folder %ls:%ls",
				*WriteToString<256>(OwningContentBrowserName),
				*WriteToString<256>(Source->GetFName()),
				*WriteToString<256>(InItemData.GetVirtualPath()));

			AddFolderItemInternal(MoveTemp(InItemData), &OldItemsByInvariantPath);
			return true;
		});
	VisibleRootItems = RootItems;
	++Version;
}

void FPathViewData::PopulateWithFavorites(const FContentBrowserDataCompiledFilter& CompiledDataFilter)
{
	// Clear all root items and clear selection
	TMap<FName, TSharedPtr<FTreeItem>> OldItemsByInvariantPath = MoveTemp(InvariantPathToItem);
	RootItems.Reset();
	VisibleRootItems.Reset();
	InvariantPathToItem.Reset();
	VirtualPathToItem.Reset();

	const TArray<FString>& FavoritePaths = ContentBrowserUtils::GetFavoriteFolders();
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	FEmptyFolderFilter EmptyFilter = GetEmptyFolderFilter(CompiledDataFilter);
	for (const FString& InvariantPath : FavoritePaths)
	{
		FName VirtualPath;
		IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual(InvariantPath, VirtualPath);
		const FString Path = VirtualPath.ToString();

		ContentBrowserData->EnumerateItemsAtPath(*Path,
			CompiledDataFilter.ItemTypeFilter,
			[this, &CompiledDataFilter, EmptyFilter, &OldItemsByInvariantPath](FContentBrowserItemData&& InItemData) {
				UContentBrowserDataSource* ItemDataSource = InItemData.GetOwnerDataSource();
				if (!ItemDataSource->IsFolderVisible(InItemData.GetVirtualPath(), EmptyFilter.FolderFlags, EmptyFilter.FolderFilter))
				{
					UE_LOGF(LogPathView,
						VeryVerbose,
						"Hiding folder %ls that fails current pre-text filtering",
						*WriteToString<256>(InItemData.GetVirtualPath()));
					return true; // continue enumerating
				}
				ItemDataSource->ConvertItemForFilter(InItemData, CompiledDataFilter);
				if (ItemDataSource->DoesItemPassFilter(InItemData, CompiledDataFilter))
				{
					AddFolderItemInternal(MoveTemp(InItemData), &OldItemsByInvariantPath);
				}

				return true;
			});
	}
	++Version;
}

void FPathViewData::ProcessDataUpdates(TConstArrayView<FContentBrowserItemDataUpdate> InUpdatedItems,
	const FContentBrowserDataCompiledFilter& CompiledDataFilter)
{
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	FEmptyFolderFilter EmptyFilter = GetEmptyFolderFilter(CompiledDataFilter);
	auto DoesItemPassFilter = [this, EmptyFilter, ContentBrowserData, &CompiledDataFilter](
								  const FContentBrowserItemData& InItemData) {
		UContentBrowserDataSource* ItemDataSource = InItemData.GetOwnerDataSource();
		if (!ItemDataSource->DoesItemPassFilter(InItemData, CompiledDataFilter))
		{
			UE_LOGF(LogPathView, VeryVerbose, "[%ls] Fails compiled data filter",
				*WriteToString<256>(OwningContentBrowserName));
			return false;
		}

		if (!ContentBrowserData->IsFolderVisible(InItemData.GetVirtualPath(), EmptyFilter.FolderFlags, EmptyFilter.FolderFilter))
		{
			UE_LOGF(LogPathView, VeryVerbose, "[%ls] Fails folder visibility filter",
				*WriteToString<256>(OwningContentBrowserName));
			return false;
		}

		return true;
	};

	TArray<TSharedRef<FTreeItem>> NewItems;
	TArray<TSharedRef<FTreeItem>> ModifiedParents; // Parents who need their bHasVisibleDescendants updated
	for (const FContentBrowserItemDataUpdate& ItemDataUpdate : InUpdatedItems)
	{
		const FContentBrowserItemData& ItemDataRef = ItemDataUpdate.GetItemData();
		if (!ItemDataRef.IsFolder())
		{
			continue;
		}

		FContentBrowserItemData ItemData = ItemDataRef;
		ItemData.GetOwnerDataSource()->ConvertItemForFilter(ItemData, CompiledDataFilter);

		switch (ItemDataUpdate.GetUpdateType())
		{
			case EContentBrowserItemUpdateType::Added:
				UE_LOGF(LogPathView, VeryVerbose, "[%ls] Added item %ls:%ls", *WriteToString<256>(OwningContentBrowserName), 
					*WriteToString<256>(ItemData.GetOwnerDataSource()->GetFName()),
					*WriteToString<256>(ItemData.GetVirtualPath()));
				if (DoesItemPassFilter(ItemData))
				{
					NewItems.Emplace(AddFolderItemInternal(MoveTemp(ItemData), nullptr));
				}
				break;

			case EContentBrowserItemUpdateType::Modified:
				UE_LOGF(LogPathView, VeryVerbose, "[%ls] Modified item %ls:%ls", *WriteToString<256>(OwningContentBrowserName), 
					*WriteToString<256>(ItemData.GetOwnerDataSource()->GetFName()),
					*WriteToString<256>(ItemData.GetVirtualPath()));
				if (DoesItemPassFilter(ItemData))
				{
					NewItems.Emplace(AddFolderItemInternal(MoveTemp(ItemData), nullptr));
				}
				else
				{
					TSharedPtr<FTreeItem> Parent = TryRemoveFolderItemInternal(ItemData);
					if (Parent.IsValid())
					{
						ModifiedParents.Emplace(Parent.ToSharedRef());
					}
				}
				break;

			case EContentBrowserItemUpdateType::Moved:
			{
				UE_LOGF(LogPathView, VeryVerbose, "[%ls] Moved item %ls:%ls->%ls", *WriteToString<256>(OwningContentBrowserName), 
					*WriteToString<256>(ItemData.GetOwnerDataSource()->GetFName()),
					*WriteToString<256>(ItemDataUpdate.GetPreviousVirtualPath()),
					*WriteToString<256>(ItemData.GetVirtualPath()));
				const FContentBrowserMinimalItemData OldItemKey(ItemData.GetItemType(), ItemDataUpdate.GetPreviousVirtualPath(), ItemData.GetOwnerDataSource());
				TSharedPtr<FTreeItem> Parent = TryRemoveFolderItemInternal(OldItemKey);
				if (DoesItemPassFilter(ItemData))
				{
					NewItems.Emplace(AddFolderItemInternal(MoveTemp(ItemData), nullptr));
				}
				else if (Parent.IsValid())
				{
					ModifiedParents.Emplace(Parent.ToSharedRef());
				}
			}
			break;

			case EContentBrowserItemUpdateType::Removed:
				UE_LOGF(LogPathView, VeryVerbose, "[%ls] Removed item %ls:%ls", *WriteToString<256>(OwningContentBrowserName), 
					*WriteToString<256>(ItemData.GetOwnerDataSource()->GetFName()),
					*WriteToString<256>(ItemData.GetVirtualPath()));
				TryRemoveFolderItemInternal(ItemData);
				break;

			case EContentBrowserItemUpdateType::Unmounted:
				UE_LOGF(LogPathView, VeryVerbose, "[%ls] Unmounted item %ls:%ls", *WriteToString<256>(OwningContentBrowserName),
					*WriteToString<256>(ItemData.GetOwnerDataSource()->GetFName()),
					*WriteToString<256>(ItemData.GetVirtualPath()));
				TryRemoveFolderItemInternal(ItemData);
				break;

			default:
				checkf(false, TEXT("Unexpected EContentBrowserItemUpdateType!"));
				break;
		}
	}

	++Version;
	// Determine visibility for new items and their parents
	if (!FolderPathTextFilter.GetRawFilterText().IsEmpty())
	{
		// Clear visible descendents flag on modified parents because we will reset it
		for (const TSharedRef<FTreeItem>& Parent : ModifiedParents)
		{
			Parent->SetHasVisibleDescendants(false);
		}

		for (const TSharedRef<FTreeItem>& Item : NewItems)
		{
			bool bVisible = PassesTextFilter(Item);
			Item->SetVisible(bVisible);
			if (bVisible)
			{
				// Propagate to parents
				for (TSharedPtr<FTreeItem> Parent = Item->GetParent(); Parent.IsValid() && !Parent->IsVisible();
					 Parent = Parent->GetParent())
				{
					Parent->SetHasVisibleDescendants(true);
				}
			}
		}

		// Sort modified parents so if items are related, we visit the items furthest from the root first
		Algo::Sort(ModifiedParents,
			[](const TSharedRef<FTreeItem>& A, const TSharedRef<FTreeItem>& B) { return A->IsChildOf(*B); });
		for (const TSharedRef<FTreeItem>& Parent : ModifiedParents)
		{
			// May have already figured this out when dealing with directly modified items
			if (!Parent->GetHasVisibleDescendants())
			{
				bool bVisibleChildren = Algo::AnyOf(Parent->GetChildren(),
					[](const TSharedPtr<FTreeItem>& Child) { return Child.IsValid() && Child->IsVisible(); });
				Parent->SetHasVisibleDescendants(bVisibleChildren);
			}
		}
	}
	else 
	{
		// If filtering is not active and we created some new root items, we need them to be visible
		VisibleRootItems = RootItems;
	}
}

bool FPathViewData::PassesTextFilter(const TSharedPtr<FTreeItem>& InItem)
{
	return FolderPathTextFilter.PassesFilter(WriteToString<256>(InItem->GetItem().GetVirtualPath()))
		// TODO: this will not match a string like LocName1/LocName2 when both parent and child are localized
		|| FolderPathTextFilter.PassesFilter(FStringView(InItem->GetItem().GetDisplayName().ToString()));
}

void FPathViewData::ClearItemFilterState()
{
	for (const TPair<FName, TSharedPtr<FTreeItem>>& Pair : VirtualPathToItem)
	{
		FName VirtualPath = Pair.Key;
		Pair.Value->SetVisible(true);
		Pair.Value->SetHasVisibleDescendants(true);
	}
	VisibleRootItems = RootItems;
	++Version;
}

void FPathViewData::FilterFullFolderTree()
{
	for (const TPair<FName, TSharedPtr<FTreeItem>>& Pair : VirtualPathToItem)
	{
		FName VirtualPath = Pair.Key;
		Pair.Value->SetVisible(PassesTextFilter(Pair.Value));
		Pair.Value->SetHasVisibleDescendants(false);
	}

	// Propagate visibility down to parents
	for (const TPair<FName, TSharedPtr<FTreeItem>>& Pair : VirtualPathToItem)
	{
		if (Pair.Value->IsVisible())
		{
			for (TSharedPtr<FTreeItem> Parent = Pair.Value->GetParent(); Parent.IsValid() && !Parent->IsVisible();
				 Parent = Parent->GetParent())
			{
				Parent->SetHasVisibleDescendants(true);
			}
		}
	}
	VisibleRootItems.Reset();
	Algo::CopyIf(RootItems, VisibleRootItems, UE_PROJECTION_MEMBER(FTreeItem, IsVisible));
	++Version;
}

void FPathViewData::SortRootItems()
{
	UE::PathView::DefaultSort(RootItems);
	UE::PathView::DefaultSort(VisibleRootItems);
}

TSharedPtr<FTreeItem> FPathViewData::FindTreeItem(FName InVirtualPath, bool bVisibleOnly)
{
	if (TSharedPtr<FTreeItem> Found = VirtualPathToItem.FindRef(InVirtualPath))
	{
		if (bVisibleOnly && !Found->IsVisible())
		{
			return {};
		}
		return Found;
	}
	return {};
}

TSharedPtr<FTreeItem> FPathViewData::FindBestItemForPath(FStringView InVirtualPath)
{
	if (bFlat)
	{
		return FindTreeItem(FName(InVirtualPath), false);
	}

	TSharedPtr<FTreeItem> Found;
	FPathViews::IterateAncestors(InVirtualPath, [this, &Found](FStringView Ancestor) {
		FName ItemName{ Ancestor };
		if (TSharedPtr<FTreeItem>* Item = VirtualPathToItem.Find(ItemName))
		{
			Found = *Item;
			return false; // Found the leafmost item matching this path
		}
		return true; // continue
	});
	return Found;
}

void FPathViewData::ProcessVirtualPaths(TConstArrayView<FName> InVirtualPaths, TFunction<FContentBrowserDataCompiledFilter()> CompiledFilterGetter)
{
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

	TArray<FContentBrowserItemDataUpdate> FolderAtPath;
	for (const FName& Path : InVirtualPaths)
	{
		ContentBrowserData->EnumerateItemsAtPath(Path, EContentBrowserItemTypeFilter::IncludeFolders, [&FolderAtPath](FContentBrowserItemData&& InItem) -> bool
		{
			FolderAtPath.Add(FContentBrowserItemDataUpdate::MakeItemAddedUpdate(MoveTemp(InItem)));
			return true;
		});
	}

	if (!FolderAtPath.IsEmpty())
	{
		ProcessDataUpdates(FolderAtPath, CompiledFilterGetter());
	}
}

void FPathViewData::ProcessVirtualPaths(TConstArrayView<FName> InVirtualPaths, TFunction<FContentBrowserDataCompiledFilter()> CompiledFilterGetter, TSet<TSharedRef<FTreeItem>>& OutItems)
{
	ProcessVirtualPaths(InVirtualPaths, MoveTemp(CompiledFilterGetter));

	for (const FName& VirtualPath : InVirtualPaths)
	{
		TSharedPtr<FTreeItem> Item = FindTreeItem(VirtualPath);
		if (Item.IsValid())
		{
			OutItems.Add(Item.ToSharedRef());
		}
	}
}

TSharedRef<FTreeItem> FPathViewData::AddFolderItem(FContentBrowserItemData&& InItemData)
{
	TSharedRef<FTreeItem> NewOrUpdatedItem = AddFolderItemInternal(MoveTemp(InItemData), nullptr);
	++Version;
	return NewOrUpdatedItem;
}

TSharedRef<FTreeItem> FPathViewData::AddFolderItemInternal(FContentBrowserItemData&& InItemData,
	TMap<FName, TSharedPtr<FTreeItem>>* OldItemsByInvariantPath)
{
	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	UContentBrowserDataSource* OriginalDataSource = InItemData.GetOwnerDataSource();
	FName ItemVirtualPath = InItemData.GetVirtualPath();
	TSharedPtr<FTreeItem> LeafItem = VirtualPathToItem.FindRef(ItemVirtualPath);
	if (LeafItem.IsValid())
	{
		UE_LOGF(LogPathView, VeryVerbose, "[%ls] Appending data to existing item %ls:%ls", 
			*WriteToString<256>(OwningContentBrowserName),
			*WriteToString<256>(OriginalDataSource->GetFName()),
			*WriteToString<256>(ItemVirtualPath));

		// Item already existed - duplicate item returned by multiple data sources, merge data and move on.
		// We will have already created all the parent items.
		LeafItem->AppendItemData(InItemData);
		return LeafItem.ToSharedRef();
	}

	UE_LOGF(LogPathView, VeryVerbose, "[%ls] Creating new tree item for %ls:%ls", 
		*WriteToString<256>(OwningContentBrowserName),
		*WriteToString<256>(OriginalDataSource->GetFName()),
		*WriteToString<256>(ItemVirtualPath));

	FName ItemInvariantPath = InItemData.GetInvariantPath();
	TStringBuilder<FName::StringBufferSize> PathBuffer(InPlace, ItemVirtualPath);
	LeafItem = UE::PathView::CreateOrReuseNode(MoveTemp(InItemData), OldItemsByInvariantPath);
	// InItemData is now no longer valid!!

	VirtualPathToItem.Add(ItemVirtualPath, LeafItem);
	InvariantPathToItem.Add(ItemInvariantPath, LeafItem);

	if (bFlat)
	{
		RootItems.Add(LeafItem);
		return LeafItem.ToSharedRef();
	}

	TSharedRef<FTreeItem> PreviousItem = LeafItem.ToSharedRef();

	// Work backwards from the leaf path of the requested item until we encounter an item that already existed
	FPathViews::IterateAncestors(PathBuffer.ToView(),
		[this, &PathBuffer, &PreviousItem, OriginalDataSource, OldItemsByInvariantPath, &ContentBrowserData](FStringView PathView) {
			if (PathView.Len() == PathBuffer.Len())
			{
				// This is the item returned by the data source, we already added it
				return true;
			}
			if (PathView == TEXTVIEW("/"))
			{
				UE_LOGF(LogPathView, VeryVerbose, "[%ls] Adding root item %ls:%.*ls", 
					*WriteToString<256>(OwningContentBrowserName),
					*WriteToString<256>(OriginalDataSource->GetFName()),
					PathView.Len(), PathView.GetData());

				// PreviousItem must have been new, add it to the set of root items
				RootItems.Add(PreviousItem);
				return false;
			}
			FName ParentVirtualPath{ PathView };
			TSharedPtr<FTreeItem> ParentItem = VirtualPathToItem.FindRef(ParentVirtualPath);
			bool bContinue = false;
			if (!ParentItem.IsValid())
			{
				UE_LOGF(LogPathView, VeryVerbose, "[%ls] Creating placeholder or virtual parent %ls:%.*ls", 
					*WriteToString<256>(OwningContentBrowserName),
					*WriteToString<256>(OriginalDataSource->GetFName()),
					PathView.Len(), PathView.GetData());
				// TODO: If another data source provides this path in future, can that data source become the 'primary'?
				FName ItemName(FPathViews::GetPathLeaf(PathView)); 
				FName InternalPath;
				if (ContentBrowserData->TryConvertVirtualPath(ParentVirtualPath, InternalPath) != EContentBrowserPathType::Internal)
				{
					InternalPath = FName(); // Assuming this is a virtual path with no internal path
				}
				ParentItem = UE::PathView::CreateOrReuseNode(FContentBrowserItemData(OriginalDataSource,
																 EContentBrowserItemFlags::Type_Folder,
																 ParentVirtualPath,
																 ItemName,
																 FText(),
																 nullptr, 
																 InternalPath
																 ),
					OldItemsByInvariantPath);
				VirtualPathToItem.Add(ParentVirtualPath, ParentItem);
				// TODO: Do fully virtual paths have an invariant path?
				InvariantPathToItem.Add(ParentItem->GetItem().GetInvariantPath()); 
				bContinue = true;
			}
			else
			{
				UE_LOGF(LogPathView, VeryVerbose, "[%ls] Found existing parent %.*ls", 
					*WriteToString<256>(OwningContentBrowserName),
					PathView.Len(), PathView.GetData());
			}
			ParentItem->AddChild(PreviousItem);
			PreviousItem = ParentItem.ToSharedRef();
			return bContinue; // If we made a node here, keep checking if we need to make more parent nodes
		});
	return LeafItem.ToSharedRef();
}

TSharedPtr<FTreeItem> FPathViewData::TryRemoveFolderItemInternal(const FContentBrowserItemData& InItemData)
{
	return TryRemoveFolderItemInternal(FContentBrowserMinimalItemData(InItemData));
}

TSharedPtr<FTreeItem> FPathViewData::TryRemoveFolderItemInternal(const FContentBrowserMinimalItemData& InItemKey)
{
	// Find the folder in the tree
	if (TSharedPtr<FTreeItem> ItemToRemove = VirtualPathToItem.FindRef(InItemKey.GetVirtualPath()))
	{
		// Only fully remove this item if every sub-item is removed (items become invalid when empty)
		FContentBrowserItemData OldItemData = ItemToRemove->RemoveItemData(InItemKey);
		if (ItemToRemove->GetItem().IsValid())
		{
			return {};
		}

		// Found the folder to remove. Remove it.
		TSharedPtr<FTreeItem> ItemParent = ItemToRemove->GetParent();
		if (ItemParent.IsValid())
		{
			// Remove the folder from its parent's list
			ItemParent->RemoveChild(ItemToRemove.ToSharedRef());
		}
		else
		{
			// This is a root item. Remove the folder from the root items list.
			RootItems.Remove(ItemToRemove);
			VisibleRootItems.Remove(ItemToRemove);
		}

		VirtualPathToItem.Remove(InItemKey.GetVirtualPath());
		InvariantPathToItem.Remove(OldItemData.GetInvariantPath());
		return ItemParent;
	}

	// Did not find the folder to remove
	return {};
}

void FPathViewData::RemoveFolderItem(const TSharedRef<FTreeItem>& TreeItem)
{
	if (TSharedPtr<FTreeItem> Parent = TreeItem->GetParent())
	{
		// Remove this item from it's parent's list
		Parent->RemoveChild(TreeItem);
	}
	else
	{
		// This was a root node, remove from the root list
		RootItems.Remove(TreeItem);
		VisibleRootItems.Remove(TreeItem);
	}

	VirtualPathToItem.Remove(TreeItem->GetItem().GetVirtualPath());
	InvariantPathToItem.Remove(TreeItem->GetItem().GetInvariantPath());

	Version++;
}


SLATE_IMPLEMENT_WIDGET(SPathView)
void SPathView::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	struct FInvalidation
	{
		static void HandleShowRedirectorsChanged(SWidget& OwningWidget)
		{
			SPathView& PathView = static_cast<SPathView&>(OwningWidget);
			UE_LOGF(LogPathView, Verbose, "PathView bShowRedirectors changed to %d", PathView.bShowRedirectorsAttribute.Get());
			PathView.HandleSettingChanged("ShowRedirectors");
		}
		static void RequestTreeRefresh(SWidget& OwningWidget)
		{
			SPathView& PathView = static_cast<SPathView&>(OwningWidget);
			if (ensure(PathView.TreeViewPtr))
			{
				PathView.TreeViewPtr->RequestTreeRefresh();
			}
		}
	};
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, bShowRedirectorsAttribute, EInvalidateWidgetReason::Paint)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::HandleShowRedirectorsChanged));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, TreeDataVersionAttribute, EInvalidateWidgetReason::Paint)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateStatic(FInvalidation::RequestTreeRefresh));

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, bPopulationRequestedAttribute, [](const SWidget& OwningWidget)
	{
		// Invalidate Paint only if value changed to true as this is where we process the Populate
		return static_cast<const SPathView&>(OwningWidget).bPopulationRequestedAttribute.Get() ? EInvalidateWidgetReason::Paint : EInvalidateWidgetReason::None;
	});
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, bInitialPopulationRequestedAttribute, [](const SWidget& OwningWidget)
	{
		// Invalidate Paint only if value changed to true as this is where we process the Populate
		return static_cast<const SPathView&>(OwningWidget).bInitialPopulationRequestedAttribute.Get() ? EInvalidateWidgetReason::Paint : EInvalidateWidgetReason::None;
	});
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, bPopulationForSettingChangeRequestedAttribute, [](const SWidget& OwningWidget)
	{
		// Invalidate Paint only if value changed to true as this is where we process the Populate
		return static_cast<const SPathView&>(OwningWidget).bPopulationForSettingChangeRequestedAttribute.Get() ? EInvalidateWidgetReason::Paint : EInvalidateWidgetReason::None;
	});
}

SPathView::FScopedSelectionChangedEvent::FScopedSelectionChangedEvent(const TSharedRef<SPathView>& InPathView, const bool InShouldEmitEvent)
	: PathView(InPathView)
	, bShouldEmitEvent(InShouldEmitEvent)
{
	PathView->PreventTreeItemChangedDelegateCount++;
	InitialSelectionSet = GetSelectionSet();
}

SPathView::FScopedSelectionChangedEvent::~FScopedSelectionChangedEvent()
{
	check(PathView->PreventTreeItemChangedDelegateCount > 0);
	PathView->PreventTreeItemChangedDelegateCount--;

	if (bShouldEmitEvent)
	{
		const TSet<FName> FinalSelectionSet = GetSelectionSet();
		const bool bHasSelectionChanges = InitialSelectionSet.Num() != FinalSelectionSet.Num() || InitialSelectionSet.Difference(FinalSelectionSet).Num() > 0;
		if (bHasSelectionChanges)
		{
			const TArray<TSharedPtr<FTreeItem>> NewSelectedItems = PathView->TreeViewPtr->GetSelectedItems();
			PathView->TreeSelectionChanged(NewSelectedItems.Num() > 0 ? NewSelectedItems[0] : nullptr,
				ESelectInfo::Direct);
		}
	}
}

bool SPathView::FScopedSelectionChangedEvent::IsInitialSelectionEmpty() const
{
	return InitialSelectionSet.IsEmpty();
}

TSet<FName> SPathView::FScopedSelectionChangedEvent::GetSelectionSet() const
{
	TSet<FName> SelectionSet;
	Algo::Transform(PathView->TreeViewPtr->GetSelectedItems(), SelectionSet, [](const TSharedPtr<FTreeItem>& Item) {
		return Item->GetItem().GetVirtualPath();
	});
	return SelectionSet;
}

SPathView::SPathView()
	: bShowRedirectorsAttribute(*this, false)
	, TreeDataVersionAttribute(*this, 0)
	, bPopulationRequestedAttribute(*this, false)
	, bInitialPopulationRequestedAttribute(*this, false)
	, bPopulationForSettingChangeRequestedAttribute(*this, false)
{
}

SPathView::~SPathView()
{
	UE::PathView::AllPathViews.RemoveAllSwap([this](const TWeakPtr<SPathView> Weak) { return Weak.Pin().Get() == this; });

	if (IContentBrowserDataModule* ContentBrowserDataModule = IContentBrowserDataModule::GetPtr())
	{
		if (UContentBrowserDataSubsystem* ContentBrowserData = ContentBrowserDataModule->GetSubsystem())
		{
			ContentBrowserData->OnItemDataUpdated().RemoveAll(this);
			ContentBrowserData->OnItemDataRefreshed().RemoveAll(this);
			ContentBrowserData->OnItemDataDiscoveryComplete().RemoveAll(this);
		}
	}

	TreeData->GetFolderPathTextFilter().OnChanged().RemoveAll(this);
}

void SPathView::Construct( const FArguments& InArgs )
{
	SetCanTick(false);

	UE::PathView::AllPathViews.Add(SharedThis(this));

	OwningContentBrowserName = InArgs._OwningContentBrowserName;
	OnItemSelectionChanged = InArgs._OnItemSelectionChanged;
	bAllowContextMenu = InArgs._AllowContextMenu;
	OnGetItemContextMenu = InArgs._OnGetItemContextMenu;
	InitialCategoryFilter = InArgs._InitialCategoryFilter;
	bAllowClassesFolder = InArgs._AllowClassesFolder;
	bAllowReadOnlyFolders = InArgs._AllowReadOnlyFolders;
	bCanShowDevelopersFolder = InArgs._CanShowDevelopersFolder;
	bForceShowEngineContent = InArgs._ForceShowEngineContent;
	bForceShowPluginContent = InArgs._ForceShowPluginContent;
	PreventTreeItemChangedDelegateCount = 0;
	TreeTitle = LOCTEXT("AssetTreeTitle", "Asset Tree");
	if ( InArgs._FocusSearchBoxWhenOpened )
	{
		RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SPathView::SetFocusPostConstruct ) );
	}

	TreeData = MakeShared<FPathViewData>(OwningContentBrowserName, bFlat);
	TreeData->GetFolderPathTextFilter().OnChanged().AddSP(this, &SPathView::FilterUpdated);

	UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
	ContentBrowserData->OnItemDataUpdated().AddSP(this, &SPathView::HandleItemDataUpdated);
	ContentBrowserData->OnItemDataRefreshed().AddSP(this, &SPathView::HandleItemDataRefreshed);

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	FolderPermissionList = AssetToolsModule.Get().GetFolderPermissionList();
	WritableFolderPermissionList = AssetToolsModule.Get().GetWritableFolderPermissionList();

	// Listen for when view settings are changed
	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	ContentBrowserModule.GetOnContentBrowserSettingChanged().AddSP(this, &SPathView::HandleSettingChanged);

	// Binds the commands for the PathView
	BindCommands();

	// Setup plugin filters
	PluginPathFilters = InArgs._PluginPathFilters;
	if (PluginPathFilters.IsValid())
	{
		// Add all built-in filters here
		AllPluginPathFilters.Add( MakeShareable(new FContentBrowserPluginFilter_ContentOnlyPlugins()) );

		// Add external filters
		for (const FContentBrowserModule::FAddPathViewPluginFilters& Delegate : ContentBrowserModule.GetAddPathViewPluginFilters())
		{
			if (Delegate.IsBound())
			{
				Delegate.Execute(AllPluginPathFilters);
			}
		}
	}

	STreeView<TSharedPtr<FTreeItem>>::FArguments TreeViewArgs;
	ConfigureTreeView(TreeViewArgs);
	TreeViewPtr = SArgumentNew(TreeViewArgs, STreeView<TSharedPtr<FTreeItem>>)
					  .TreeItemsSource(TreeData->GetVisibleRootItems())
					  .OnGetChildren(this, &SPathView::GetChildrenForTree)
					  .OnGenerateRow(this, &SPathView::GenerateTreeRow)
					  .OnItemScrolledIntoView(this, &SPathView::TreeItemScrolledIntoView)
					  .SelectionMode(InArgs._SelectionMode)
					  .AllowInvisibleItemSelection(true)
					  .OnSelectionChanged(this, &SPathView::TreeSelectionChanged)
					  .OnContextMenuOpening(this, &SPathView::MakePathViewContextMenu)
					  .ClearSelectionOnClick(false)
					  .ShadowBoxStyle(InArgs._ShadowBoxStyle)
					  .bEnableShadowBoxStyle(InArgs._bEnableShadowBoxStyle);

	SearchPtr = InArgs._ExternalSearch;
	if (!SearchPtr)
	{
		SearchPtr = MakeShared<FSourcesSearch>();
		SearchPtr->Initialize();
		SearchPtr->SetHintText(LOCTEXT("AssetTreeSearchBoxHint", "Search Folders"));
	}
	SearchPtr->OnSearchChanged().AddSP(this, &SPathView::SetSearchFilterText);

	TSharedRef<SBox> SearchBox = SNew(SBox);
	if (!InArgs._ExternalSearch)
	{
		SearchBox->SetContent(
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				InArgs._SearchContent.Widget
			]

			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SBox)
				.Visibility(InArgs._SearchBarVisibility)
				[
					SearchPtr->GetWidget()
				]
			]

			+SHorizontalBox::Slot()
			.Padding(4.f, 0.f, 0.f, 0.f)
			.AutoWidth()
			[
				SNew(SComboButton)
				.Visibility(InArgs._ShowViewOptions ? EVisibility::Visible : EVisibility::Collapsed)
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
				.OnGetMenuContent(this, &SPathView::GetViewButtonContent)
				.HasDownArrow(false)
				.ButtonContent()
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
				]
			]
		);
	}

	TSharedRef<SVerticalBox> ContentBox = SNew(SVerticalBox);

	if (!InArgs._ExternalSearch || InArgs._ShowTreeTitle)
	{
		ContentBox->AddSlot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.Padding(8.f)
			[
				SNew(SVerticalBox)

				// Search
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SearchBox
				]

				// Tree title
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Font(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetFontStyle("ContentBrowser.SourceTitleFont") )
					.Text(this, &SPathView::GetTreeTitle)
					.Visibility(InArgs._ShowTreeTitle ? EVisibility::Visible : EVisibility::Collapsed)
				]
			]
		];
	}

	// Separator
	if (InArgs._ShowSeparator)
	{
		ContentBox->AddSlot()
		.AutoHeight()
		.Padding(0, 0, 0, 1)
		[
			SNew(SSeparator)
		];
	}

	if (InArgs._ShowFavorites)
	{
		ContentBox->AddSlot()
		.FillHeight(1.f)
		[
			SNew(SSplitter)
			.Orientation(Orient_Vertical)
			+ SSplitter::Slot()
			.SizeRule_Lambda([this]()
				{ 
					return (FavoritesArea.IsValid() && FavoritesArea->IsExpanded()) ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent;
				})
			.MinSize(24)
			.Value(0.25f)
			[
				CreateFavoritesView()
			]
			+ SSplitter::Slot()
			.Value(0.75f)
			[
				TreeViewPtr.ToSharedRef()
			]
		];
	}
	else
	{
		// Tree
		ContentBox->AddSlot()
		.FillHeight(1.f)
		[
			TreeViewPtr.ToSharedRef()
		];
	}

	ChildSlot
	[
		ContentBox
	];

	CustomFolderPermissionList = InArgs._CustomFolderPermissionList;

	bShowRedirectorsAttribute.Assign(*this, InArgs._ShowRedirectors);
	TreeDataVersionAttribute.Assign(*this, TAttribute<uint64>::CreateLambda([this](){return TreeData->GetVersion();}));

	// Add all paths currently gathered from the asset registry
	bInitialPopulationRequestedAttribute.Assign(*this, true);

	DefaultPath = InArgs._DefaultPath;
	bCreateDefaultPath = InArgs._CreateDefaultPath;
}

void SPathView::ConfigureTreeView(STreeView<TSharedPtr<FTreeItem>>::FArguments& InArgs)
{
	InArgs.OnExpansionChanged(this, &SPathView::TreeExpansionChanged)
		.OnSetExpansionRecursive(this, &SPathView::SetTreeItemExpansionRecursive)
		.HighlightParentNodesForSelection(true);
}

int32 SPathView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	SPathView* MutableThis = const_cast<SPathView*>(this);

	if (IsPopulationRequested())
	{
		MutableThis->Populate();
	}

	MutableThis->UpdateLastExpandedPathsIfDirty();

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

FReply SPathView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (Commands->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

bool SPathView::IsEmpty() const
{
	return TreeViewPtr->GetRootItems().IsEmpty();
}

void SPathView::PopulatePathViewFiltersMenu(UToolMenu* Menu)
{
	{
		FToolMenuSection& Section = Menu->AddSection("Reset");
		Section.AddMenuEntry(
			"ResetPluginPathFilters",
			LOCTEXT("ResetPluginPathFilters_Label", "Reset Path View Filters"),
			LOCTEXT("ResetPluginPathFilters_Tooltip", "Reset current path view filters state"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SPathView::ResetPluginPathFilters))
		);
	}

	{
		FToolMenuSection& Section = Menu->AddSection("Filters", LOCTEXT("PathViewFilters_Label", "Filters"));

		for (const TSharedRef<FContentBrowserPluginFilter>& Filter : AllPluginPathFilters)
		{
			Section.AddMenuEntry(
				NAME_None,
				Filter->GetDisplayName(),
				Filter->GetToolTipText(),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), Filter->GetIconName()),
				FUIAction(
					FExecuteAction::CreateSP(this, &SPathView::PluginPathFilterClicked, Filter),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SPathView::IsPluginPathFilterChecked, Filter)
				),
				EUserInterfaceActionType::ToggleButton
			);
		}
	}
}

void SPathView::PluginPathFilterClicked(TSharedRef<FContentBrowserPluginFilter> Filter)
{
	SetPluginPathFilterActive(Filter, !IsPluginPathFilterInUse(Filter));
	
	bPopulationRequestedAttribute.Assign(*this, true);
}

bool SPathView::IsPluginPathFilterChecked(TSharedRef<FContentBrowserPluginFilter> Filter) const
{
	if (IsPluginPathFilterInUse(Filter))
	{
		return !Filter->IsInverseFilter();
	}
	return Filter->IsInverseFilter();
}

bool SPathView::IsPluginPathFilterInUse(TSharedRef<FContentBrowserPluginFilter> Filter) const
{
	if (PluginPathFilters.IsValid())
	{
		for (const TSharedPtr<IFilter<FPluginFilterType>>& PluginFilter : *PluginPathFilters.Get())
		{
			if (PluginFilter == Filter)
			{
				return true;
			}
		}
	}

	return false;
}

void SPathView::ResetPluginPathFilters()
{
	for (const TSharedRef<FContentBrowserPluginFilter>& Filter : AllPluginPathFilters)
	{
		SetPluginPathFilterActive(Filter, false);
	}

	bPopulationRequestedAttribute.Assign(*this, true);
}

bool SPathView::DisablePluginPathFiltersThatHideItems(TConstArrayView<FContentBrowserItem> Items)
{
	if (!PluginPathFilters.IsValid())
	{
		return false;
	}

	TSet<TSharedRef<IPlugin>> RelevantPlugins;

	for (const FContentBrowserItem& Item : Items)
	{
		FName InternalPath = Item.GetInternalPath();
		if (InternalPath.IsNone())
		{
			continue;
		}
		TStringBuilder<256> PathBuffer(InPlace, InternalPath);
		const FStringView MountPoint = FPathViews::GetMountPointNameFromPath(PathBuffer);
		if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(MountPoint))
		{
			RelevantPlugins.Add(Plugin.ToSharedRef());
		}
	}

	bool bAnyChanges = false;
	for (const TSharedRef<FContentBrowserPluginFilter>& Filter : AllPluginPathFilters)
	{
		if (Algo::AnyOf(RelevantPlugins, [&Filter](const TSharedRef<IPlugin>& Plugin) { return !Filter->PassesFilter(Plugin); }))
		{
			// Whether the filter is inverse or not, we don't want it in the list
			if (IsPluginPathFilterInUse(Filter))
			{
				SetPluginPathFilterActive(Filter, Filter->IsInverseFilter());
				bAnyChanges = true;
			}
		}
	}
	return bAnyChanges;
}

void SPathView::SetPluginPathFilterActive(const TSharedRef<FContentBrowserPluginFilter>& Filter, bool bActive)
{
	if (Filter->IsInverseFilter())
	{
		//Inverse filters are active when they are "disabled"
		bActive = !bActive;
	}

	UE_LOGF(LogPathView, Verbose, "[%ls] Setting%ls plugin filter %ls to %ls", 
		*WriteToString<64>(OwningContentBrowserName), 
		Filter->IsInverseFilter() ? TEXT(" inverse") : TEXT(""),
		*Filter->GetName(), bActive ? TEXT("Active") : TEXT("Inactive"));

	Filter->ActiveStateChanged(bActive);

	if (bActive)
	{
		PluginPathFilters->Add(Filter);
	}
	else
	{
		PluginPathFilters->Remove(Filter);
	}

	if (FPathViewConfig* PathViewConfig = GetPathViewConfig())
	{
		const FString FilterName = Filter->GetName();
		if (bActive)
		{
			if (!PathViewConfig->PluginFilters.Contains(FilterName))
			{
				PathViewConfig->PluginFilters.Add(FilterName);
				UContentBrowserConfig::Get()->SaveEditorConfig();
			}
		}
		else
		{
			PathViewConfig->PluginFilters.Remove(FilterName);
			UContentBrowserConfig::Get()->SaveEditorConfig();
		}
	}
}

FPathViewConfig* SPathView::GetPathViewConfig() const
{
	return ContentBrowserUtils::GetPathViewConfig(OwningContentBrowserName);
}

FContentBrowserInstanceConfig* SPathView::GetContentBrowserConfig() const
{
	return ContentBrowserUtils::GetContentBrowserConfig(OwningContentBrowserName);
}

void SPathView::SetSelectedPaths(const TArray<FName>& Paths)
{
	TArray<FString> PathStrings;
	Algo::Transform(Paths, PathStrings, [](const FName& Name) { return Name.ToString(); });
	SetSelectedPaths(PathStrings);
}

void SPathView::SetSelectedPaths(const TArray<FString>& Paths)
{
	if (!ensure(TreeViewPtr.IsValid()))
	{
		return;
	}

	// Prevent the selection changed delegate since the invoking code requested it
	FScopedPreventTreeItemChangedDelegate DelegatePrevention( SharedThis(this) );

	// If the selection was changed before all pending initial paths were found, stop attempting to select them
	PendingInitialPaths.Empty();

	// Clear the selection to start, then add the selected paths as they are found
	LastSelectedPaths.Empty();
	TreeViewPtr->ClearSelection();

	if (ShouldDoJustInTimePopulation())
	{
		TArray<const FName> InPathToPopulate;
		for (const FString& Path : Paths)
		{
			FName PathAsName = *Path;
			TSharedPtr<FTreeItem> Item = TreeData->FindTreeItem(PathAsName);
			if (!Item)
			{
				InPathToPopulate.Add(PathAsName);
			}
		}

		if (!InPathToPopulate.IsEmpty())
		{
			TreeData->ProcessVirtualPaths(InPathToPopulate,
				[this]() { return CreateCompiledFolderFilter(); });
		}
	}

	for (const FString& Path : Paths)
	{
		TSharedPtr<FTreeItem> BestItem = TreeData->FindBestItemForPath(Path);
		if (BestItem.IsValid())
		{
			if (!BestItem->IsVisible())
			{
				// Clear the search box if it potentially hides a path we want to select
				SearchPtr->ClearSearch();
			}

			for (TSharedPtr<FTreeItem> Parent = BestItem->GetParent(); Parent.IsValid(); Parent = Parent->GetParent())
			{
				TreeViewPtr->SetItemExpansion(Parent, true);
			}

			// Set the selection to the closest found folder and scroll it into view
			LastSelectedPaths.Add(BestItem->GetItem().GetInvariantPath());
			TreeViewPtr->SetItemSelection(BestItem, true);
			TreeViewPtr->RequestScrollIntoView(BestItem);
		}
	}
}

void SPathView::ClearSelection()
{
	// Prevent the selection changed delegate since the invoking code requested it
	FScopedPreventTreeItemChangedDelegate DelegatePrevention( SharedThis(this) );

	// If the selection was changed before all pending initial paths were found, stop attempting to select them
	PendingInitialPaths.Empty();

	// Clear the selection to start, then add the selected paths as they are found
	TreeViewPtr->ClearSelection();
}

FString SPathView::GetSelectedPath() const
{
	// TODO: Abstract away?
	TArray<TSharedPtr<FTreeItem>> Items = TreeViewPtr->GetSelectedItems();
	if ( Items.Num() > 0 )
	{
		return Items[0]->GetItem().GetVirtualPath().ToString();
	}

	return FString();
}

TArray<FString> SPathView::GetSelectedPaths() const
{
	TArray<FString> RetArray;

	// TODO: Abstract away?
	TArray<TSharedPtr<FTreeItem>> Items = TreeViewPtr->GetSelectedItems();
	for ( int32 ItemIdx = 0; ItemIdx < Items.Num(); ++ItemIdx )
	{
		RetArray.Add(Items[ItemIdx]->GetItem().GetVirtualPath().ToString());
	}

	return RetArray;
}

TArray<FContentBrowserItem> SPathView::GetSelectedFolderItems() const
{
	TArray<TSharedPtr<FTreeItem>> SelectedViewItems = TreeViewPtr->GetSelectedItems();

	TArray<FContentBrowserItem> SelectedFolders;
	for (const TSharedPtr<FTreeItem>& SelectedViewItem : SelectedViewItems)
	{
		if (!SelectedViewItem->GetItem().IsTemporary())
		{
			SelectedFolders.Emplace(SelectedViewItem->GetItem());
		}
	}
	return SelectedFolders;
}

void SPathView::RenameFolderItem(const FContentBrowserItem& InItem)
{
	if (!ensure(TreeViewPtr.IsValid()))
	{
		// No tree view for some reason
		return;
	}

	if (!InItem.IsFolder())
	{
		// Not a folder
		return;
	}

	// Find the folder in the tree
	if (TSharedPtr<FTreeItem> ItemToRename = TreeData->FindTreeItem(InItem.GetVirtualPath()))
	{
		if (!ItemToRename->IsVisible())
		{
			SearchPtr->ClearSearch();
		}
		ItemToRename->SetNamingFolder(true);

		TreeViewPtr->SetSelection(ItemToRename);
		TreeViewPtr->RequestScrollIntoView(ItemToRename);
	}
}

FContentBrowserDataCompiledFilter SPathView::CreateCompiledFolderFilter() const
{
	UE_LOGF(LogPathView, VeryVerbose, "[%ls] Creating folder filter", *WriteToString<256>(OwningContentBrowserName));

	const UContentBrowserSettings* ContentBrowserSettings = GetDefault<UContentBrowserSettings>();
	bool bDisplayPluginFolders = ContentBrowserSettings->GetDisplayPluginFolders();
	// check to see if we have an instance config that overrides the default in UContentBrowserSettings
	if (FContentBrowserInstanceConfig* EditorConfig = GetContentBrowserConfig())
	{
		bDisplayPluginFolders = EditorConfig->bShowPluginContent;
	}

	FContentBrowserDataFilter DataFilter;
	DataFilter.bRecursivePaths = true;
	DataFilter.ItemTypeFilter = EContentBrowserItemTypeFilter::IncludeFolders;
	DataFilter.ItemCategoryFilter = GetContentBrowserItemCategoryFilter();
	DataFilter.ItemAttributeFilter = GetContentBrowserItemAttributeFilter();

	UE_LOGF(LogPathView, VeryVerbose, "[%ls] bDisplayPluginFolders:%d ItemCategoryFilter:%d ItemAttributeFilter:%d", 
		*WriteToString<256>(OwningContentBrowserName), bDisplayPluginFolders, EnumToUnderlyingType(DataFilter.ItemCategoryFilter), EnumToUnderlyingType(DataFilter.ItemAttributeFilter));

	TSharedPtr<FPathPermissionList> CombinedFolderPermissionList = ContentBrowserUtils::GetCombinedFolderPermissionList(FolderPermissionList, bAllowReadOnlyFolders ? nullptr : WritableFolderPermissionList);

	if (CustomFolderPermissionList.IsValid())
	{
		if (!CombinedFolderPermissionList.IsValid())
		{
			CombinedFolderPermissionList = MakeShared<FPathPermissionList>();
		}
		CombinedFolderPermissionList->Append(*CustomFolderPermissionList);
	}

	if (PluginPathFilters.IsValid() && PluginPathFilters->Num() > 0 && bDisplayPluginFolders)
	{
		UE_SUPPRESS(LogPathView, VeryVerbose, {
			FString PluginFiltersString;
			for (int32 i=0; i < PluginPathFilters->Num(); ++i)
			{
				if (i != 0)
				{
					PluginFiltersString += TEXT(", ");
				}
				PluginFiltersString += PluginPathFilters->GetFilterAtIndex(i)->GetName();
			}
			UE_LOGF(LogPathView, VeryVerbose, "[%ls] Active plugin filters: %ls", 
				*WriteToString<256>(OwningContentBrowserName), *PluginFiltersString);
		});
		TArray<TSharedRef<IPlugin>> Plugins = IPluginManager::Get().GetEnabledPluginsWithContent();
		for (const TSharedRef<IPlugin>& Plugin : Plugins)
		{
			if (!PluginPathFilters->PassesAllFilters(Plugin))
			{
				FString MountedAssetPath = Plugin->GetMountedAssetPath();
				MountedAssetPath.RemoveFromEnd(TEXT("/"), ESearchCase::CaseSensitive);

				if (!CombinedFolderPermissionList.IsValid())
				{
					CombinedFolderPermissionList = MakeShared<FPathPermissionList>();
				}
				CombinedFolderPermissionList->AddDenyListItem("PluginPathFilters", MountedAssetPath);
			}
		}
	}

	UE_LOGF(LogPathView, VeryVerbose, "Compiled folder permission list: %ls", CombinedFolderPermissionList.IsValid() ? *CombinedFolderPermissionList->ToString() : TEXT("null"));

	ContentBrowserUtils::AppendAssetFilterToContentBrowserFilter(FARFilter(), nullptr, CombinedFolderPermissionList, DataFilter);

	FContentBrowserDataCompiledFilter CompiledDataFilter;
	{
		static const FName RootPath = "/";
		UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
		ContentBrowserData->CompileFilter(RootPath, DataFilter, CompiledDataFilter);
	}
	return CompiledDataFilter;
}

EContentBrowserItemCategoryFilter SPathView::GetContentBrowserItemCategoryFilter() const
{
	const UContentBrowserSettings* ContentBrowserSettings = GetDefault<UContentBrowserSettings>();
	bool bDisplayCppFolders = ContentBrowserSettings->GetDisplayCppFolders();
	// check to see if we have an instance config that overrides the default in UContentBrowserSettings
	if (FContentBrowserInstanceConfig* EditorConfig = GetContentBrowserConfig())
	{
		bDisplayCppFolders = EditorConfig->bShowCppFolders;
	}

	EContentBrowserItemCategoryFilter ItemCategoryFilter = InitialCategoryFilter;
	if (bAllowClassesFolder && bDisplayCppFolders)
	{
		ItemCategoryFilter |= EContentBrowserItemCategoryFilter::IncludeClasses;
	}
	else
	{
		ItemCategoryFilter &= ~EContentBrowserItemCategoryFilter::IncludeClasses;
	}
	ItemCategoryFilter &= ~EContentBrowserItemCategoryFilter::IncludeCollections;
	if (bShowRedirectorsAttribute.Get())
	{
		ItemCategoryFilter |= EContentBrowserItemCategoryFilter::IncludeRedirectors;
	}
	else
	{
		ItemCategoryFilter &= ~EContentBrowserItemCategoryFilter::IncludeRedirectors;
	}

	return ItemCategoryFilter;
}

EContentBrowserItemAttributeFilter SPathView::GetContentBrowserItemAttributeFilter() const
{
	const UContentBrowserSettings* ContentBrowserSettings = GetDefault<UContentBrowserSettings>();
	bool bDisplayEngineContent = ContentBrowserSettings->GetDisplayEngineFolder();
	bool bDisplayPluginContent = ContentBrowserSettings->GetDisplayPluginFolders();
	bool bDisplayDevelopersContent = ContentBrowserSettings->GetDisplayDevelopersFolder();
	bool bDisplayL10NContent = ContentBrowserSettings->GetDisplayL10NFolder();
	
	// check to see if we have an instance config that overrides the defaults in UContentBrowserSettings
	if (FContentBrowserInstanceConfig* EditorConfig = GetContentBrowserConfig())
	{
		bDisplayEngineContent = EditorConfig->bShowEngineContent;
		bDisplayPluginContent = EditorConfig->bShowPluginContent;
		bDisplayDevelopersContent = EditorConfig->bShowDeveloperContent;
		bDisplayL10NContent = EditorConfig->bShowLocalizedContent;
	}
	
	return EContentBrowserItemAttributeFilter::IncludeProject
			| (bDisplayEngineContent || bForceShowEngineContent ? EContentBrowserItemAttributeFilter::IncludeEngine : EContentBrowserItemAttributeFilter::IncludeNone)
			| (bDisplayPluginContent || bForceShowPluginContent ? EContentBrowserItemAttributeFilter::IncludePlugins : EContentBrowserItemAttributeFilter::IncludeNone)
			| (bDisplayDevelopersContent && bCanShowDevelopersFolder ? EContentBrowserItemAttributeFilter::IncludeDeveloper : EContentBrowserItemAttributeFilter::IncludeNone)
			| (bDisplayL10NContent ? EContentBrowserItemAttributeFilter::IncludeLocalized : EContentBrowserItemAttributeFilter::IncludeNone);
}

FVector2D SPathView::GetScrollDistance()
{
	if (!TreeViewPtr.IsValid())
	{
		return FVector2D::ZeroVector;
	}

	return TreeViewPtr->GetScrollDistance();
}

FVector2D SPathView::GetScrollDistanceRemaining()
{
	if (!TreeViewPtr.IsValid())
	{
		return FVector2D::ZeroVector;
	}

	return TreeViewPtr->GetScrollDistanceRemaining();
}

TSharedRef<SWidget> SPathView::GetScrollWidget()
{
	return SharedThis(this);
}

bool SPathView::InternalPathPassesBlockLists(const FStringView InInternalPath, const int32 InAlreadyCheckedDepth) const
{
	TArray<const FPathPermissionList*, TInlineAllocator<2>> BlockLists;
	if (FolderPermissionList.IsValid() && FolderPermissionList->HasFiltering())
	{
		BlockLists.Add(FolderPermissionList.Get());
	}

	if (!bAllowReadOnlyFolders && WritableFolderPermissionList.IsValid() && WritableFolderPermissionList->HasFiltering())
	{
		BlockLists.Add(WritableFolderPermissionList.Get());
	}

	for (const FPathPermissionList* Filter : BlockLists)
	{
		if (!Filter->PassesStartsWithFilter(InInternalPath, /*bAllowParentPaths*/true))
		{
			return false;
		}
	}

	if (InAlreadyCheckedDepth < 1 && PluginPathFilters.IsValid() && PluginPathFilters->Num() > 0)
	{
		const UContentBrowserSettings* ContentBrowserSettings = GetDefault<UContentBrowserSettings>();
		bool bDisplayPluginFolders = ContentBrowserSettings->GetDisplayPluginFolders();

		// check to see if we have an instance config that overrides the default in UContentBrowserSettings
		if (FContentBrowserInstanceConfig* EditorConfig = GetContentBrowserConfig())
		{
			bDisplayPluginFolders = EditorConfig->bShowPluginContent;
		}

		if (bDisplayPluginFolders)
		{
			const FStringView FirstFolderName = FPathViews::GetMountPointNameFromPath(InInternalPath);
			if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(FirstFolderName))
			{
				if (!PluginPathFilters->PassesAllFilters(Plugin.ToSharedRef()))
				{
					return false;
				}
			}
		}
	}

	return true;
}

void SPathView::SyncToItems(TArrayView<const FContentBrowserItem> ItemsToSync, const bool bAllowImplicitSync)
{
	TSet<TSharedRef<FTreeItem>> SyncTreeItems;
	TArray<const FName> VirtualPathsNotFound;
	for (const FContentBrowserItem& ItemToSync : ItemsToSync)
	{
		FName VirtualPathToSync;
		if (ItemToSync.IsFile())
		{
			// Files need to sync their parent folder in the tree, so chop off the end of their path
			VirtualPathToSync = FName(*FPaths::GetPath(ItemToSync.GetVirtualPath().ToString()));
		}
		else
		{
			VirtualPathToSync = ItemToSync.GetVirtualPath();
		}

		bool bShouldDoJustInTimePopulation = ShouldDoJustInTimePopulation();

		TSharedPtr<FTreeItem> Item = TreeData->FindTreeItem(VirtualPathToSync);
		if (Item.IsValid())
		{
			SyncTreeItems.Add(Item.ToSharedRef());
		}
		else if (bShouldDoJustInTimePopulation)
		{
			VirtualPathsNotFound.Add(VirtualPathToSync);
		}
	}

	if (!VirtualPathsNotFound.IsEmpty())
	{
		TreeData->ProcessVirtualPaths(VirtualPathsNotFound,
			[this]() { return CreateCompiledFolderFilter(); },
			SyncTreeItems);
	}

	if (Algo::AnyOf(SyncTreeItems, [](const TSharedRef<FTreeItem>& Item) { return !Item->IsVisible(); }))
	{
		// Clear the search box if it potentially hides a path we want to select
		SearchPtr->ClearSearch();
	}

	if (SyncTreeItems.Num() > 0)
	{
		// Batch the selection changed event
		FScopedSelectionChangedEvent ScopedSelectionChangedEvent(SharedThis(this));

		if (bAllowImplicitSync)
		{
			// Prune the current selection so that we don't unnecessarily change the path which might disorientate the user.
			// If a parent tree item is currently selected we don't need to clear it and select the child
			TSet<TSharedPtr<FTreeItem>> SelectedTreeItems{ TreeViewPtr->GetSelectedItems() };
			TSet<TSharedRef<FTreeItem>> FinalItems;
			for (const TSharedRef<FTreeItem>& ItemToSelect : SyncTreeItems)
			{
				// If the target item or any of its parents are already selected, maintain that object in the final
				// selection
				TSharedPtr<FTreeItem> It = ItemToSelect;
				while (It.IsValid() && !SelectedTreeItems.Contains(It))
				{
					It = It->GetParent();
				}

				if (It.IsValid())
				{
					FinalItems.Add(It.ToSharedRef());
				}
				else
				{
					// Otherwise select the specific folder we were asked for
					FinalItems.Add(ItemToSelect);
				}
			}
			SyncTreeItems = FinalItems;
		}

		// SyncTreeItems now shows exactly what we want to be selected and no more
		TreeViewPtr->ClearSelection();

		// SyncTreeItems should now only contain items which aren't already shown explicitly or implicitly (as a child)
		for (const TSharedRef<FTreeItem>& Item : SyncTreeItems)
		{
			RecursiveExpandParents(Item);
			TreeViewPtr->SetItemSelection(Item, true);
		}
	}

	// > 0 as some may have been removed in the code above
	if (SyncTreeItems.Num() > 0)
	{
		// Scroll the first item into view if applicable
		TreeViewPtr->RequestScrollIntoView(*SyncTreeItems.CreateConstIterator());
	}
}

void SPathView::ApplyHistoryData( const FHistoryData& History )
{
	// Prevent the selection changed delegate because it would add more history when we are just setting a state
	FScopedPreventTreeItemChangedDelegate DelegatePrevention( SharedThis(this) );

	// Update paths
	TArray<FString> SelectedPaths;
	for (const FName& HistoryPath : History.ContentSources.GetVirtualPaths())
	{
		SelectedPaths.Add(HistoryPath.ToString());
	}
	SetSelectedPaths(SelectedPaths);
}

void SPathView::SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& InstanceName) const
{
	FString SelectedPathsString;

	UContentBrowserDataSubsystem* DataSubsystem = IContentBrowserDataModule::Get().GetSubsystem();
	{
		TArray< TSharedPtr<FTreeItem> > PathItems = TreeViewPtr->GetSelectedItems();

		for (const TSharedPtr<FTreeItem>& Item : PathItems)
		{
			if (!SelectedPathsString.IsEmpty())
			{
				SelectedPathsString += TEXT(",");
			}

			FName InvariantPath;
			DataSubsystem->TryConvertVirtualPath(Item->GetItem().GetVirtualPath(), InvariantPath);
			InvariantPath.AppendString(SelectedPathsString);
		}
	}

	if (SelectedPathsString.IsEmpty())
	{
		/**
		 * We didn't finish selecting the initial paths so save those instead
		 * This can happen if the we save the settings before the view is populated or the ar scan is finished and the user didn't change the selected paths.
		 */
		for (FName SelectedPath : PendingInitialPaths)
		{
			if (!SelectedPathsString.IsEmpty())
			{
				SelectedPathsString += TEXT(",");
			}

			FName InvariantPath;
			DataSubsystem->TryConvertVirtualPath(SelectedPath, InvariantPath);
			InvariantPath.AppendString(SelectedPathsString);
		}
	}

	GConfig->SetString(*IniSection, *(InstanceName + TEXT(".SelectedPaths")), *SelectedPathsString, IniFilename);

	FString PluginFiltersString;
	if (PluginPathFilters.IsValid())
	{
		for (const TSharedPtr<IFilter<FPluginFilterType>>& Filter : *PluginPathFilters.Get())
		{
			if (PluginFiltersString.Len() > 0)
			{
				PluginFiltersString += TEXT(",");
			}

			TSharedPtr<FContentBrowserPluginFilter> PluginFilter = StaticCastSharedPtr<FContentBrowserPluginFilter>(Filter);
			PluginFiltersString += PluginFilter->GetName();
		}
		GConfig->SetString(*IniSection, *(InstanceName + TEXT(".PluginFilters")), *PluginFiltersString, IniFilename);
	}
}

void SPathView::LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString)
{
	// Selected Paths
	TArray<FName> NewSelectedPaths;
	{
		FString SelectedPathsString;
		if (GConfig->GetString(*IniSection, *(SettingsString + TEXT(".SelectedPaths")), SelectedPathsString, IniFilename))
		{
			TArray<FString> ParsedPaths;
			SelectedPathsString.ParseIntoArray(ParsedPaths, TEXT(","), /*bCullEmpty*/true);

			Algo::Transform(ParsedPaths, NewSelectedPaths, [](const FString& Str) { return *Str; });
		}
	}

	// Replace each path in NewSelectedPaths with virtual version of that path
	for (FName& Path : NewSelectedPaths)
	{
		IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual(Path, Path);
	}

	UE_LOGF(LogPathView, Verbose, "[%ls] LoadSettings: SelectedPaths: %ls", 
		*WriteToString<256>(OwningContentBrowserName), *FString::JoinBy(NewSelectedPaths, TEXT(", "), UE_PROJECTION_MEMBER(FName, ToString)));

	{
		// Batch the selection changed event
		FScopedSelectionChangedEvent ScopedSelectionChangedEvent(SharedThis(this));

		UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
		if (ContentBrowserData->IsDiscoveringItems())
		{
			PendingInitialPaths = NewSelectedPaths;

			// If any of the pending paths are available, select only them
			// otherwise, leave the selection unchanged until we discover some
			if (Algo::AnyOf(NewSelectedPaths, [this](FName VirtualPath) {
					return TreeData->FindTreeItem(VirtualPath, /* bVisibleOnly */ true).IsValid();
				}))
			{
				// Clear any previously selected paths
				LastSelectedPaths.Empty();
				TreeViewPtr->ClearSelection();
			}

			// If the selected paths is empty, the path was "All assets"
			// This should handle that case properly
			for (const FName& Path : NewSelectedPaths)
			{
				ExplicitlyAddPathToSelection(Path);
			}

			// Keep entire list of pending paths around until discovery is complete or all of them are selected
			PendingInitialPaths = NewSelectedPaths;
		}
		else
		{
			PendingInitialPaths.Reset();
			// If all assets are already discovered, just select paths the best we can
			SetSelectedPaths(NewSelectedPaths);
		}
	}

	// Plugin Filters
	if (PluginPathFilters.IsValid())
	{
		TArray<FString> NewSelectedFilters;
		if (FPathViewConfig* PathViewConfig = GetPathViewConfig())
		{
			UE_LOGF(LogPathView, Verbose, "[%ls] LoadSettings: Loading plugin filters from editor config: %ls", 
				*WriteToString<256>(OwningContentBrowserName), *FString::Join(NewSelectedFilters, TEXT(", ")));
			NewSelectedFilters = PathViewConfig->PluginFilters;
		}
		else
		{
			FString PluginFiltersString;
			if (GConfig->GetString(*IniSection, *(SettingsString + TEXT(".PluginFilters")), PluginFiltersString, IniFilename))
			{
				UE_LOGF(LogPathView, Verbose, "[%ls] LoadSettings: Loading plugin filters from ini: %ls", 
					*WriteToString<256>(OwningContentBrowserName), *PluginFiltersString);
				PluginFiltersString.ParseIntoArray(NewSelectedFilters, TEXT(","), /*bCullEmpty*/ true);
			}
		}

		bool bAnyFiltersActive = false;
		for (const TSharedRef<FContentBrowserPluginFilter>& Filter : AllPluginPathFilters)
		{
			bool bFilterActive = NewSelectedFilters.Contains(Filter->GetName());
			SetPluginPathFilterActive(Filter, bFilterActive);

			bAnyFiltersActive |= bFilterActive;
		}

		if (bAnyFiltersActive)
		{
			Populate();
		}
	}
}

EActiveTimerReturnType SPathView::SetFocusPostConstruct( double InCurrentTime, float InDeltaTime )
{
	FWidgetPath WidgetToFocusPath;
	FSlateApplication::Get().GeneratePathToWidgetUnchecked( SearchPtr->GetWidget(), WidgetToFocusPath );
	FSlateApplication::Get().SetKeyboardFocus( WidgetToFocusPath, EFocusCause::SetDirectly );

	return EActiveTimerReturnType::Stop;
}

TSharedPtr<SWidget> SPathView::MakePathViewContextMenu()
{
	if (!bAllowContextMenu || !OnGetItemContextMenu.IsBound())
	{
		return nullptr;
	}

	const TArray<FContentBrowserItem> CurrentSelectedItems = GetSelectedFolderItems();
	if (CurrentSelectedItems.Num() == 0)
	{
		return nullptr;
	}

	return OnGetItemContextMenu.Execute(CurrentSelectedItems);
}

void SPathView::NewFolderItemRequested(const FContentBrowserItemTemporaryContext& NewItemContext)
{
	bool bAddedTemporaryFolder = false;
	TSharedPtr<FTreeItem> NewItem;
	// TODO: Consider having FTreeItem explicitly store FContentBrowserItemTemporaryContext
	for (const FContentBrowserItemData& NewItemData : NewItemContext.GetItem().GetInternalItems())
	{
		NewItem = TreeData->AddFolderItem(CopyTemp(NewItemData));
	}

	if (NewItem.IsValid())
	{
		PendingNewFolderContext = NewItemContext;

		PendingInitialPaths.Reset();

		RecursiveExpandParents(NewItem);
		TreeViewPtr->SetSelection(NewItem);
		NewItem->SetNamingFolder(true);
		TreeViewPtr->RequestScrollIntoView(NewItem);
	}
}

bool SPathView::ExplicitlyAddPathToSelection(const FName Path)
{
	if ( !ensure(TreeViewPtr.IsValid()) )
	{
		return false;
	}

	if (TSharedPtr<FTreeItem> FoundItem = TreeData->FindTreeItem(Path))
	{
		if (TreeViewPtr->IsItemSelected(FoundItem))
		{
			return true;
		}

		if (!FoundItem->IsVisible())
		{
			SearchPtr->ClearSearch();
		}

		// Set the selection to the closest found folder and scroll it into view
		RecursiveExpandParents(FoundItem);
		LastSelectedPaths.Add(FoundItem->GetItem().GetInvariantPath());
		TreeViewPtr->SetItemSelection(FoundItem, true);
		TreeViewPtr->RequestScrollIntoView(FoundItem);

		return true;
	}

	return false;
}

bool SPathView::ShouldAllowTreeItemChangedDelegate() const
{
	return PreventTreeItemChangedDelegateCount == 0;
}

void SPathView::RecursiveExpandParents(const TSharedPtr<FTreeItem>& Item)
{
	if (TSharedPtr<FTreeItem> Parent = Item->GetParent())
	{
		RecursiveExpandParents(Parent);
		TreeViewPtr->SetItemExpansion(Parent, true);
	}
}

TSharedRef<ITableRow> SPathView::GenerateTreeRow( TSharedPtr<FTreeItem> TreeItem, const TSharedRef<STableViewBase>& OwnerTable )
{
	check(TreeItem.IsValid());

	return
		SNew(STableRow<TSharedPtr<FTreeItem>>, OwnerTable)
		.OnDragDetected(this, &SPathView::OnFolderDragDetected)
		[
			SNew(SAssetTreeItem)
			.TreeItem(TreeItem)
			.OnNameChanged(this, &SPathView::FolderNameChanged)
			.OnVerifyNameChanged(this, &SPathView::VerifyFolderNameChanged)
			.IsItemExpanded(this, &SPathView::IsTreeItemExpanded, TreeItem)
			.HighlightText(this, &SPathView::GetHighlightText)
			.IsSelected(this, &SPathView::IsTreeItemSelected, TreeItem)
		];
}

void SPathView::TreeItemScrolledIntoView( TSharedPtr<FTreeItem> TreeItem, const TSharedPtr<ITableRow>& Widget )
{
	if ( TreeItem->IsNamingFolder() && Widget.IsValid() && Widget->GetContent().IsValid() )
	{
		TreeItem->OnRenameRequested().Broadcast();
	}
}

void SPathView::GetChildrenForTree( TSharedPtr< FTreeItem > TreeItem, TArray< TSharedPtr<FTreeItem> >& OutChildren )
{
	TreeItem->GetSortedVisibleChildren(OutChildren);
}

void SPathView::SetTreeItemExpansionRecursive( TSharedPtr< FTreeItem > TreeItem, bool bInExpansionState )
{
	TreeViewPtr->SetItemExpansion(TreeItem, bInExpansionState);

	TreeItem->ForAllChildrenRecursive([this, bInExpansionState](const TSharedRef<FTreeItem>& Child) {
		TreeViewPtr->SetItemExpansion(Child, bInExpansionState);
	});
}

void SPathView::TreeSelectionChanged( TSharedPtr< FTreeItem > TreeItem, ESelectInfo::Type SelectInfo )
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		PendingInitialPaths.Reset();
	}

	if ( ShouldAllowTreeItemChangedDelegate() )
	{
		const TArray<TSharedPtr<FTreeItem>> NewSelectedItems = TreeViewPtr->GetSelectedItems();

		LastSelectedPaths.Empty();
		for (int32 ItemIdx = 0; ItemIdx < NewSelectedItems.Num(); ++ItemIdx)
		{
			const TSharedPtr<FTreeItem> Item = NewSelectedItems[ItemIdx];
			if ( !ensure(Item.IsValid()) )
			{
				// All items must exist
				continue;
			}

			// Keep track of the last paths that we broadcasted for selection reasons when filtering
			LastSelectedPaths.Add(Item->GetItem().GetInvariantPath());
		}

		if ( OnItemSelectionChanged.IsBound() )
		{
			if ( TreeItem.IsValid() )
			{
				OnItemSelectionChanged.Execute(TreeItem->GetItem(), SelectInfo);
			}
			else
			{
				OnItemSelectionChanged.Execute(FContentBrowserItem(), SelectInfo);
			}
		}
	}

	if (TreeItem.IsValid())
	{
		// Prioritize the content scan for the selected path
		UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
		ContentBrowserData->PrioritizeSearchPath(TreeItem->GetItem().GetVirtualPath());
	}
}

void SPathView::TreeExpansionChanged( TSharedPtr< FTreeItem > TreeItem, bool bIsExpanded )
{
	if ( ShouldAllowTreeItemChangedDelegate() )
	{
		DirtyLastExpandedPaths();

		if (!bIsExpanded)
		{
			const TArray<TSharedPtr<FTreeItem>> CurrentSelectedItems = TreeViewPtr->GetSelectedItems();
			bool bSelectTreeItem = false;

			// If any selected item was a child of the collapsed node, then add the collapsed node to the current selection
			// This avoids the selection ever becoming empty, as this causes the Content Browser to show everything
			for (const TSharedPtr<FTreeItem>& SelectedItem : CurrentSelectedItems)
			{
				if (SelectedItem->IsChildOf(*TreeItem.Get()))
				{
					bSelectTreeItem = true;
					break;
				}
			}

			if (bSelectTreeItem)
			{
				TreeViewPtr->SetItemSelection(TreeItem, true);
			}
		}
	}
}

void SPathView::FilterUpdated()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPathView::FilterUpdated);

	// Batch the selection changed event
	// Only emit events when the user isn't filtering, as the selection may be artificially limited by the filter
	FScopedSelectionChangedEvent ScopedSelectionChangedEvent(SharedThis(this), false);

	if (TreeData->GetFolderPathTextFilter().GetRawFilterText().IsEmpty())
	{
		TreeData->ClearItemFilterState();
		TreeViewPtr->ClearExpandedItems();

		// First expand the default expanded paths
		for (const FName PathToExpand : GetDefaultPathsToExpand())
		{
			if (TSharedPtr<FTreeItem> FoundItem = TreeData->FindTreeItem(PathToExpand))
			{
				RecursiveExpandParents(FoundItem);
				TreeViewPtr->SetItemExpansion(FoundItem, true);
			}
		}

		TArray<TSharedPtr<FTreeItem>> SelectedItems = TreeViewPtr->GetSelectedItems();
		if (!SelectedItems.IsEmpty())
		{
			for (const TSharedPtr<FTreeItem>& SelectedItem : SelectedItems)
			{
				for (TSharedPtr<FTreeItem> Parent = SelectedItem->GetParent(); Parent.IsValid();
					 Parent = Parent->GetParent())
				{
					TreeViewPtr->SetItemExpansion(Parent, true);
				}
			}

			TreeViewPtr->RequestScrollIntoView(SelectedItems[0]);
		}
	}
	else
	{
		TreeData->FilterFullFolderTree();
		TreeViewPtr->ClearExpandedItems();
		for (const TSharedPtr<FTreeItem>& Root : *TreeData->GetVisibleRootItems())
		{
			TreeViewPtr->SetItemExpansion(Root, true);
			Root->ForAllChildrenRecursive([this](const TSharedPtr<FTreeItem> Descendant) {
				if (Descendant->GetHasVisibleDescendants())
				{
					TreeViewPtr->SetItemExpansion(Descendant, true);
				}
			});
		}
	}
}

void SPathView::SetSearchFilterText(const FText& InSearchText, TArray<FText>& OutErrors)
{
	TreeData->GetFolderPathTextFilter().SetRawFilterText(InSearchText);

	const FText ErrorText = TreeData->GetFolderPathTextFilter().GetFilterErrorText();
	if (!ErrorText.IsEmpty())
	{
		OutErrors.Add(ErrorText);
	}
}

FText SPathView::GetHighlightText() const
{
	return TreeData->GetFolderPathTextFilter().GetRawFilterText();
}

void SPathView::RequestPopulation(FSimpleMulticastDelegate::FDelegate&& AfterPopulation)
{
	bPopulationRequestedAttribute.Assign(*this, true);

	if (AfterPopulation.IsBound())
	{
		OnNextPopulate.Add(MoveTemp(AfterPopulation));
	}
}


void SPathView::Populate(const bool bIsRefreshingFilter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SPathView::Populate);
	UE_LOGF(LogPathView, Verbose, "Repopulating path view");

	const bool bFilteringByText = !TreeData->GetFolderPathTextFilter().GetRawFilterText().IsEmpty();

	// Batch the selection changed event
	// Only emit events when the user isn't filtering
	FScopedSelectionChangedEvent ScopedSelectionChangedEvent(SharedThis(this),
		!bFilteringByText && !bIsRefreshingFilter);
	TreeData->PopulateFullFolderTree(CreateCompiledFolderFilter());
	TreeData->FilterFullFolderTree();
	TreeData->SortRootItems();

	if (bPopulationForSettingChangeRequestedAttribute.Get())
	{
		PostPopulateHandleSettingChanged(!ScopedSelectionChangedEvent.IsInitialSelectionEmpty());
	}

	if (bInitialPopulationRequestedAttribute.Get())
	{
		PostInitialPopulation();
	}

	// Select any of our initial paths which aren't currently selected
	if (Algo::AllOf(PendingInitialPaths, [this](FName VirtualPath) {
		return ExplicitlyAddPathToSelection(VirtualPath);
		}))
	{
		UE_LOGF(LogPathView, VeryVerbose, "[%ls] Resetting pending initial paths as all are selected", *WriteToString<256>(OwningContentBrowserName));
		PendingInitialPaths.Reset();
	}

	SharedPostPopulate();
}

void SPathView::SharedPostPopulate()
{
	OnNextPopulate.Broadcast();
	OnNextPopulate.Clear();

	bPopulationRequestedAttribute.Assign(*this, false);
	bInitialPopulationRequestedAttribute.Assign(*this, false);
	bPopulationForSettingChangeRequestedAttribute.Assign(*this, false);
}


FReply SPathView::OnFolderDragDetected(const FGeometry& Geometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton ) )
	{
		if (TSharedPtr<FDragDropOperation> DragDropOp = DragDropHandler::CreateDragOperation(GetSelectedFolderItems()))
		{
			return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
		}
	}

	return FReply::Unhandled();
}

bool SPathView::VerifyFolderNameChanged(const TSharedPtr< FTreeItem >& TreeItem, const FString& ProposedName, FText& OutErrorMessage) const
{
	if (PendingNewFolderContext.IsValid())
	{
		checkf(FContentBrowserItemKey(TreeItem->GetItem()) == FContentBrowserItemKey(PendingNewFolderContext.GetItem()), TEXT("PendingNewFolderContext was still set when attempting to rename a different item!"));

		return PendingNewFolderContext.ValidateItem(ProposedName, &OutErrorMessage);
	}
	else if (!TreeItem->GetItem().GetItemName().ToString().Equals(ProposedName))
	{
		UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();

		return TreeItem->GetItem().CanRename(&ProposedName, ContentBrowserData->CreateHideFolderIfEmptyFilter().Get(), &OutErrorMessage);
	}

	return true;
}

void SPathView::FolderNameChanged(const TSharedPtr<FTreeItem>& TreeItem,
	const FString& ProposedName,
	const UE::Slate::FDeprecateVector2DParameter& MessageLocation,
	const ETextCommit::Type CommitType)
{
	if (!TreeItem.IsValid())
	{
		return;
	}

	bool bSuccess = false;
	FText ErrorMessage;

	// Group the deselect and reselect into a single operation
	FScopedSelectionChangedEvent ScopedSelectionChangedEvent(SharedThis(this));
	FContentBrowserItem NewItem;
	if (PendingNewFolderContext.IsValid())
	{
		checkf(FContentBrowserItemKey(TreeItem->GetItem()) == FContentBrowserItemKey(PendingNewFolderContext.GetItem()), TEXT("PendingNewFolderContext was still set when attempting to rename a different item!"));

		// Remove the temporary item before we do any work to ensure the new item creation is not prevented
		TreeData->RemoveFolderItem(TreeItem.ToSharedRef());
		TreeViewPtr->SetItemSelection(TreeItem.ToSharedRef(), false);

		// Clearing the rename box on a newly created item cancels the entire creation process
		if (CommitType == ETextCommit::OnCleared)
		{
			// We need to select the parent item of this folder, as the folder would have become selected while it was being named
			if (TSharedPtr<FTreeItem> ParentTreeItem = TreeItem->GetParent())
			{
				TreeViewPtr->SetItemSelection(ParentTreeItem, true);
			}
			else
			{
				TreeViewPtr->ClearSelection();
			}
		}
		else
		{
			UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
			FScopedSuppressContentBrowserDataTick TickSuppression(ContentBrowserData);

			if (PendingNewFolderContext.ValidateItem(ProposedName, &ErrorMessage))
			{
				NewItem = PendingNewFolderContext.FinalizeItem(ProposedName, &ErrorMessage);
				if (NewItem.IsValid())
				{
					bSuccess = true;
				}
			}
		}

		PendingNewFolderContext = FContentBrowserItemTemporaryContext();
	}
	else if (CommitType != ETextCommit::OnCleared && !TreeItem->GetItem().GetItemName().ToString().Equals(ProposedName))
	{
		UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
		FScopedSuppressContentBrowserDataTick TickSuppression(ContentBrowserData);
		
		if (TreeItem->GetItem().CanRename(&ProposedName, ContentBrowserData->CreateHideFolderIfEmptyFilter().Get(), &ErrorMessage) && TreeItem->GetItem().Rename(ProposedName, &NewItem))
		{
			bSuccess = true;
		}
	}

	if (bSuccess && NewItem.IsValid())
	{
		// Add result to view
		TSharedPtr<FTreeItem> NewTreeItem;
		for (const FContentBrowserItemData& NewItemData : NewItem.GetInternalItems())
		{
			NewTreeItem = TreeData->AddFolderItem(CopyTemp(NewItemData));
		}

		// Select the new item
		if (NewTreeItem)
		{
			TreeViewPtr->SetItemSelection(NewTreeItem, true);
			TreeViewPtr->RequestScrollIntoView(NewTreeItem);
		}
	}

	if (!bSuccess && !ErrorMessage.IsEmpty())
	{
		// Display the reason why the folder was invalid
		FSlateRect MessageAnchor(MessageLocation.X, MessageLocation.Y, MessageLocation.X, MessageLocation.Y);
		ContentBrowserUtils::DisplayMessage(ErrorMessage, MessageAnchor, SharedThis(this), ContentBrowserUtils::EDisplayMessageType::Error);
	}
}

bool SPathView::IsTreeItemExpanded(TSharedPtr<FTreeItem> TreeItem) const
{
	return TreeViewPtr->IsItemExpanded(TreeItem);
}

bool SPathView::IsTreeItemSelected(TSharedPtr<FTreeItem> TreeItem) const
{
	return TreeViewPtr->IsItemSelected(TreeItem);
}

void SPathView::HandleItemDataUpdated(TArrayView<const FContentBrowserItemDataUpdate> InUpdatedItems)
{
	if (IsPopulationRequested())
	{
		/**
		 * Skip the data updates.
		 * We are about to refresh the view from scratch.
		 */
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(SPathView::HandleItemDataUpdated);

	if (InUpdatedItems.Num() == 0)
	{
		return;
	}

	// TODO: Consider batching if sometimes we get very few items and filter construction time dominates
	if (!Algo::AnyOf(InUpdatedItems,
			[](const FContentBrowserItemDataUpdate& Update) { return Update.GetItemData().IsFolder(); }))
	{
		UE_LOGF(LogPathView, VeryVerbose, "[%ls] Skipping item data update because there were no folders present", *WriteToString<256>(OwningContentBrowserName));
		return;
	}

	const bool bFilteringByText = !TreeData->GetFolderPathTextFilter().GetRawFilterText().IsEmpty();

	// Batch the selection changed event
	// Only emit events when the user isn't filtering, as the selection may be artificially limited by the filter
	FScopedSelectionChangedEvent ScopedSelectionChangedEvent(SharedThis(this), !bFilteringByText);

	const double HandleItemDataUpdatedStartTime = FPlatformTime::Seconds();
	TreeData->ProcessDataUpdates(InUpdatedItems, CreateCompiledFolderFilter());
	UE_LOGF(LogPathView, VeryVerbose, "[%ls] PathView - HandleItemDataUpdated completed in %0.4f seconds for %d items",
		*WriteToString<256>(OwningContentBrowserName),
		FPlatformTime::Seconds() - HandleItemDataUpdatedStartTime,
		InUpdatedItems.Num());

	// Select any of our initial paths which aren't currently selected
	if (Algo::AllOf(PendingInitialPaths, [this](FName VirtualPath) { 
		return ExplicitlyAddPathToSelection(VirtualPath);
	}))
	{
		UE_LOGF(LogPathView, VeryVerbose, "[%ls] Resetting pending initial paths as all are selected", *WriteToString<256>(OwningContentBrowserName));
		PendingInitialPaths.Reset();
	}
}

void SPathView::HandleItemDataRefreshed()
{
	bPopulationRequestedAttribute.Assign(*this, true);
}

void SPathView::HandleSettingChanged(FName PropertyName)
{
	if ((PropertyName == GET_MEMBER_NAME_CHECKED(UContentBrowserSettings, DisplayEmptyFolders))
		|| (PropertyName == "ShowRedirectors")
		|| (PropertyName == "DisplayDevelopersFolder")
		|| (PropertyName == "DisplayEngineFolder")
		|| (PropertyName == "DisplayPluginFolders")
		|| (PropertyName == "DisplayL10NFolder")
		|| (PropertyName == GET_MEMBER_NAME_CHECKED(UContentBrowserSettings, bDisplayContentFolderSuffix))
		|| (PropertyName == GET_MEMBER_NAME_CHECKED(UContentBrowserSettings, bDisplayFriendlyNameForPluginFolders))
		|| (PropertyName == NAME_None)) // @todo: Needed if PostEditChange was called manually, for now
	{
		UE_LOGF(LogPathView,
			Log,
			"[%ls][PathView] HandleSettingChanged %ls",
			*WriteToString<256>(OwningContentBrowserName),
			*WriteToString<256>(PropertyName));
		const bool bHadSelectedPath = TreeViewPtr->GetNumItemsSelected() > 0;

		// Update our path view so that it can include/exclude the permitted folder
		bPopulationForSettingChangeRequestedAttribute.Assign(*this, true);
	}
}

void SPathView::PostPopulateHandleSettingChanged(bool bHadSelectedPath)
{
	// If folder is no longer visible but we're inside it...
	if (TreeViewPtr->GetNumItemsSelected() == 0 && bHadSelectedPath)
	{
		for (const FName VirtualPath : GetDefaultPathsToSelect())
		{
			if (TSharedPtr<FTreeItem> TreeItemToSelect = TreeData->FindTreeItem(VirtualPath))
			{
				TreeViewPtr->SetSelection(TreeItemToSelect);
				break;
			}
		}
	}

	// If the dev or engine folder has become visible and we're inside it...
	const UContentBrowserSettings* ContentBrowserSettings = GetDefault<UContentBrowserSettings>();
	bool bDisplayDev = ContentBrowserSettings->GetDisplayDevelopersFolder();
	bool bDisplayEngine = ContentBrowserSettings->GetDisplayEngineFolder();
	bool bDisplayPlugins = ContentBrowserSettings->GetDisplayPluginFolders();
	bool bDisplayL10N = ContentBrowserSettings->GetDisplayL10NFolder();
	// check to see if we have an instance config that overrides the default in UContentBrowserSettings
	if (FContentBrowserInstanceConfig* EditorConfig = GetContentBrowserConfig())
	{
		bDisplayDev = EditorConfig->bShowDeveloperContent;
		bDisplayEngine = EditorConfig->bShowEngineContent;
		bDisplayPlugins = EditorConfig->bShowPluginContent;
		bDisplayL10N = EditorConfig->bShowLocalizedContent;
	}

	if (bDisplayDev || bDisplayEngine || bDisplayPlugins || bDisplayL10N)
	{
		const TArray<FContentBrowserItem> NewSelectedItems = GetSelectedFolderItems();
		if (NewSelectedItems.Num() > 0)
		{
			const FContentBrowserItem& NewSelectedItem = NewSelectedItems[0];

			if ((bDisplayDev && ContentBrowserUtils::IsItemDeveloperContent(NewSelectedItem)) ||
				(bDisplayEngine && ContentBrowserUtils::IsItemEngineContent(NewSelectedItem)) ||
				(bDisplayPlugins && ContentBrowserUtils::IsItemPluginContent(NewSelectedItem)) ||
				(bDisplayL10N && ContentBrowserUtils::IsItemLocalizedContent(NewSelectedItem))
				)
			{
				// Refresh the contents
				OnItemSelectionChanged.ExecuteIfBound(NewSelectedItem, ESelectInfo::Direct);
			}
		}
	}
}

void SPathView::PostInitialPopulation()
{
	for (const FName PathToExpand : GetDefaultPathsToExpand())
	{
		if (TSharedPtr<FTreeItem> FoundItem = TreeData->FindTreeItem(PathToExpand))
		{
			RecursiveExpandParents(FoundItem);
			TreeViewPtr->SetItemExpansion(FoundItem, true);
		}
	}

	if (!DefaultPath.IsEmpty() && InternalPathPassesBlockLists(DefaultPath))
	{
		UContentBrowserDataSubsystem* ContentBrowserData = IContentBrowserDataModule::Get().GetSubsystem();
		const FName VirtualPath = ContentBrowserData->ConvertInternalPathToVirtual(*DefaultPath);
		FName InternalPath;
		if (ContentBrowserData->TryConvertVirtualPath(VirtualPath, InternalPath) != EContentBrowserPathType::Internal)
		{
			InternalPath = FName();
		}
		if (bCreateDefaultPath && !TreeData->FindTreeItem(VirtualPath))
		{
			const FString DefaultPathLeafName = FPaths::GetPathLeaf(VirtualPath.ToString());
			TreeData->AddFolderItem(FContentBrowserItemData(nullptr,
				EContentBrowserItemFlags::Type_Folder,
				VirtualPath,
				*DefaultPathLeafName,
				FText(),
				nullptr,
				InternalPath));
		}

		SetSelectedPaths({ VirtualPath.ToString() });
	}

	DefaultPath.Empty();
}

bool SPathView::IsPopulationRequested() const
{
	return bPopulationRequestedAttribute.Get() || bPopulationForSettingChangeRequestedAttribute.Get() || bInitialPopulationRequestedAttribute.Get();
}

TArray<FName> SPathView::GetDefaultPathsToSelect() const
{
	TArray<FName> VirtualPaths;
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	if (!ContentBrowserModule.GetDefaultSelectedPathsDelegate().ExecuteIfBound(VirtualPaths))
	{
		VirtualPaths.Add(IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual(TEXT("/Game")));
	}

	return VirtualPaths;
}

TArray<FName> SPathView::GetRootPathItemNames() const
{
	TArray<FName> RootPathItemNames;
	RootPathItemNames.Reserve(TreeData->GetVisibleRootItems()->Num());
	for (const TSharedPtr<FTreeItem>& RootItem : *TreeData->GetVisibleRootItems())
	{
		if (RootItem.IsValid())
		{
			RootPathItemNames.Add(RootItem->GetItem().GetItemName());
		}
	}

	return RootPathItemNames;
}

TArray<FName> SPathView::GetDefaultPathsToExpand() const
{
	TArray<FName> VirtualPaths;
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	if (!ContentBrowserModule.GetDefaultPathsToExpandDelegate().ExecuteIfBound(VirtualPaths))
	{
		VirtualPaths.Add(IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual(TEXT("/Game")));
	}

	return VirtualPaths;
}

void SPathView::DirtyLastExpandedPaths()
{
	bLastExpandedPathsDirty = true;
	Invalidate(EInvalidateWidgetReason::Paint);
}

void SPathView::UpdateLastExpandedPathsIfDirty()
{
	if (bLastExpandedPathsDirty)
	{
		TSet<TSharedPtr<FTreeItem>> ExpandedItemSet;
		TreeViewPtr->GetExpandedItems(ExpandedItemSet);

		LastExpandedPaths.Empty(ExpandedItemSet.Num());
		for (const TSharedPtr<FTreeItem>& Item : ExpandedItemSet)
		{
			if (!ensure(Item.IsValid()))
			{
				// All items must exist
				continue;
			}

			// Keep track of the last paths that we broadcasted for expansion reasons when filtering
			LastExpandedPaths.Add(Item->GetItem().GetInvariantPath());
		}

		bLastExpandedPathsDirty = false;
	}
}

TSharedRef<SWidget> SPathView::CreateFavoritesView()
{
	return SAssignNew(FavoritesArea, SExpandableArea)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
		.BodyBorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		.HeaderPadding(FMargin(4.0f, 4.0f))
		.Padding(0.f)
		.AllowAnimatedTransition(false)
		.InitiallyCollapsed(true)
		.HeaderContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Favorites", "Favorites"))
			.TextStyle(FAppStyle::Get(), "ButtonText")
			.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
		]
		.BodyContent()
		[
			SNew(SFavoritePathView)
			.OnItemSelectionChanged(OnItemSelectionChanged)
			.OnGetItemContextMenu(OnGetItemContextMenu)
			.FocusSearchBoxWhenOpened(false)
			.ShowTreeTitle(false)
			.ShowSeparator(false)
			.AllowClassesFolder(bAllowClassesFolder)
			.CanShowDevelopersFolder(bCanShowDevelopersFolder)
			.AllowReadOnlyFolders(bAllowReadOnlyFolders)
			.AllowContextMenu(bAllowContextMenu)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ContentBrowserFavorites")))
			.ExternalSearch(SearchPtr)
		];
}

void SPathView::RegisterGetViewButtonMenu()
{
	if (!UToolMenus::Get()->IsMenuRegistered("ContentBrowser.PathViewOptions"))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu("ContentBrowser.PathViewOptions");
		Menu->bCloseSelfOnly = true;
		Menu->AddDynamicSection("DynamicContent", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			FName ContextOwningContentBrowserName = NAME_None;
			FFiltersAdditionalParams Params;
			if (UContentBrowserPathViewContextMenuContext* Context = InMenu->FindContext<UContentBrowserPathViewContextMenuContext>())
			{
				if (Context->PathView.IsValid())
				{
					TSharedPtr<SPathView> PathView = Context->PathView.Pin();
					PathView->PopulateFilterAdditionalParams(Params);

					if (!PathView->OwningContentBrowserName.IsNone())
					{
						ContextOwningContentBrowserName = PathView->OwningContentBrowserName;
					}
				}

				if (ContextOwningContentBrowserName.IsNone() && !Context->OwningContentBrowserName.IsNone())
				{
					ContextOwningContentBrowserName = Context->OwningContentBrowserName;
				}

				ContentBrowserMenuUtils::AddFiltersToMenu(InMenu, ContextOwningContentBrowserName, Params);
			}
		}));
	}
}

void SPathView::PopulateFilterAdditionalParams(FFiltersAdditionalParams& OutParams)
{
	OutParams.CanShowCPPClasses = FCanExecuteAction::CreateSP(this, &SPathView::IsToggleShowCppContentAllowed);
	OutParams.CanShowDevelopersContent = FCanExecuteAction::CreateSP(this, &SPathView::IsToggleShowDevelopersContentAllowed);
	OutParams.CanShowEngineFolder = FCanExecuteAction::CreateSP(this, &SPathView::IsToggleShowEngineContentAllowed);
	OutParams.CanShowPluginFolder = FCanExecuteAction::CreateSP(this, &SPathView::IsToggleShowPluginContentAllowed);
	OutParams.CanShowLocalizedContent = FCanExecuteAction::CreateSP(this, &SPathView::IsToggleShowLocalizedContentAllowed);
}

bool SPathView::IsToggleShowCppContentAllowed() const
{
	return bAllowClassesFolder;
}

bool SPathView::IsToggleShowDevelopersContentAllowed() const
{
	return bCanShowDevelopersFolder;
}

bool SPathView::IsToggleShowEngineContentAllowed() const
{
	return !bForceShowEngineContent;
}

bool SPathView::IsToggleShowPluginContentAllowed() const
{
	return !bForceShowPluginContent;
}

bool SPathView::IsToggleShowLocalizedContentAllowed() const
{
	return true;
}

TSharedRef<SWidget> SPathView::GetViewButtonContent()
{
	SPathView::RegisterGetViewButtonMenu();

	UContentBrowserPathViewContextMenuContext* Context = NewObject<UContentBrowserPathViewContextMenuContext>();
	Context->PathView = SharedThis(this);
	Context->OwningContentBrowserName = OwningContentBrowserName;

	const FToolMenuContext MenuContext(Context);
					
	return UToolMenus::Get()->GenerateWidget("ContentBrowser.PathViewOptions", MenuContext);
}

void SPathView::CopySelectedFolder() const
{
	ContentBrowserUtils::CopyFolderReferencesToClipboard(GetSelectedFolderItems());
}

void SPathView::BindCommands()
{
	Commands = TSharedPtr<FUICommandList>(new FUICommandList);

	Commands->MapAction(FGenericCommands::Get().Copy, FUIAction(
		FExecuteAction::CreateSP(this, &SPathView::CopySelectedFolder)
	));
}

void SFavoritePathView::Construct(const FArguments& InArgs)
{
	// Bind the favorites menu to update after folder changes
	AssetViewUtils::OnFolderPathChanged().AddSP(this, &SFavoritePathView::FixupFavoritesFromExternalChange); 

	OnFavoritesChangedHandle = FContentBrowserSingleton::Get().RegisterOnFavoritesChangedHandler(
		FOnFavoritesChanged::FDelegate::CreateSP(this, &SFavoritePathView::OnFavoriteAdded));

	SPathView::Construct(InArgs);
}

void SFavoritePathView::ConfigureTreeView(STreeView<TSharedPtr<FTreeItem>>::FArguments& InArgs)
{
	// Don't bind some stuff that the parent class binds such as item expansion
}

void SFavoriteAssetTreeItem::Construct(const FArguments& InArgs)
{
	SAssetTreeItem::Construct(InArgs._Parent);

	UpdateItem(InArgs._Parent._TreeItem->GetItem().GetItemPath());

	// Once favorites are updated, check if this item is concerned to update alias
	FContentBrowserSingleton::Get().RegisterOnFavoritesChangedHandler(
		FOnFavoritesChanged::FDelegate::CreateSP(this, &SFavoriteAssetTreeItem::OnFavoriteUpdated)
	);
}

FText SFavoriteAssetTreeItem::GetNameText() const
{
	if (!Alias.IsEmpty())
	{
		return FText::Format(LOCTEXT("FavoriteAlias", "@{0}"), Alias);
	}

	return SAssetTreeItem::GetNameText();
}

FText SFavoriteAssetTreeItem::GetToolTipText() const
{
	FText BaseTooltip = SAssetTreeItem::GetToolTipText();

	if (!Shortcut.IsEmpty())
	{
		return FText::Format(LOCTEXT("FavoriteTooltip", "{0}\nShortcut : {1}"), BaseTooltip, Shortcut);
	}

	return BaseTooltip;
}

void SFavoriteAssetTreeItem::OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	if (const TSharedPtr<FContentBrowserDataDragDropOp> DragDropOp = GetFavoriteMoveOperation(InDragDropEvent))
	{
		DragDropOp->SetToolTip(LOCTEXT("AddOrReorderFavoriteFolder", "Add or Reorder favorite folder"), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")));
	}
	else
	{
		SAssetTreeItem::OnDragEnter(InGeometry, InDragDropEvent);
	}
}

void SFavoriteAssetTreeItem::OnDragLeave(const FDragDropEvent& InDragDropEvent)
{
	if (const TSharedPtr<FContentBrowserDataDragDropOp> DragDropOp = GetFavoriteMoveOperation(InDragDropEvent))
	{
		DragDropOp->SetToolTip(FText::GetEmpty(), nullptr);
		bDraggedOver = false;
	}
	else
	{
		SAssetTreeItem::OnDragLeave(InDragDropEvent);
	}
}

FReply SFavoriteAssetTreeItem::OnDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	if (!GetFavoriteMoveOperation(InDragDropEvent))
	{
		return SAssetTreeItem::OnDragOver(InGeometry, InDragDropEvent);
	}

	// Let the tree handle this
	return FReply::Unhandled();
}

FReply SFavoriteAssetTreeItem::OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	if (!GetFavoriteMoveOperation(InDragDropEvent))
	{
		return SAssetTreeItem::OnDrop(InGeometry, InDragDropEvent);
	}

	// Let the tree handle this
	return FReply::Unhandled();
}

void SFavoriteAssetTreeItem::OnFavoriteUpdated(const FContentBrowserItemPath& InItemPath, bool bInAdded)
{
	if (const TSharedPtr<FTreeItem> Item = TreeItem.Pin())
	{
		if (Item->GetItem().GetItemPath().GetVirtualPathName() == InItemPath.GetVirtualPathName())
		{
			UpdateItem(InItemPath);
		}
	}
}

void SFavoriteAssetTreeItem::UpdateItem(const FContentBrowserItemPath& InPath)
{
	if (const FContentBrowserFavoriteItem* FavoriteItem = ContentBrowserUtils::FindFavoriteFolder(InPath))
	{
		Alias = FText::FromString(FavoriteItem->Alias);

		FContentBrowserCommands& ContentBrowserCommands = FContentBrowserCommands::Get();
		const TSharedPtr<FUICommandInfo>* FavoriteShortcut = ContentBrowserCommands.FavoriteShortcuts.FindByPredicate([CommandName = FavoriteItem->ShortcutCommandName](const TSharedPtr<FUICommandInfo>& InCommand)
		{
			return CommandName == InCommand->GetCommandName();
		});

		if (FavoriteShortcut && FavoriteShortcut->IsValid())
		{
			Shortcut = (*FavoriteShortcut)->GetInputText();
		}
		else
		{
			Shortcut = FText::GetEmpty();
		}
	}
}

TSharedPtr<FContentBrowserDataDragDropOp> SFavoriteAssetTreeItem::GetFavoriteMoveOperation(const FDragDropEvent& InDragDropEvent)
{
	if (const TSharedPtr<FContentBrowserDataDragDropOp> DragDropOp = InDragDropEvent.GetOperationAs<FContentBrowserDataDragDropOp>())
	{
		// Drag operation contains only folders and CTRL is not pressed => Add/Reordering
		if (!FSlateApplication::Get().GetModifierKeys().IsControlDown()
			&& !DragDropOp->GetDraggedFolders().IsEmpty() 
			&& DragDropOp->GetDraggedFiles().IsEmpty())
		{
			return DragDropOp;
		}
	}

	// Drag operation contains assets files or folders while CTRL is pressed => Move/Copy
	return nullptr;
}

SFavoritePathView::SFavoritePathView()
{
	bFlat = true;
}

SFavoritePathView::~SFavoritePathView()
{
	FContentBrowserSingleton::Get().UnregisterOnFavoritesChangedDelegate(OnFavoritesChangedHandle);
}

void SFavoritePathView::Populate(const bool bIsRefreshingFilter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SFavoritePathView::Populate);

	// Don't allow the selection changed delegate to be fired here
	FScopedPreventTreeItemChangedDelegate DelegatePrevention(SharedThis(this));
	TreeData->PopulateWithFavorites(CreateCompiledFolderFilter());
	TreeData->FilterFullFolderTree();

	SharedPostPopulate();
}

void SFavoritePathView::LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString)
{
	TGuardValue<bool> Guard(bIsLoadingSettings, true);
	SPathView::LoadSettings(IniFilename, IniSection, SettingsString);

	// Migrate favorites to new project settings
	if (UContentBrowserFavoriteProjectSettings* FavoriteSettings = UContentBrowserFavoriteProjectSettings::Get())
	{
		// We clear the initial selection for the favorite view, as it conflicts with the main paths view and results in a phantomly selected favorite item
		ClearSelection();

		if (FavoriteSettings->FavoriteItems.IsEmpty())
		{
			// Favorite Paths
			FString FavoritePathsString;
			TArray<FString> NewFavoritePaths;
			if (GConfig->GetString(*IniSection, TEXT("FavoritePaths"), FavoritePathsString, IniFilename))
			{
				FavoritePathsString.ParseIntoArray(NewFavoritePaths, TEXT(","), /*bCullEmpty*/true);
			}

			// If the selected paths is empty, the path was "All assets"
			// This should handle that case properly
			FavoriteSettings->FavoriteItems.Reserve(NewFavoritePaths.Num());
			for (const FString& InvariantPath : NewFavoritePaths)
			{
				FStringView InvariantPathView(InvariantPath);
				InvariantPathView.TrimStartAndEndInline();
				if (!InvariantPathView.IsEmpty() && InvariantPathView != TEXT("None"))
				{
					ContentBrowserUtils::AddFavoriteFolder(FContentBrowserItemPath(InvariantPathView, EContentBrowserPathType::Internal));
				}
			}

			UE_LOGF(LogContentBrowser, Log, "Migrated %i editor favorites to project favorites", FavoriteSettings->FavoriteItems.Num())

			// Discard previous data
			GConfig->RemoveKey(*IniSection, TEXT("FavoritePaths"), IniFilename);
		}

		if (!FavoriteSettings->FavoriteItems.IsEmpty())
		{
			Populate();
		}
	}
}

TSharedPtr<FContentBrowserDataDragDropOp> SFavoritePathView::GetContentBrowserDragDropOpFromEvent(const FDragDropEvent& DragDropEvent) const
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (Operation.IsValid() && OnFolderFavoriteUpdated.IsBound())
	{
		if (Operation->IsOfType<FContentBrowserDataDragDropOp>())
		{
			TSharedPtr<FContentBrowserDataDragDropOp> DragDropOp = StaticCastSharedPtr<FContentBrowserDataDragDropOp>(Operation);

			// Only agree to the operation if the drag op only contains folders, since favorites cannot contain files. 
			if (DragDropOp &&
				!DragDropOp->GetDraggedFolders().IsEmpty() && 
				DragDropOp->GetDraggedFiles().IsEmpty())
			{
				return DragDropOp;
			}
		}
	}

	return {};
}

void SFavoritePathView::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	// If we don't have the appropriate drop content, indicate to the user that nothing will happen.
	if (!GetContentBrowserDragDropOpFromEvent(DragDropEvent))
	{
		DragDropEvent.GetOperation()->SetCursorOverride(EMouseCursor::SlashedCircle);
	}
}

void SFavoritePathView::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (Operation.IsValid())
	{
		Operation->SetCursorOverride(TOptional<EMouseCursor::Type>());
	}	
}

FReply SFavoritePathView::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FContentBrowserDataDragDropOp> DragDropOp = GetContentBrowserDragDropOpFromEvent(DragDropEvent);
	if (!DragDropOp.IsValid() || !TreeData.IsValid())
	{
		return FReply::Unhandled();
	}

	if (OnFolderFavoriteUpdated.IsBound())
	{
		TArray<FString> FolderPaths;
		FolderPaths.Reserve(DragDropOp->GetDraggedFolders().Num());
		for (const FContentBrowserItem& BrowserItem: DragDropOp->GetDraggedFolders())
		{
			const FName VirtualPath = BrowserItem.GetVirtualPath();
			if (!TreeData->FindTreeItem(VirtualPath))
			{
				FolderPaths.Add(VirtualPath.ToString());
			}
		}

		if (!FolderPaths.IsEmpty())
		{
			OnFolderFavoriteUpdated.Execute(FolderPaths, FString(), EItemDropZone::BelowItem);
		}
	}

	return FReply::Handled();
}

void SFavoritePathView::SetOnFolderFavoriteUpdated(const FOnFolderFavoriteUpdated& InOnFolderFavoriteUpdated)
{
	OnFolderFavoriteUpdated = InOnFolderFavoriteUpdated;
}

TSharedRef<ITableRow> SFavoritePathView::GenerateTreeRow(TSharedPtr<FTreeItem> TreeItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(TreeItem.IsValid());

	SAssetTreeItem::FArguments Arguments;
	Arguments
		.TreeItem(TreeItem)
		.OnNameChanged(this, &SFavoritePathView::FolderNameChanged)
		.OnVerifyNameChanged(this, &SFavoritePathView::VerifyFolderNameChanged)
		.IsItemExpanded(false)
		.HighlightText(this, &SFavoritePathView::GetHighlightText)
		.IsSelected(this, &SFavoritePathView::IsTreeItemSelected, TreeItem)
		.FontOverride(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetFontStyle("ContentBrowser.SourceTreeItemFont"));

	return
		SNew(STableRow<TSharedPtr<FTreeItem>>, OwnerTable)
		.OnDragDetected(this, &SFavoritePathView::OnFolderDragDetected)
		.OnCanAcceptDrop(this, &SFavoritePathView::OnFolderCanAcceptDrop)
		.OnAcceptDrop(this, &SFavoritePathView::OnFolderAcceptDrop)
		[
			SNew(SFavoriteAssetTreeItem)
			.Parent(Arguments)
		];
}

TOptional<EItemDropZone> SFavoritePathView::OnFolderCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<FTreeItem> InTreeItem)
{
	if (InItemDropZone == EItemDropZone::OntoItem)
	{
		return TOptional<EItemDropZone>();
	}

	TSharedPtr<FContentBrowserDataDragDropOp> DragDropOp = GetContentBrowserDragDropOpFromEvent(InDragDropEvent);
	if (!DragDropOp.IsValid() || !InTreeItem.IsValid() || !TreeData.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	const int32 TargetIndex = TreeData->GetVisibleRootItems()->Find(InTreeItem);
	if (TargetIndex == INDEX_NONE)
	{
		return TOptional<EItemDropZone>();
	}

	TSet<FContentBrowserItemKey> DisallowDropItemKeys
	{
		// Don't allow drop on itself
		FContentBrowserItemKey(InTreeItem->GetItem())
	};

	if (InItemDropZone == EItemDropZone::BelowItem)
	{
		/*
		 * If we drop onto or below a target item, make sure next item is not contained within dragged items, eg:
		 * - A <-
		 * - B
		 * - C
		 * - D
		 * If B,C are dragged, disallow dropping after A and before B, only before A and after D
		 */
		if (TreeData->GetVisibleRootItems()->IsValidIndex(TargetIndex + 1))
		{
			if (const TSharedPtr<FTreeItem> NextItem = (*TreeData->GetVisibleRootItems())[TargetIndex + 1])
			{
				const FContentBrowserItemKey NextItemKey(NextItem->GetItem());
				DisallowDropItemKeys.Add(NextItemKey);
			}
		}
	}
	else
	{
		/*
		 * If we drop above a target item, make sure prev item is not contained within dragged items, eg:
		 * - A
		 * - B
		 * - C
		 * - D <-
		 * If B,C are dragged, disallow dropping after C and before D, only before A and after D
		 */
		if (TreeData->GetVisibleRootItems()->IsValidIndex(TargetIndex - 1))
		{
			if (const TSharedPtr<FTreeItem> PrevItem = (*TreeData->GetVisibleRootItems())[TargetIndex - 1])
			{
				const FContentBrowserItemKey PrevItemKey(PrevItem->GetItem());
				DisallowDropItemKeys.Add(PrevItemKey);
			}
		}
	}

	const bool bDropZoneDisallowed = DragDropOp->GetDraggedFolders().ContainsByPredicate([&DisallowDropItemKeys](const FContentBrowserItem& InDraggedFolder)
		{
			return DisallowDropItemKeys.Contains(FContentBrowserItemKey(InDraggedFolder));
		});

	if (bDropZoneDisallowed)
	{
		return TOptional<EItemDropZone>();
	}

	return InItemDropZone;
}

FReply SFavoritePathView::OnFolderAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<FTreeItem> InTreeItem)
{
	TSharedPtr<FContentBrowserDataDragDropOp> DragDropOp = GetContentBrowserDragDropOpFromEvent(InDragDropEvent);
	if (!DragDropOp.IsValid() || !InTreeItem.IsValid())
	{
		return FReply::Unhandled();
	}

	if (OnFolderFavoriteUpdated.IsBound())
	{
		TArray<FString> FolderPaths;
		FolderPaths.Reserve(DragDropOp->GetDraggedFolders().Num());
		for (const FContentBrowserItem& BrowserItem: DragDropOp->GetDraggedFolders())
		{
			FolderPaths.Add(BrowserItem.GetVirtualPath().ToString());
		}

		const FString FolderContext = InTreeItem->GetItem().GetVirtualPath().ToString();
		OnFolderFavoriteUpdated.Execute(FolderPaths, FolderContext, InItemDropZone);
	}

	return FReply::Handled().EndDragDrop();
}

void SFavoritePathView::OnFavoriteAdded(const FContentBrowserItemPath&, bool)
{
	if (!bIsLoadingSettings)
	{
		Populate();
	}
}

void SFavoritePathView::HandleItemDataUpdated(TArrayView<const FContentBrowserItemDataUpdate> InUpdatedItems)
{
	if (InUpdatedItems.Num() == 0)
	{
		return;
	}

	if (!Algo::AnyOf(InUpdatedItems,
			[](const FContentBrowserItemDataUpdate& Update) { return Update.GetItemData().IsFolder(); }))
	{
		return;
	}

	TSet<FName> FavoritePaths;
	{
		const TArray<FString>& FavoritePathStrs = ContentBrowserUtils::GetFavoriteFolders();
		for (const FString& InvariantPath : FavoritePathStrs)
		{
			FName VirtualPath;
			IContentBrowserDataModule::Get().GetSubsystem()->ConvertInternalPathToVirtual(InvariantPath, VirtualPath);
			FavoritePaths.Add(VirtualPath);
		}
	}
	if (FavoritePaths.Num() == 0)
	{
		UE_LOGF(LogPathView, VeryVerbose, "[%ls] Skipping item data update because there were no favorites present", *WriteToString<256>(OwningContentBrowserName));
		return;
	}

	// Don't allow the selection changed delegate to be fired here
	FScopedPreventTreeItemChangedDelegate DelegatePrevention(SharedThis(this));
	const double HandleItemDataUpdatedStartTime = FPlatformTime::Seconds();

	// Limit the updates to only folders which are favorites
	TArray<FContentBrowserItemDataUpdate> FilteredUpdates;
	Algo::CopyIf(InUpdatedItems, FilteredUpdates, [FavoritePaths](const FContentBrowserItemDataUpdate& Update) {
		return FavoritePaths.Contains(Update.GetItemData().GetVirtualPath());
	});
	if (FilteredUpdates.Num())
	{
		TreeData->ProcessDataUpdates(MakeArrayView(FilteredUpdates), CreateCompiledFolderFilter());
	}

	// Update saved favorites
	for (const FContentBrowserItemDataUpdate& ItemDataUpdate : InUpdatedItems)
	{
		const FContentBrowserItemData& ItemData = ItemDataUpdate.GetItemData();
		if (!ItemData.IsFolder())
		{
			continue;
		}

		switch (ItemDataUpdate.GetUpdateType())
		{
			case EContentBrowserItemUpdateType::Added:
				break;
			case EContentBrowserItemUpdateType::Modified:
				break;
			case EContentBrowserItemUpdateType::Moved:
				ContentBrowserUtils::RemoveFavoriteFolder(
					FContentBrowserItemPath(ItemDataUpdate.GetPreviousVirtualPath(), EContentBrowserPathType::Virtual));
				break;
			case EContentBrowserItemUpdateType::Removed:
				ContentBrowserUtils::RemoveFavoriteFolder(
					FContentBrowserItemPath(ItemData.GetVirtualPath(), EContentBrowserPathType::Virtual));
				break;
			case EContentBrowserItemUpdateType::Unmounted:
				// Do not remove favorite because a plugin can be unmounted then mounted again later
				break;
			default:
				checkf(false, TEXT("Unexpected EContentBrowserItemUpdateType!"));
				break;
		}
	}

	UE_LOGF(LogPathView, VeryVerbose, "FavoritePathView - HandleItemDataUpdated completed in %0.4f seconds for %d items", FPlatformTime::Seconds() - HandleItemDataUpdatedStartTime, InUpdatedItems.Num());
}

void SFavoritePathView::FixupFavoritesFromExternalChange(TArrayView<const AssetViewUtils::FMovedContentFolder> MovedFolders)
{
	for (const AssetViewUtils::FMovedContentFolder& MovedFolder : MovedFolders)
	{
		FContentBrowserItemPath ItemPath(MovedFolder.Key, EContentBrowserPathType::Virtual);
		const bool bWasFavorite = ContentBrowserUtils::IsFavoriteFolder(ItemPath);
		if (bWasFavorite)
		{
			// Remove the original path
			ContentBrowserUtils::RemoveFavoriteFolder(ItemPath);

			// Add the new path to favorites instead
			const FString& NewPath = MovedFolder.Value;
			ContentBrowserUtils::AddFavoriteFolder(FContentBrowserItemPath(NewPath, EContentBrowserPathType::Virtual));
			TSharedPtr<FTreeItem> Item = TreeData->FindTreeItem(*NewPath);
			if (Item.IsValid())
			{
				TreeViewPtr->SetItemSelection(Item, true);
				TreeViewPtr->RequestScrollIntoView(Item);
			}
		}
	}
	Populate();
}

#undef LOCTEXT_NAMESPACE
