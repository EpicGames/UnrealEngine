// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetViewUtils.h"
#include "ContentBrowserConfig.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDataFilter.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserItem.h"
#include "Delegates/Delegate.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IContentBrowserSingleton.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Misc/TextFilter.h"
#include "PathViewTypes.h"
#include "SourcesViewWidgets.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateConstants.h"
#include "Types/SlateEnums.h"
#include "Types/SlateVector2.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

struct FContentBrowserFavoriteItem;
class FContentBrowserDataDragDropOp;
class FContentBrowserItemData;
class FContentBrowserItemDataUpdate;
class FContentBrowserPluginFilter;
class FPathPermissionList;
class FSourcesSearch;
class ITableRow;
class SWidget;
class UToolMenu;
struct FAssetData;
struct FFiltersAdditionalParams;
struct FGeometry;
struct FHistoryData;
struct FPathViewConfig;
struct FPathViewData;
struct FPointerEvent;
struct FContentBrowserInstanceConfig;

template<>
struct TWidgetTypeTraits<class SPathView>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * The tree view of folders which contain content.
 */
class SPathView
	: public SCompoundWidget
	, public IScrollableWidget
{
	SLATE_DECLARE_WIDGET(SPathView, SCompoundWidget)

public:
	/** Delegate for when plugin filters have changed */
	DECLARE_DELEGATE( FOnFrontendPluginFilterChanged );

	SLATE_BEGIN_ARGS( SPathView )
		: _bEnableShadowBoxStyle(false)
		, _InitialCategoryFilter(EContentBrowserItemCategoryFilter::IncludeAll)
		, _FocusSearchBoxWhenOpened(true)
		, _ShowTreeTitle(false)
		, _SearchBarVisibility(EVisibility::Visible)
		, _ShowSeparator(true)
		, _AllowContextMenu(true)
		, _AllowClassesFolder(false)
		, _AllowReadOnlyFolders(true)
		, _ShowFavorites(false)
		, _CanShowDevelopersFolder(false)
		, _ForceShowEngineContent(false)
		, _ForceShowPluginContent(false)
		, _ShowViewOptions(false)
		, _SelectionMode( ESelectionMode::Multi )
		{}

		/** Content displayed to the left of the search bar */
		SLATE_NAMED_SLOT( FArguments, SearchContent )

		/** Shadow box style used for the STreeView if bEnableShadowBoxStyle is true */
		SLATE_ARGUMENT(const FScrollBoxStyle*, ShadowBoxStyle)
		
		/** If true, overriding shadow box style of STreeView is enabled */
		SLATE_ARGUMENT(bool, bEnableShadowBoxStyle)

		/** Called when a tree paths was selected */
		SLATE_EVENT( FOnContentBrowserItemSelectionChanged, OnItemSelectionChanged )

		/** Called when a context menu is opening on a item */
		SLATE_EVENT( FOnGetContentBrowserItemContextMenu, OnGetItemContextMenu )

		/** Initial set of item categories that this view should show - may be adjusted further by things like AllowClassesFolder */
		SLATE_ARGUMENT( EContentBrowserItemCategoryFilter, InitialCategoryFilter )

		/** If true, the search box will be focus the frame after construction */
		SLATE_ARGUMENT( bool, FocusSearchBoxWhenOpened )

		/** If true, The tree title will be displayed */
		SLATE_ARGUMENT( bool, ShowTreeTitle )

		/** If EVisibility::Visible, The tree search bar will be displayed */
		SLATE_ATTRIBUTE( EVisibility, SearchBarVisibility )

		/** If true, The tree search bar separator be displayed */
		SLATE_ARGUMENT( bool, ShowSeparator )

		/** If false, the context menu will be suppressed */
		SLATE_ARGUMENT( bool, AllowContextMenu )

		/** If false, the classes folder will be suppressed */
		SLATE_ARGUMENT( bool, AllowClassesFolder )

		/** If true, read only folders will be displayed */
		SLATE_ARGUMENT( bool, AllowReadOnlyFolders )

		/** If true, the favorites expander will be displayed */
		SLATE_ARGUMENT(bool, ShowFavorites);

		/** Indicates if the 'Show Developers' option should be enabled or disabled */
		SLATE_ARGUMENT(bool, CanShowDevelopersFolder)

		/** Should always show engine content */
		SLATE_ARGUMENT(bool, ForceShowEngineContent)

		/** Should always show plugin content */
		SLATE_ARGUMENT(bool, ForceShowPluginContent)

		/** Should show the filter setting button. Note: If ExternalSearch is valid, then view options are not shown regardless of this setting */
		SLATE_ARGUMENT(bool, ShowViewOptions)

		/** If true, redirectors are taken into consideration when deciding if folders are empty */
		SLATE_ATTRIBUTE(bool, ShowRedirectors);

		/** The selection mode for the tree view */
		SLATE_ARGUMENT( ESelectionMode::Type, SelectionMode )

		/** Optional external search. Will hide and replace our internal search UI */
		SLATE_ARGUMENT( TSharedPtr<FSourcesSearch>, ExternalSearch )

		/** Optional Custom Folder permission list to be used to filter folders. */
		SLATE_ARGUMENT( TSharedPtr<FPathPermissionList>, CustomFolderPermissionList)

		/** The plugin filter collection */
		SLATE_ARGUMENT( TSharedPtr<FPluginFilterCollectionType>, PluginPathFilters)

		/** The instance name of the owning content browser. */
		SLATE_ARGUMENT( FName, OwningContentBrowserName )

		/** Default path to select use by path picker */
		SLATE_ARGUMENT(FString, DefaultPath)

		/** If DefaultPath doesn't exist, create it */
		SLATE_ARGUMENT(bool, CreateDefaultPath)

	SLATE_END_ARGS()

	SPathView();

	/** Destructor */
	~SPathView();

	/** Constructs this widget with InArgs */
	virtual void Construct( const FArguments& InArgs );

	/** Paint to poll attributes. This avoids the need for the widget to tick each frame */
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	/** Process the Commands of the PathView */
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

	/** Returns true if there are no items being presented in this widget. */
	virtual bool IsEmpty() const;

	/** Selects the closest matches to the supplied paths in the tree. "/" delimited */
	void SetSelectedPaths(const TArray<FName>& Paths);

	/** Selects the closest matches to the supplied paths in the tree. "/" delimited */
	void SetSelectedPaths(const TArray<FString>& Paths);

	/** Clears selection of all paths */
	void ClearSelection();

	/** Returns the first selected path in the tree view */
	FString GetSelectedPath() const;

	/** Returns all selected paths in the tree view */
	TArray<FString> GetSelectedPaths() const;

	/** Returns all the folder items currently selected in the view */
	TArray<FContentBrowserItem> GetSelectedFolderItems() const;

	/** Called when "new folder" is selected in the context menu */
	void NewFolderItemRequested(const FContentBrowserItemTemporaryContext& NewItemContext);

	/** Sets up an inline rename for the specified folder */
	void RenameFolderItem(const FContentBrowserItem& InItem);

	/**
	 * Selects the paths containing or corresponding to the specified items.
	 *
	 *	@param ItemsToSync			- A list of items to sync the view to
	 *
	 *	@param bAllowImplicitSync	- true to allow the view to sync to parent folders if they are already selected,
	 *								  false to force the view to select the explicit Parent folders of each asset
	 */
	void SyncToItems( TArrayView<const FContentBrowserItem> ItemsToSync, const bool bAllowImplicitSync = false );

	/** Sets the state of the path view to the one described by the history data */
	void ApplyHistoryData( const FHistoryData& History );

	/** Saves any settings to config that should be persistent between editor sessions */
	virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& InstanceName) const;

	/** Loads any settings to config that should be persistent between editor sessions */
	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& InstanceName);

	/**
	 * Return true if passes path block lists
	 * 
	 *	@param InInternalPath			- Internal Path (e.g. /Game)
	 *	@param InAlreadyCheckedDepth	- Folder depth that has already been checked, 0 if no parts of path already checked
	*/
	bool InternalPathPassesBlockLists(const FStringView InInternalPath, const int32 InAlreadyCheckedDepth = 0) const;

	/** 
	 * Disable any filters which would prevent us from syncing to the given items in the content browser.
	 * Returns true if any filtering changed. 
	 */
	bool DisablePluginPathFiltersThatHideItems(TConstArrayView<FContentBrowserItem> Items);

	/**
	 * Request a population of the path view the next time the widget is ticked.
	 */
	void RequestPopulation(FSimpleMulticastDelegate::FDelegate&& AfterPopulation = FSimpleMulticastDelegate::FDelegate());

	/**
	 * Populates the tree with all folders that are not filtered out
	 * Note: This can be very long in large project. prefer calling request population to avoid multiple population in one frame.
	 */
	virtual void Populate(const bool bIsRefreshingFilter = false);

	/** Sets an alternate tree title*/
	void SetTreeTitle(FText InTitle)
	{
		TreeTitle = MoveTemp(InTitle);
	};

	FText GetTreeTitle() const
	{
		return TreeTitle;
	}

	void PopulatePathViewFiltersMenu(UToolMenu* Menu);

	/** Get paths to select by default */
	TArray<FName> GetDefaultPathsToSelect() const;

	/** Get list of root path item names */
	TArray<FName> GetRootPathItemNames() const;	

	/** Get current item category filter enum */
	EContentBrowserItemCategoryFilter GetContentBrowserItemCategoryFilter() const;

	/** Get current item attribute filter enum */
	EContentBrowserItemAttributeFilter GetContentBrowserItemAttributeFilter() const;

	//~ Begin IScrollableWidget
	virtual FVector2D GetScrollDistance() override;
	virtual FVector2D GetScrollDistanceRemaining() override;
	virtual TSharedRef<SWidget> GetScrollWidget() override;
	//~ End IScrollableWidget

protected:
	/** Expands all parents of the specified item */
	void RecursiveExpandParents(const TSharedPtr<FTreeItem>& Item);

	/** Handles updating the view when content items are changed */
	virtual void HandleItemDataUpdated(TArrayView<const FContentBrowserItemDataUpdate> InUpdatedItems);

	/** Handles updating the view when content items are refreshed */
	void HandleItemDataRefreshed();

	/** Creates a list item for the tree view */
	virtual TSharedRef<ITableRow> GenerateTreeRow(TSharedPtr<FTreeItem> TreeItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Handles focusing a folder widget after it has been created with the intent to rename */
	void TreeItemScrolledIntoView(TSharedPtr<FTreeItem> TreeItem, const TSharedPtr<ITableRow>& Widget);

	/** Handler for tree view selection changes */
	void TreeSelectionChanged(TSharedPtr< FTreeItem > TreeItem, ESelectInfo::Type SelectInfo);

	/** Gets the content for a context menu */
	TSharedPtr<SWidget> MakePathViewContextMenu();

	/** Handler for returning a list of children associated with a particular tree node */
	void GetChildrenForTree(TSharedPtr< FTreeItem > TreeItem, TArray< TSharedPtr<FTreeItem> >& OutChildren);

	/** Handler for when a name was given to a new folder */
	void FolderNameChanged(const TSharedPtr< FTreeItem >& TreeItem, const FString& ProposedName, const UE::Slate::FDeprecateVector2DParameter& MessageLocation, const ETextCommit::Type CommitType);

	/** Handler used to verify the name of a new folder */
	bool VerifyFolderNameChanged(const TSharedPtr< FTreeItem >& TreeItem, const FString& ProposedName, FText& OutErrorMessage) const;

	/** Set the active filter text */
	void SetSearchFilterText(const FText& InSearchText, TArray<FText>& OutErrors);

	/** Gets the string to highlight in tree items (used in folder searching) */
	FText GetHighlightText() const;

	/** True if the specified item is selected in the asset tree */
	bool IsTreeItemSelected(TSharedPtr<FTreeItem> TreeItem) const;

	/** Handler for tree view folders are dragged */
	FReply OnFolderDragDetected(const FGeometry& Geometry, const FPointerEvent& MouseEvent);

	FContentBrowserDataCompiledFilter CreateCompiledFolderFilter() const;

	/** Get this path view's editor config if OwningContentBrowserName is set. */
	FPathViewConfig* GetPathViewConfig() const;

	/** Get this path view's content browser instance config if OwningContentBrowserName is set. */
	FContentBrowserInstanceConfig* GetContentBrowserConfig() const;

	virtual void ConfigureTreeView(STreeView<TSharedPtr<FTreeItem>>::FArguments& InArgs);

	/** Should be call after a populate */
	void SharedPostPopulate();

	constexpr virtual bool ShouldDoJustInTimePopulation() const
	{
		return true;
	}

private:
	/** Selects the given path only if it exists. Returns true if selected. */
	bool ExplicitlyAddPathToSelection(const FName Path);

	/** Returns true if the selection changed delegate should be allowed */
	bool ShouldAllowTreeItemChangedDelegate() const;

	/** Handler for recursively expanding/collapsing items in the tree view */
	void SetTreeItemExpansionRecursive( TSharedPtr< FTreeItem > TreeItem, bool bInExpansionState );

	/** Handler for tree view expansion changes */
	void TreeExpansionChanged( TSharedPtr< FTreeItem > TreeItem, bool bIsExpanded );

	/** Handler for when the search box filter has changed */
	void FilterUpdated();

	/** Returns true if the supplied folder item already exists in the tree. If so, ExistingItem will be set to the found item. */
	bool FolderAlreadyExists(const TSharedPtr< FTreeItem >& TreeItem, TSharedPtr< FTreeItem >& ExistingItem);

	/** True if the specified item is expanded in the asset tree */
	bool IsTreeItemExpanded(TSharedPtr<FTreeItem> TreeItem) const;

	/** Delegate called when an editor setting is changed */
	void HandleSettingChanged(FName PropertyName);

	/** Handle the logic that is done after population when requested by a setting change */
	void PostPopulateHandleSettingChanged(bool bHadSelectedPath);

	/** Handle the logic that is done after the initial population of the view */
	void PostInitialPopulation();

	/** Is the view waiting for a population of its tree */
	bool IsPopulationRequested() const;

	/** One-off active timer to focus the widget post-construct */
	EActiveTimerReturnType SetFocusPostConstruct(double InCurrentTime, float InDeltaTime);

	/** Sets the active state of a filter. */
	void SetPluginPathFilterActive(const TSharedRef<FContentBrowserPluginFilter>& Filter, bool bActive);

	/** Unchecks all plugin filters. */
	void ResetPluginPathFilters();

	/** Toggle plugin filter. */
	void PluginPathFilterClicked(TSharedRef<FContentBrowserPluginFilter> Filter);

	/** Return whether the given filter should be shown enabled in the UI. */
	bool IsPluginPathFilterChecked(TSharedRef<FContentBrowserPluginFilter> Filter) const;

	/** 
	 * Returns true if filter is being used. Note that because some filters are 'inverse' this is not the same as whether
	 * the filter should be visibly checked in the UI to be shown as 'enabled' to the user. 
	 */
	bool IsPluginPathFilterInUse(TSharedRef<FContentBrowserPluginFilter> Filter) const;

	TArray<FName> GetDefaultPathsToExpand() const;

	/** Tell the tree that the LastExpandedPath set should be refreshed */
	void DirtyLastExpandedPaths();

	/** Update the LastExpandedPath if required */
	void UpdateLastExpandedPathsIfDirty();

	/** Create a favorites view. */
	TSharedRef<SWidget> CreateFavoritesView();

	/** Register menu for when the view combo button is clicked */
	static void RegisterGetViewButtonMenu();

	/** Populate the given params for this PathView */
	void PopulateFilterAdditionalParams(FFiltersAdditionalParams& OutParams);

	/** Whether or not it's possible to show C++ content */
	bool IsToggleShowCppContentAllowed() const;

	/** Whether or not it's possible to toggle developers content */
	bool IsToggleShowDevelopersContentAllowed() const;

	/** Whether or not it's possible to toggle engine content */
	bool IsToggleShowEngineContentAllowed() const;

	/** Whether or not it's possible to toggle plugin content */
	bool IsToggleShowPluginContentAllowed() const;

	/** Whether or not it's possible to show localized content */
	bool IsToggleShowLocalizedContentAllowed() const;

	/** Handler for when the view combo button is clicked */
	TSharedRef<SWidget> GetViewButtonContent();

	/** Callback for the Copy command for the PathView */
	void CopySelectedFolder() const;

	/** Bind our UI commands */
	void BindCommands();

protected:
	/** A helper class to manage PreventTreeItemChangedDelegateCount by incrementing it when constructed (on the stack) and decrementing when destroyed */
	class FScopedPreventTreeItemChangedDelegate
	{
	public:
		FScopedPreventTreeItemChangedDelegate(const TSharedRef<SPathView>& InPathView)
			: PathView(InPathView)
		{
			PathView->PreventTreeItemChangedDelegateCount++;
		}

		~FScopedPreventTreeItemChangedDelegate()
		{
			check(PathView->PreventTreeItemChangedDelegateCount > 0);
			PathView->PreventTreeItemChangedDelegateCount--;
		}

	private:
		TSharedRef<SPathView> PathView;
	};

	/** A helper class to scope a selection change notification so that it only emits if the selection has actually changed after the scope ends */
	class FScopedSelectionChangedEvent
	{
	public:
		FScopedSelectionChangedEvent(const TSharedRef<SPathView>& InPathView, const bool InShouldEmitEvent = true);
		~FScopedSelectionChangedEvent();

		bool IsInitialSelectionEmpty() const;

	private:
		TSet<FName> GetSelectionSet() const;

		TSharedRef<SPathView> PathView;
		TSet<FName> InitialSelectionSet;
		bool bShouldEmitEvent = true;
	};

	/** The tree view widget */
	TSharedPtr< STreeView< TSharedPtr<FTreeItem>> > TreeViewPtr;

	/** The path view search interface */
	TSharedPtr<FSourcesSearch> SearchPtr;

	/** Items in the tree and associated data. Shared ptr for easy binding to delegates/attributes */
	TSharedPtr<FPathViewData> TreeData;

	/** Should this path tree be flat like the favorites tree */
	bool bFlat = false;

	/** The paths that were last reported by OnPathSelected event. Used in preserving selection when filtering folders */
	TSet<FName> LastSelectedPaths;

	/**
	 * If not empty, this is the path of the folders to sync once they are available while assets are still being discovered.
	 * It should be emptied when user interaction overides the default selections.
	 */
	TArray<FName> PendingInitialPaths;

	/** Context information for the folder item that is currently being created, if any */
	FContentBrowserItemTemporaryContext PendingNewFolderContext;

	TSharedPtr<SWidget> PathViewWidget;

	/** Permission filter to hide folders */
	TSharedPtr<FPathPermissionList> FolderPermissionList;

	/** Writable folder filter */
	TSharedPtr<FPathPermissionList> WritableFolderPermissionList;

	/** Custom Folder permissions */
	TSharedPtr<FPathPermissionList> CustomFolderPermissionList;

	/** The config instance to use. */
	FName OwningContentBrowserName;

	/** Event(s) called to run any queued logic after a population of the widget */
	FSimpleMulticastDelegate OnNextPopulate;

private:
	TSlateAttribute<bool> bShowRedirectorsAttribute;
	/** Last version number retrieved from TreeData so we can decide if the tree view need rebuilding */
	TSlateAttribute<uint64> TreeDataVersionAttribute;

	/** Used to track if the list of last expanded path should be updated */
	bool bLastExpandedPathsDirty = false;

	/** The paths that were last reported by OnPathExpanded event. Used in preserving expansion when filtering folders */
	TSet<FName> LastExpandedPaths;

	/** Delegate to invoke when selection changes. */
	FOnContentBrowserItemSelectionChanged OnItemSelectionChanged;

	/** Delegate to invoke when generating the context menu for an item */
	FOnGetContentBrowserItemContextMenu OnGetItemContextMenu;

	/** If > 0, the selection or expansion changed delegate will not be called. Used to update the tree from an external source or in certain bulk operations. */
	int32 PreventTreeItemChangedDelegateCount = 0;

	/** Initial set of item categories that this view should show - may be adjusted further by things like AllowClassesFolder */
	EContentBrowserItemCategoryFilter InitialCategoryFilter = EContentBrowserItemCategoryFilter::IncludeAll;

	/** If false, the context menu will not open when right clicking an item in the tree */
	bool bAllowContextMenu : 1;

	/** If false, the classes folder will not be added to the tree automatically */
	bool bAllowClassesFolder : 1;

	/** If true, read only folders will be displayed */
	bool bAllowReadOnlyFolders : 1;

	/** Indicates if the 'Show Developers' option should be enabled or disabled */
	bool bCanShowDevelopersFolder : 1;

	/** If true, engine content is always shown */
	bool bForceShowEngineContent : 1;

	/** If true, plugin content is always shown */
	bool bForceShowPluginContent : 1;
	
	/** Request a population of the path view */
	TSlateAttribute<bool> bPopulationRequestedAttribute;

	/** Request the initial population of the path view */
	TSlateAttribute<bool> bInitialPopulationRequestedAttribute;

	/** Request a population of the path view but because of a setting change */
	TSlateAttribute<bool> bPopulationForSettingChangeRequestedAttribute;

	bool bCreateDefaultPath : 1;
	FString DefaultPath;

	/** The title of this path view */
	FText TreeTitle;

	/** Commands handled by this widget */
	TSharedPtr<FUICommandList> Commands;

	/** The filter collection used to filter plugins */
	TSharedPtr<FPluginFilterCollectionType> PluginPathFilters;

	/** Plugins filters that are currently active */
	TArray< TSharedRef<FContentBrowserPluginFilter> > AllPluginPathFilters;

	/** The favorites path view if one is set. */
	TSharedPtr<SExpandableArea> FavoritesArea;
};

template<>
struct TWidgetTypeTraits<class SFavoriteAssetTreeItem>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** AssetTreeItem for the Favorite view to override some behavior */
class SFavoriteAssetTreeItem : public SAssetTreeItem
{
	SLATE_BEGIN_ARGS(SFavoriteAssetTreeItem){}
		SLATE_ARGUMENT(SAssetTreeItem::FArguments, Parent)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:
	//~ Begin SAssetTreeItem
	virtual FText GetNameText() const override;
	virtual FText GetToolTipText() const override;
	//~ End SAssetTreeItem

	//~ Begin SWidget
	virtual void OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& InDragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;
	//~ End SWidget

	void OnFavoriteUpdated(const FContentBrowserItemPath& InItemPath, bool bInAdded);
	void UpdateItem(const FContentBrowserItemPath& InPath);

private:
	/** Retrieves only drag operation that favorites view will handle */
	static TSharedPtr<FContentBrowserDataDragDropOp> GetFavoriteMoveOperation(const FDragDropEvent& InDragDropEvent);

	FText Alias;
	FText Shortcut;
};

template<>
struct TWidgetTypeTraits<class SFavoritePathView>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
* The tree view of folders which contain favorited folders.
*/
class SFavoritePathView : public SPathView
{
public:
	SFavoritePathView();
	virtual ~SFavoritePathView() override;
	
	/** Constructs this widget with InArgs */
	virtual void Construct(const FArguments& InArgs) override;

	virtual void Populate(const bool bIsRefreshingFilter = false) override;

	/** Loads any settings to config that should be persistent between editor sessions */
	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) override;

	/** Validate that the drop operation is valid for the favorites. Accept FAssetDragDropOp objects with folders only. */
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/** Restore any cursor overrides, in case once was set for invalid drop in OnDragEnter. */
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	
	/** Detect if a folder was dropped in the favorites view and add that folder (or folders) to favorites */
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/** Updates favorites based on an external change. */
	void FixupFavoritesFromExternalChange(TArrayView<const AssetViewUtils::FMovedContentFolder> MovedFolders);

	DECLARE_DELEGATE_ThreeParams(FOnFolderFavoriteUpdated, const TArray<FString>& /** UpdatedFolders */, const FString& /** FolderContext */, EItemDropZone /** PositionContext */)
	void SetOnFolderFavoriteUpdated(const FOnFolderFavoriteUpdated& InOnFolderFavoriteUpdated);

protected:
	constexpr virtual bool ShouldDoJustInTimePopulation() const override
	{
		return false;
	}

private:
	virtual TSharedRef<ITableRow> GenerateTreeRow(TSharedPtr<FTreeItem> TreeItem, const TSharedRef<STableViewBase>& OwnerTable) override;

	/** Handler for tree view to accept drop */
	TOptional<EItemDropZone> OnFolderCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<FTreeItem> InTreeItem);

	/** Handler for tree view when dropping items */
	FReply OnFolderAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<FTreeItem> InTreeItem);

	void OnFavoriteAdded(const FContentBrowserItemPath& ChangedItem, bool bAdded);

	/** Handles updating the view when content items are changed */
	virtual void HandleItemDataUpdated(TArrayView<const FContentBrowserItemDataUpdate> InUpdatedItems) override;

	virtual void ConfigureTreeView(STreeView<TSharedPtr<FTreeItem>>::FArguments& InArgs) override;

	/** Returns an FContentBrowserDataDragDropOp object but only if it qualifies as a proper droppable content browser drag drop op */
	TSharedPtr<FContentBrowserDataDragDropOp> GetContentBrowserDragDropOpFromEvent(const FDragDropEvent& DragDropEvent) const;

	TArray<FString> RemovedByFolderMove;
	FDelegateHandle OnFavoritesChangedHandle;

	FOnFolderFavoriteUpdated OnFolderFavoriteUpdated; 
	bool bIsLoadingSettings = false;
};
