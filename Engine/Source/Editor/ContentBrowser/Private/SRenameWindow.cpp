// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRenameWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "SRenameWindow"

/**
 * EditableTextBox used by the SRenameWindow
 */
class SRenameEditableTextBox : public SEditableTextBox
{
	SLATE_BEGIN_ARGS(SRenameEditableTextBox)
	{}

	SLATE_EVENT(FOnTextCommitted, OnTextCommitted)
	SLATE_EVENT(FOnVerifyTextChanged, OnVerifyTextChanged)
	SLATE_ARGUMENT(FText, InitialText)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<SWindow> InOwnerWindow)
	{
		OwnerWindow = InOwnerWindow;
		OnRenameCommittedDelegate = InArgs._OnTextCommitted;

		SEditableTextBox::Construct(SEditableTextBox::FArguments()
			.SelectAllTextWhenFocused(true)
			.OnVerifyTextChanged(InArgs._OnVerifyTextChanged)
			.Text(InArgs._InitialText)
			.OnTextCommitted(this, &SRenameEditableTextBox::OnRenameTextCommitted));
	}

	void EndRename(bool bForceRename = false)
	{
		if (!bOnTextCommittedCalled)
		{
			bOnTextCommittedCalled = true;
			OnRenameCommittedDelegate.ExecuteIfBound(GetText(), bForceRename ? ETextCommit::OnEnter : ETextCommit::OnCleared);
		}
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			EndRename();
			if (const TSharedPtr<SWindow>& Window = OwnerWindow.Pin())
			{
				Window->RequestDestroyWindow();
			}
			return FReply::Handled();
		}
		return SEditableTextBox::OnKeyDown(MyGeometry, InKeyEvent);
	}

private:
	void OnRenameTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
	{
		if (!bOnTextCommittedCalled)
		{
			bOnTextCommittedCalled = true;
			OnRenameCommittedDelegate.ExecuteIfBound(InText, InCommitType);
			if (const TSharedPtr<SWindow>& Window = OwnerWindow.Pin())
			{
				Window->RequestDestroyWindow();
			}
		}
	}

private:
	/** The OwnerWindow of this Widget */
	TWeakPtr<SWindow> OwnerWindow;

	/** Callback to call during the text OnCommitted callback */
	FOnTextCommitted OnRenameCommittedDelegate;

	/** Will be set to true once the OnCommitted is called once, there are cases which is called more time, but we want to call the passed callback just once */
	bool bOnTextCommittedCalled = false;
};

void SRenameWindow::Construct(const FArguments& InArgs)
{
	float DPIScale = 1.f;
	{
		IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
		const TSharedPtr<SWindow>& MainFrameParentWindow = MainFrameModule.GetParentWindow();
		if (MainFrameParentWindow.IsValid())
		{
			DPIScale = MainFrameParentWindow->GetDPIScaleFactor();
		}
	}

	const float AppScale = FSlateApplication::Get().GetApplicationScale();
	constexpr float RenameWindowMinWidth = 100.f;
	constexpr float RenameWindowFixedHeight = 30.f;

	SWindow::Construct(SWindow::FArguments()
		.SizingRule(ESizingRule::Autosized)
		.AutoCenter(EAutoCenter::None)
		.ScreenPosition(InArgs._ScreenPosition * (1.f/DPIScale))
		.Type(EWindowType::ToolTip)
		.LayoutBorder(0)
		.ActivationPolicy(EWindowActivationPolicy::FirstShown)
		.bDragAnywhere(false)
		.CreateTitleBar(false)
		.FocusWhenFirstShown(true)
		.MinWidth(RenameWindowMinWidth * AppScale * DPIScale)
		.MinHeight(RenameWindowFixedHeight * AppScale * DPIScale)
		.MaxHeight(RenameWindowFixedHeight * AppScale * DPIScale)
		[
			SAssignNew(RenameEditableTextBox, SRenameEditableTextBox, StaticCastSharedRef<SWindow>(AsShared()))
			.OnVerifyTextChanged(InArgs._OnVerifyTextChanged)
			.InitialText(InArgs._InitialText)
			.OnTextCommitted(InArgs._OnTextCommitted)
		]);

	// Once you open the SRenameWindow call the OnBeginText already since that is the moment where the Rename begin
	InArgs._OnBeginTextEdit.ExecuteIfBound(InArgs._InitialText);

	SetWidgetToFocusOnActivate(RenameEditableTextBox);
	GetOnWindowDeactivatedEvent().AddSP(this, &SRenameWindow::DeactivateWindow);
}

void SRenameWindow::DeactivateWindow()
{
	GetOnWindowDeactivatedEvent().RemoveAll(this);

	if (RenameEditableTextBox.IsValid())
	{
		// Always force it when deactivating the window
		// This will not rename when pressing ESC since the rename already happened with TextCommit = OnCleared
		RenameEditableTextBox->EndRename(true);
	}
	RequestDestroyWindow();
}

#undef LOCTEXT_NAMESPACE
