// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

class SRenameEditableTextBox;

/**
 * floating Rename Window
 */
class SRenameWindow : public SWindow
{
	SLATE_BEGIN_ARGS(SRenameWindow)
		: _ScreenPosition(FVector2D(0.f, 0.f))
		, _OnTextCommitted()
		, _OnVerifyTextChanged()
		, _OnBeginTextEdit()
		, _InitialText()
	{}

	SLATE_ARGUMENT(UE::Slate::FDeprecateVector2DParameter, ScreenPosition)

	SLATE_EVENT(FOnTextCommitted, OnTextCommitted)

	SLATE_EVENT(FOnVerifyTextChanged, OnVerifyTextChanged)

	SLATE_EVENT(FOnBeginTextEdit, OnBeginTextEdit)

	SLATE_ARGUMENT(FText, InitialText)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	void DeactivateWindow();

private:
	TSharedPtr<SRenameEditableTextBox> RenameEditableTextBox;
};
