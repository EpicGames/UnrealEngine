// Copyright Epic Games, Inc. All Rights Reserved.


#include "SThumbnailEditModeTools.h"

#include "AssetRegistry/AssetData.h"
#include "AssetThumbnail.h"
#include "AssetToolsModule.h"
#include "Containers/EnumAsByte.h"
#include "ContentBrowserStyle.h"
#include "Delegates/Delegate.h"
#include "Editor/UnrealEdEngine.h"
#include "GenericPlatform/ICursor.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ThumbnailRendering/SceneThumbnailInfoWithPrimitive.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "UnrealEdGlobals.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"

struct FGeometry;

#define LOCTEXT_NAMESPACE "ContentBrowser"

//////////////////////////////////////////////
// SThumbnailEditModeTools
//////////////////////////////////////////////

void SThumbnailEditModeTools::Construct( const FArguments& InArgs, const TSharedPtr<FAssetThumbnail>& InAssetThumbnail )
{
	AssetThumbnail = InAssetThumbnail;
	bModifiedThumbnailWhileDragging = false;
	DragStartLocation = FIntPoint(ForceInitToZero);
	bInSmallView = InArgs._SmallView;

	ChildSlot
	[
		SNew(SHorizontalBox)

		// Primitive tools
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Left)
		.Padding(1.f)
		[
			SNew(SButton)
			.Visibility(this, &SThumbnailEditModeTools::GetPrimitiveToolsVisibility)
			.ContentPadding(0.f)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SThumbnailEditModeTools::ChangePrimitive)
			.ToolTipText(LOCTEXT("CyclePrimitiveThumbnailShapes", "Cycle through primitive shape for this thumbnail"))
			.Content()
			[
				SNew(SImage).Image(this, &SThumbnailEditModeTools::GetCurrentPrimitiveBrush)
			]
		]
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Top)
		.Padding(1.f)
		[
			SNew(SButton)
			.Visibility(this, &SThumbnailEditModeTools::GetPrimitiveToolsResetToDefaultVisibility)
			.ContentPadding(0.f)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SThumbnailEditModeTools::ResetToDefault)
			.ToolTipText(LOCTEXT("ResetThumbnailToDefault", "Resets thumbnail to the default"))
			.Content()
			[
				SNew(SImage)
				.Image(UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetBrush("ContentBrowser.ResetPrimitiveToDefault"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];
}

EVisibility SThumbnailEditModeTools::GetPrimitiveToolsVisibility() const
{
	const bool bIsVisible = !bInSmallView && (GetSceneThumbnailInfo() != nullptr);
	return bIsVisible ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SThumbnailEditModeTools::GetPrimitiveToolsResetToDefaultVisibility() const
{
	USceneThumbnailInfo* ThumbnailInfo = GetSceneThumbnailInfo();
	
	EVisibility ResetToDefaultVisibility = EVisibility::Collapsed;
	if (ThumbnailInfo)
	{
		ResetToDefaultVisibility = ThumbnailInfo && ThumbnailInfo->DiffersFromDefault() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return ResetToDefaultVisibility;
}

const FSlateBrush* SThumbnailEditModeTools::GetCurrentPrimitiveBrush() const
{
	const FSlateStyleSet& ContentBrowserStyle = UE::ContentBrowser::Private::FContentBrowserStyle::Get();

	USceneThumbnailInfoWithPrimitive* ThumbnailInfo = GetSceneThumbnailInfoWithPrimitive();
	if ( ThumbnailInfo )
	{
		// Note this is for the icon only.  we are assuming the thumbnail renderer does the right thing when rendering
		EThumbnailPrimType PrimType = ThumbnailInfo->bUserModifiedShape ? ThumbnailInfo->PrimitiveType.GetValue() : (EThumbnailPrimType)ThumbnailInfo->DefaultPrimitiveType.Get(EThumbnailPrimType::TPT_Sphere);
		switch (PrimType)
		{
		case TPT_None: return ContentBrowserStyle.GetBrush("ContentBrowser.PrimitiveCustom");
		case TPT_Sphere: return ContentBrowserStyle.GetBrush("ContentBrowser.PrimitiveSphere");
		case TPT_Cube: return ContentBrowserStyle.GetBrush("ContentBrowser.PrimitiveCube");
		case TPT_Cylinder: return ContentBrowserStyle.GetBrush("ContentBrowser.PrimitiveCylinder");
		case TPT_Plane:
		default:
			// Fall through and return a plane
			break;
		}
	}

	return ContentBrowserStyle.GetBrush( "ContentBrowser.PrimitivePlane" );
}

FReply SThumbnailEditModeTools::ChangePrimitive()
{
	USceneThumbnailInfoWithPrimitive* ThumbnailInfo = GetSceneThumbnailInfoWithPrimitive();
	if ( ThumbnailInfo )
	{
		uint8 PrimitiveIdx = ThumbnailInfo->PrimitiveType.GetIntValue() + 1;
		if ( PrimitiveIdx >= TPT_MAX )
		{
			if ( ThumbnailInfo->PreviewMesh.IsValid() )
			{
				PrimitiveIdx = TPT_None;
			}
			else
			{
				PrimitiveIdx = TPT_None + 1;
			}
		}

		ThumbnailInfo->PrimitiveType = TEnumAsByte<EThumbnailPrimType>(PrimitiveIdx);
		ThumbnailInfo->bUserModifiedShape = true;

		if ( AssetThumbnail.IsValid() )
		{
			AssetThumbnail.Pin()->RefreshThumbnail();
		}

		ThumbnailInfo->MarkPackageDirty();
	}

	return FReply::Handled();
}

FReply SThumbnailEditModeTools::ResetToDefault()
{
	USceneThumbnailInfo* ThumbnailInfo = GetSceneThumbnailInfo();
	if (ThumbnailInfo)
	{
		ThumbnailInfo->ResetToDefault();

		if (AssetThumbnail.IsValid())
		{
			AssetThumbnail.Pin()->RefreshThumbnail();
		}

		ThumbnailInfo->MarkPackageDirty();

	}

	return FReply::Handled();
}

FReply SThumbnailEditModeTools::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( AssetThumbnail.IsValid() &&
		(MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton || MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		)
	{
		// Load the asset, unless it is in an unloaded map package or already loaded
		const FAssetData& AssetData = AssetThumbnail.Pin()->GetAssetData();
		
		// Getting the asset loads it, if it isn't already.
		UObject* Asset = AssetData.GetAsset();

		USceneThumbnailInfo* ThumbnailInfo = GetSceneThumbnailInfo();
		if ( ThumbnailInfo )
		{
			FThumbnailRenderingInfo* RenderInfo = GUnrealEd->GetThumbnailManager()->GetRenderingInfo(Asset);
			if (RenderInfo != nullptr && RenderInfo->Renderer != nullptr)
			{
				bModifiedThumbnailWhileDragging = false;
				DragStartLocation = FIntPoint(FMath::TruncToInt32(MouseEvent.GetScreenSpacePosition().X), FMath::TruncToInt32(MouseEvent.GetScreenSpacePosition().Y));

				return FReply::Handled().CaptureMouse(AsShared()).UseHighPrecisionMouseMovement(AsShared()).PreventThrottling();
			}
		}

		// This thumbnail does not have a scene thumbnail info but thumbnail editing is enabled. Just consume the input.
		return FReply::Handled();
	}
		
	return FReply::Unhandled();
}

FReply SThumbnailEditModeTools::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( HasMouseCapture() )
	{
		if ( bModifiedThumbnailWhileDragging )
		{
			USceneThumbnailInfo* ThumbnailInfo = GetSceneThumbnailInfo();
			if ( ThumbnailInfo )
			{
				ThumbnailInfo->MarkPackageDirty();
			}

			bModifiedThumbnailWhileDragging = false;
		}

		return FReply::Handled().ReleaseMouseCapture().SetMousePos(DragStartLocation);
	}

	return FReply::Unhandled();
}

FReply SThumbnailEditModeTools::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( HasMouseCapture() )
	{
		if ( !MouseEvent.GetCursorDelta().IsZero() )
		{
			USceneThumbnailInfo* ThumbnailInfo = GetSceneThumbnailInfo();
			if ( ThumbnailInfo )
			{
				const bool bLeftMouse = MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
				const bool bRightMouse = MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton);

				if ( bLeftMouse )
				{
					ThumbnailInfo->OrbitYaw += -MouseEvent.GetCursorDelta().X;
					ThumbnailInfo->OrbitPitch += -MouseEvent.GetCursorDelta().Y;

					// Normalize the values
					if ( ThumbnailInfo->OrbitYaw > 180 )
					{
						ThumbnailInfo->OrbitYaw -= 360;
					}
					else if ( ThumbnailInfo->OrbitYaw < -180 )
					{
						ThumbnailInfo->OrbitYaw += 360;
					}
					
					if ( ThumbnailInfo->OrbitPitch > 90 )
					{
						ThumbnailInfo->OrbitPitch = 90;
					}
					else if ( ThumbnailInfo->OrbitPitch < -90 )
					{
						ThumbnailInfo->OrbitPitch = -90;
					}
				}
				else if ( bRightMouse )
				{
					// Since zoom is a modifier of on the camera distance from the bounding sphere of the object, it is normalized in the thumbnail preview scene.
					ThumbnailInfo->OrbitZoom += MouseEvent.GetCursorDelta().Y;
				}

				// Dirty the package when the mouse is released
				bModifiedThumbnailWhileDragging = true;
			}
		}

		// Refresh the thumbnail. Do this even if the mouse did not move in case the thumbnail varies with time.
		if ( AssetThumbnail.IsValid() )
		{
			AssetThumbnail.Pin()->RefreshThumbnail();
		}

		return FReply::Handled().PreventThrottling();
	}

	return FReply::Unhandled();
}

FCursorReply SThumbnailEditModeTools::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	return HasMouseCapture() ? 
		FCursorReply::Cursor( EMouseCursor::None ) :
		FCursorReply::Cursor( EMouseCursor::Default );
}

USceneThumbnailInfo* SThumbnailEditModeTools::GetSceneThumbnailInfo() const
{
	USceneThumbnailInfo* SceneThumbnailInfo = SceneThumbnailInfoPtr.Get();
	
	if (!SceneThumbnailInfo)
	{
		if ( AssetThumbnail.IsValid() )
		{
			if ( UObject* Asset = AssetThumbnail.Pin()->GetAsset() )
			{
				static const FName AssetToolsName("AssetTools");
				FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(AssetToolsName);
				TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass( Asset->GetClass() );
				if ( AssetTypeActions.IsValid() )
				{
					SceneThumbnailInfo = Cast<USceneThumbnailInfo>(AssetTypeActions.Pin()->GetThumbnailInfo(Asset));
				}
			}
		}
	}

	return SceneThumbnailInfo;
}

USceneThumbnailInfoWithPrimitive* SThumbnailEditModeTools::GetSceneThumbnailInfoWithPrimitive() const
{
	return Cast<USceneThumbnailInfoWithPrimitive>( GetSceneThumbnailInfo() );
}

#undef LOCTEXT_NAMESPACE
