// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Math/IntPoint.h"
#include "Math/MathFwd.h"
#include "Templates/SharedPointer.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FAssetThumbnail;
class USceneThumbnailInfo;
class USceneThumbnailInfoWithPrimitive;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;

class SThumbnailEditModeTools : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SThumbnailEditModeTools)
		: _SmallView(false)
		{}
		
		SLATE_ARGUMENT( bool, SmallView )

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs, const TSharedPtr<FAssetThumbnail>& InAssetThumbnail );

	// SWidget Interface
	FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;

protected:
	/** Gets the visibility for the primitives toolbar */
	EVisibility GetPrimitiveToolsVisibility() const;

	/** Gets the visibility for the the thumbnail reset to default button */
	EVisibility GetPrimitiveToolsResetToDefaultVisibility() const;

	/** Gets the brush used to display the currently used primitive */
	const FSlateBrush* GetCurrentPrimitiveBrush() const;

	/** Sets the primitive type for the supplied thumbnail, if possible */
	FReply ChangePrimitive();

	/** Resets the primitive to default */
	FReply ResetToDefault();

	/** Helper accessors for ThumbnailInfo objects */
	USceneThumbnailInfo* GetSceneThumbnailInfo() const;
	USceneThumbnailInfoWithPrimitive* GetSceneThumbnailInfoWithPrimitive() const;

	EThumbnailPrimType GetDefaultThumbnailType() const;

protected:
	bool bInSmallView;
	bool bModifiedThumbnailWhileDragging;
	FIntPoint DragStartLocation;
	TWeakPtr<FAssetThumbnail> AssetThumbnail;

private:
	// Never access this directly.
	mutable TWeakObjectPtr<USceneThumbnailInfo> SceneThumbnailInfoPtr;
};
