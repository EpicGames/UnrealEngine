// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourcesSearch.h"

#include "Framework/Application/SlateApplication.h"
#include "Input/Events.h"
#include "Internationalization/Text.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SSearchBox.h"

void FSourcesSearch::Initialize()
{
	SearchBox = SNew(SSearchBox)
		.OnTextChanged(this, &FSourcesSearch::OnSearchBoxTextChanged)
		.OnTextCommitted(this, &FSourcesSearch::OnSearchBoxTextCommitted);
}

void FSourcesSearch::ClearSearch()
{
	if (!SearchBox->GetText().IsEmpty())
	{
		if (SearchBox->HasKeyboardFocus())
		{
			FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Cleared);
		}
		SearchBox->SetText(FText::GetEmpty());
	}
}

void FSourcesSearch::SetHintText(const TAttribute<FText>& InHintText)
{
	SearchBox->SetHintText(InHintText);
}

TSharedRef<SSearchBox> FSourcesSearch::GetWidget() const
{
	return SearchBox.ToSharedRef();
}

void FSourcesSearch::OnSearchBoxTextChanged(const FText& InSearchText)
{
	TArray<FText> SearchErrors;
	OnSearchChangedDelegate.Broadcast(InSearchText, SearchErrors);

	if (SearchErrors.Num() == 0)
	{
		SearchBox->SetError(FText::GetEmpty());
	}
	else if (SearchErrors.Num() == 1)
	{
		SearchBox->SetError(SearchErrors[0]);
	}
	else
	{
		FTextBuilder CombinedError;
		for (const FText& SearchError : SearchErrors)
		{
			CombinedError.AppendLine(SearchError);
		}
		SearchBox->SetError(CombinedError.ToText());
	}
}

void FSourcesSearch::OnSearchBoxTextCommitted(const FText& InSearchText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnCleared)
	{
		// Clear the search box
		ClearSearch();
		FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Cleared);
	}
}
