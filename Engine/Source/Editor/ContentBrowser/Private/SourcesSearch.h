// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"

class FText;
class SSearchBox;

/** Implementation for a sources search, used by the path and collections views */
class FSourcesSearch : public TSharedFromThis<FSourcesSearch>
{
public:
	/** Initialize the search widget */
	void Initialize();

	/** Delegate invoked when the search text is changed */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSearchChanged, const FText& /*InSearchText*/, TArray<FText>& /*OutErrors*/);
	FOnSearchChanged& OnSearchChanged()
	{
		return OnSearchChangedDelegate;
	}
	
	/** Clear the current search text */
	void ClearSearch();
	
	/** Set the active hint text for the underlying search widget */
	void SetHintText(const TAttribute<FText>& InHintText);

	/** Get the underlying search widget */
	TSharedRef<SSearchBox> GetWidget() const;

private:
	/** Handler for when search term changes */
	void OnSearchBoxTextChanged(const FText& InSearchText);

	/** Handler for when search term is committed */
	void OnSearchBoxTextCommitted(const FText& InSearchText, ETextCommit::Type InCommitType);

	/** Underlying search widget */
	TSharedPtr<SSearchBox> SearchBox;

	/** Delegate invoked when the search text is changed */
	FOnSearchChanged OnSearchChangedDelegate;
};
