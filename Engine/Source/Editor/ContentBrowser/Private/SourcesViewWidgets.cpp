// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourcesViewWidgets.h"

#include "AssetViewUtils.h"
#include "CollectionManagerTypes.h"
#include "CollectionViewTypes.h"
#include "CollectionViewUtils.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserItemData.h"
#include "ContentBrowserModule.h"
#include "ContentBrowserPluginFilters.h"
#include "ContentBrowserStyle.h"
#include "ContentBrowserUtils.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragAndDrop/CollectionDragDropOp.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "DragDropHandler.h"
#include "Fonts/SlateFontInfo.h"
#include "GenericPlatform/ICursor.h"
#include "IContentBrowserDataModule.h"
#include "Input/DragAndDrop.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "Misc/PathViews.h"
#include "PathViewTypes.h"
#include "SAssetTagItem.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

struct FSlateBrush;

#define LOCTEXT_NAMESPACE "ContentBrowser"

struct FAssetTreeItemBrushes
{
	/** Brushes for the different folder states */
	const FSlateBrush* FolderOpenBrush;
	const FSlateBrush* FolderClosedBrush;
	const FSlateBrush* FolderOpenVirtualBrush;
	const FSlateBrush* FolderClosedVirtualBrush;
	const FSlateBrush* FolderOpenCodeBrush;
	const FSlateBrush* FolderClosedCodeBrush;
	const FSlateBrush* FolderOpenDeveloperBrush;
	const FSlateBrush* FolderClosedDeveloperBrush;
	const FSlateBrush* FolderOpenPluginRootBrush;
	const FSlateBrush* FolderClosedPluginRootBrush;

	FAssetTreeItemBrushes()
	{
		const FSlateStyleSet& ContentBrowserStyle = UE::ContentBrowser::Private::FContentBrowserStyle::Get();

		FolderOpenBrush = ContentBrowserStyle.GetBrush("ContentBrowser.AssetTreeFolderOpen");
		FolderClosedBrush = ContentBrowserStyle.GetBrush("ContentBrowser.AssetTreeFolderClosed");
		FolderOpenVirtualBrush = ContentBrowserStyle.GetBrush("ContentBrowser.AssetTreeFolderOpenVirtual");
		FolderClosedVirtualBrush = ContentBrowserStyle.GetBrush("ContentBrowser.AssetTreeFolderClosedVirtual");
		FolderOpenCodeBrush = ContentBrowserStyle.GetBrush("ContentBrowser.AssetTreeFolderOpenCode");
		FolderClosedCodeBrush = ContentBrowserStyle.GetBrush("ContentBrowser.AssetTreeFolderClosedCode");
		FolderOpenDeveloperBrush = ContentBrowserStyle.GetBrush("ContentBrowser.AssetTreeFolderOpenDeveloper");
		FolderClosedDeveloperBrush = ContentBrowserStyle.GetBrush("ContentBrowser.AssetTreeFolderClosedDeveloper");
		FolderOpenPluginRootBrush = ContentBrowserStyle.GetBrush("ContentBrowser.AssetTreeFolderOpenPluginRoot");
		FolderClosedPluginRootBrush = ContentBrowserStyle.GetBrush("ContentBrowser.AssetTreeFolderClosedPluginRoot");
	}
	
	static FAssetTreeItemBrushes& Get()
	{
		static FAssetTreeItemBrushes Instance;
		return Instance;
	}
};



//////////////////////////
// SAssetTreeItem
//////////////////////////

void SAssetTreeItem::Construct( const FArguments& InArgs )
{
	TreeItem = InArgs._TreeItem;
	OnNameChanged = InArgs._OnNameChanged;
	OnVerifyNameChanged = InArgs._OnVerifyNameChanged;
	IsItemExpanded = InArgs._IsItemExpanded;
	bDraggedOver = false;

	IsSelected = InArgs._IsSelected;

	FolderType = EFolderType::Normal;
	const FContentBrowserItem& Item = InArgs._TreeItem->GetItem();
	if (ContentBrowserUtils::IsItemDeveloperContent(Item))
	{
		FolderType = EFolderType::Developer;
	}
	else if (EnumHasAnyFlags(Item.GetItemCategory(), EContentBrowserItemFlags::Category_Class))
	{
		FolderType = EFolderType::Code;
	}
	else
	{
		if (ContentBrowserUtils::ShouldShowPluginFolderIcon() && Item.IsInPlugin())
		{
			TSharedPtr<FTreeItem> Parent = InArgs._TreeItem->GetParent();
			if (!Parent.IsValid() || !Parent->GetItem().IsInPlugin())
			{
				FolderType = EFolderType::PluginRoot;
			}
			else
			{
				FolderType = EFolderType::Normal;
			}
		}
		else if (ContentBrowserUtils::ShouldShowCustomVirtualFolderIcon())
		{
			FContentBrowserItemDataAttributeValue VirtualAttributeValue = Item.GetItemAttribute(ContentBrowserItemAttributes::ItemIsCustomVirtualFolder);
			if (VirtualAttributeValue.IsValid() && VirtualAttributeValue.GetValue<bool>())
			{
				FolderType = EFolderType::CustomVirtual;
			}
		}
	}

	bool bIsRoot = !InArgs._TreeItem->GetParent().IsValid();

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(this, &SAssetTreeItem::GetBorderImage)
		.Padding( FMargin( 0, bIsRoot ? 3 : 0, 0, 0 ) )	// For root items in the tree, give them a little breathing room on the top
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 2, 0)
			.VAlign(VAlign_Center)
			[
				// Folder Icon
				SNew(SImage) 
				.Image(this, &SAssetTreeItem::GetFolderIcon)
				.ColorAndOpacity(this, &SAssetTreeItem::GetFolderColor)
			]

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SAssignNew(InlineRenameWidget, SInlineEditableTextBlock)
					.Text(this, &SAssetTreeItem::GetNameText)
					.ToolTipText(this, &SAssetTreeItem::GetToolTipText)
					.Font( InArgs._FontOverride.IsSet() ? InArgs._FontOverride : UE::ContentBrowser::Private::FContentBrowserStyle::Get().GetFontStyle(bIsRoot ? "ContentBrowser.SourceTreeRootItemFont" : "ContentBrowser.SourceTreeItemFont") )
					.HighlightText( InArgs._HighlightText )
					.OnTextCommitted(this, &SAssetTreeItem::HandleNameCommitted)
					.OnVerifyTextChanged(this, &SAssetTreeItem::VerifyNameChanged)
					.IsSelected( InArgs._IsSelected )
					.IsReadOnly( this, &SAssetTreeItem::IsReadOnly )
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				GenerateStateIcons()
			]
		]
	];

	if( InlineRenameWidget.IsValid() )
	{
		if ( const TSharedPtr<FTreeItem> TreeItemPtr = TreeItem.Pin() )
		{
			EnterEditingModeDelegateHandle = TreeItemPtr->OnRenameRequested().AddSP( InlineRenameWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode );
		}
	}
}

TSharedRef<SWidget> SAssetTreeItem::GenerateStateIcons()
{
	TSharedRef<SBox> ContainingBox = SNew(SBox);
	TSharedPtr<SHorizontalBox> HorizonalBox;
	FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>( TEXT("ContentBrowser") );
	if (const TSharedPtr<FTreeItem> TreeItemPinned = TreeItem.Pin())
	{
		const FContentBrowserItem& ContentBrowserItem = TreeItemPinned->GetItem();
		for(const FPathViewStateIconGenerator& Generator: ContentBrowserModule.GetAllPathViewStateIconGenerators())
		{
			if (Generator.IsBound())
			{
				if (TSharedPtr<SWidget> IconWidget = Generator.Execute(ContentBrowserItem))
				{
					if (!HorizonalBox)
					{
						HorizonalBox = SNew(SHorizontalBox);
					}
					
					HorizonalBox->AddSlot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4, 0, 0, 0)
					[
						IconWidget.ToSharedRef()
					];
				}
			}
		}
		// If we created any content, add it to the containg box
		// and set padding
		if (HorizonalBox)
		{
			ContainingBox->SetContent(HorizonalBox.ToSharedRef());
			ContainingBox->SetPadding(FMargin(2, 0, 4, 0));
		}
	}
	return ContainingBox;
}

SAssetTreeItem::~SAssetTreeItem()
{
	if( InlineRenameWidget.IsValid() )
	{
		if ( const TSharedPtr<FTreeItem> TreeItemPtr = TreeItem.Pin() )
		{
			TreeItemPtr->OnRenameRequested().Remove( EnterEditingModeDelegateHandle );
		}
	}
}

void SAssetTreeItem::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FTreeItem> TreeItemPinned = TreeItem.Pin();
	bDraggedOver = TreeItemPinned && DragDropHandler::HandleDragEnterItem(TreeItemPinned->GetItem(), DragDropEvent);
}

void SAssetTreeItem::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	if (TSharedPtr<FTreeItem> TreeItemPinned = TreeItem.Pin())
	{
		DragDropHandler::HandleDragLeaveItem(TreeItemPinned->GetItem(), DragDropEvent);
	}
	bDraggedOver = false;
}

FReply SAssetTreeItem::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FTreeItem> TreeItemPinned = TreeItem.Pin();
	bDraggedOver = TreeItemPinned && DragDropHandler::HandleDragOverItem(TreeItemPinned->GetItem(), DragDropEvent);
	return (bDraggedOver) ? FReply::Handled() : FReply::Unhandled();
}

FReply SAssetTreeItem::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FTreeItem> TreeItemPinned = TreeItem.Pin();
	if (TreeItemPinned && DragDropHandler::HandleDragDropOnItem(TreeItemPinned->GetItem(), DragDropEvent, AsShared()))
	{
		bDraggedOver = false;
		return FReply::Handled();
	}

	if (bDraggedOver)
	{
		// We were able to handle this operation, but could not due to another error - still report this drop as handled so it doesn't fall through to other widgets
		bDraggedOver = false;
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SAssetTreeItem::VerifyNameChanged(const FText& InName, FText& OutError) const
{
	if ( TSharedPtr<FTreeItem> TreeItemPtr = TreeItem.Pin() )
	{
		if(OnVerifyNameChanged.IsBound())
		{
			return OnVerifyNameChanged.Execute(TreeItemPtr, InName.ToString(), OutError);
		}
	}

	return true;
}

void SAssetTreeItem::HandleNameCommitted( const FText& NewText, ETextCommit::Type CommitInfo )
{
	if ( const TSharedPtr<FTreeItem> TreeItemPtr = TreeItem.Pin() )
	{
		if ( TreeItemPtr->IsNamingFolder() )
		{
			TreeItemPtr->SetNamingFolder(false);
		
			const FGeometry LastGeometry = GetTickSpaceGeometry();
			FVector2D MessageLoc;
			MessageLoc.X = LastGeometry.AbsolutePosition.X;
			MessageLoc.Y = LastGeometry.AbsolutePosition.Y + LastGeometry.Size.Y * LastGeometry.Scale;
		
			OnNameChanged.ExecuteIfBound(TreeItemPtr, NewText.ToString(), MessageLoc, CommitInfo);
		}
	}
}

bool SAssetTreeItem::IsReadOnly() const
{
	if ( const TSharedPtr<FTreeItem> TreeItemPtr = TreeItem.Pin() )
	{
		return !TreeItemPtr->IsNamingFolder();
	}
	else
	{
		return true;
	}
}

const FSlateBrush* SAssetTreeItem::GetFolderIcon() const
{
	const bool bIsItemExpanded = IsItemExpanded.Get(false);
	// First see if this item has a custom implementation
	if (TSharedPtr<FTreeItem> TreeItemPin = TreeItem.Pin())
	{
		FContentBrowserItemDataAttributeValue FolderAttributeValue = TreeItemPin->GetItem().GetItemAttribute(bIsItemExpanded ? ContentBrowserItemAttributes::ItemAssetTreeFolderOpenBrush : ContentBrowserItemAttributes::ItemAssetTreeFolderClosedBrush);
		if (FolderAttributeValue.IsValid())
		{
			FName BrushName = FolderAttributeValue.GetValue<FName>();

			const FSlateStyleSet& ContentBrowserStyle = UE::ContentBrowser::Private::FContentBrowserStyle::Get();
			return ContentBrowserStyle.GetBrush(BrushName);
		}
	}

	// If we made it here, nothing overrode it, so use the default brushes.
	FAssetTreeItemBrushes& Brushes = FAssetTreeItemBrushes::Get();
	switch( FolderType )
	{
	case EFolderType::Code:
		return bIsItemExpanded ? Brushes.FolderOpenCodeBrush : Brushes.FolderClosedCodeBrush;

	case EFolderType::Developer:
		return bIsItemExpanded ? Brushes.FolderOpenDeveloperBrush : Brushes.FolderClosedDeveloperBrush;

	case EFolderType::CustomVirtual:
		return bIsItemExpanded ? Brushes.FolderOpenVirtualBrush : Brushes.FolderClosedVirtualBrush;
	case EFolderType::PluginRoot:
		return bIsItemExpanded ? Brushes.FolderOpenPluginRootBrush : Brushes.FolderClosedPluginRootBrush;

	default:
		return bIsItemExpanded ? Brushes.FolderOpenBrush : Brushes.FolderClosedBrush;
	}
}

FSlateColor SAssetTreeItem::GetFolderColor() const
{
	if (TSharedPtr<FTreeItem> TreeItemPin = TreeItem.Pin())
	{
		FLinearColor FoundColor;
		FContentBrowserItemDataAttributeValue ColorAttributeValue = TreeItemPin->GetItem().GetItemAttribute(ContentBrowserItemAttributes::ItemColor);
		if (ColorAttributeValue.IsValid())
		{
			const FString ColorStr = ColorAttributeValue.GetValue<FString>();

			FLinearColor Color;
			if (Color.InitFromString(ColorStr))
			{
				FoundColor = Color;
			}
		}
		else
		{
			TOptional<FLinearColor> Color = ContentBrowserUtils::GetPathColor(TreeItemPin->GetItem().GetInvariantPath().ToString());
			if (Color.IsSet())
			{
				FoundColor = Color.GetValue();
			}
		}

		return FoundColor;
	}

	return ContentBrowserUtils::GetDefaultColor();
}

FText SAssetTreeItem::GetNameText() const
{
	if (TSharedPtr<FTreeItem> TreeItemPin = TreeItem.Pin())
	{
		return TreeItemPin->GetItem().GetDisplayName();
	}
	return FText();
}

FText SAssetTreeItem::GetToolTipText() const
{
	if (TSharedPtr<FTreeItem> TreeItemPin = TreeItem.Pin())
	{
		// If this item is a plugin folder, append the plugin description to the tooltip
		const FContentBrowserItem& Item = TreeItemPin->GetItem();
		FText PathText = IContentBrowserDataModule::Get().GetSubsystem()->ConvertVirtualPathToDisplay(Item);
		if (Item.IsInPlugin())
		{
			FNameBuilder ItemPath{Item.GetInternalPath()};
			FStringView PluginName = FPathViews::GetMountPointNameFromPath(ItemPath.ToView());
			if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName))
			{
				const FPluginDescriptor& Descriptor = Plugin->GetDescriptor();
				if (!Descriptor.Description.IsEmpty())
				{
					return FText::Format(LOCTEXT("TwoLineTooltip", "{0}\n{1}"),
						PathText,
						FText::FromString(Descriptor.Description)
						);
				}
			}
		}
		return PathText;
	}
	return FText();
}

const FSlateBrush* SAssetTreeItem::GetBorderImage() const
{
	static const FName NAME_DraggedBorderImage = TEXT("Menu.Background");
	static const FName NAME_NoBorderImage = TEXT("NoBorder");
	return bDraggedOver ? FAppStyle::GetBrush(NAME_DraggedBorderImage) : FAppStyle::GetBrush(NAME_NoBorderImage);
}



//////////////////////////
// SCollectionTreeItem
//////////////////////////

void SCollectionTreeItem::Construct( const FArguments& InArgs )
{
	ParentWidget = InArgs._ParentWidget;
	CollectionItem = InArgs._CollectionItem;
	OnBeginNameChange = InArgs._OnBeginNameChange;
	OnNameChangeCommit = InArgs._OnNameChangeCommit;
	OnVerifyRenameCommit = InArgs._OnVerifyRenameCommit;
	OnValidateDragDrop = InArgs._OnValidateDragDrop;
	OnHandleDragDrop = InArgs._OnHandleDragDrop;
	bDraggedOver = false;

	TSharedPtr<SAssetTagItem> AssetTagItem;

	ChildSlot
	[
		SAssignNew(AssetTagItem, SAssetTagItem)
		.ViewMode(EAssetTagItemViewMode::Compact)
		.BaseColor(this, &SCollectionTreeItem::GetCollectionColor)
		.DisplayName(this, &SCollectionTreeItem::GetNameText)
		.CountText(this, &SCollectionTreeItem::GetCollectionObjectCountText)
		.WarningText(this, &SCollectionTreeItem::GetCollectionWarningText)
		.HighlightText(InArgs._HighlightText)
		.OnBeginNameEdit(this, &SCollectionTreeItem::HandleBeginNameChange)
		.OnNameCommitted(this, &SCollectionTreeItem::HandleNameCommitted)
		.OnVerifyName(this, &SCollectionTreeItem::HandleVerifyNameChanged)
		.IsSelected(InArgs._IsSelected)
		.IsNameReadOnly(InArgs._IsReadOnly)
		.IsCheckBoxEnabled(InArgs._IsCheckBoxEnabled)
		.IsChecked(InArgs._IsCollectionChecked)
		.OnCheckStateChanged(InArgs._OnCollectionCheckStateChanged)
		.OnBuildToolTipInfo(this, &SCollectionTreeItem::BuildToolTipInfo)
	];

	// This is broadcast when the context menu / input binding requests a rename
	InArgs._CollectionItem->OnRenamedRequestEvent.AddSP(AssetTagItem.Get(), &SAssetTagItem::RequestRename);
}

void SCollectionTreeItem::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	// Cache this widget's geometry so it can pop up warnings over itself
	CachedGeometry = AllottedGeometry;
}

bool SCollectionTreeItem::ValidateDragDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool& OutIsKnownDragOperation ) const
{
	OutIsKnownDragOperation = false;

	TSharedPtr<FCollectionItem> CollectionItemPtr = CollectionItem.Pin();
	if (OnValidateDragDrop.IsBound() && CollectionItemPtr.IsValid())
	{
		return OnValidateDragDrop.Execute( CollectionItemPtr.ToSharedRef(), MyGeometry, DragDropEvent, OutIsKnownDragOperation );
	}

	return false;
}

void SCollectionTreeItem::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	ValidateDragDrop(MyGeometry, DragDropEvent, bDraggedOver); // updates bDraggedOver
}

void SCollectionTreeItem::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (Operation.IsValid())
	{
		Operation->SetCursorOverride(TOptional<EMouseCursor::Type>());

		if (Operation->IsOfType<FCollectionDragDropOp>() || Operation->IsOfType<FAssetDragDropOp>())
		{
			TSharedPtr<FDecoratedDragDropOp> DragDropOp = StaticCastSharedPtr<FDecoratedDragDropOp>(Operation);
			DragDropOp->ResetToDefaultToolTip();
		}
	}

	bDraggedOver = false;
}

FReply SCollectionTreeItem::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	ValidateDragDrop(MyGeometry, DragDropEvent, bDraggedOver); // updates bDraggedOver
	return (bDraggedOver) ? FReply::Handled() : FReply::Unhandled();
}

FReply SCollectionTreeItem::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if (ValidateDragDrop(MyGeometry, DragDropEvent, bDraggedOver) && OnHandleDragDrop.IsBound()) // updates bDraggedOver
	{
		bDraggedOver = false;
		return OnHandleDragDrop.Execute( CollectionItem.Pin().ToSharedRef(), MyGeometry, DragDropEvent );
	}

	if (bDraggedOver)
	{
		// We were able to handle this operation, but could not due to another error - still report this drop as handled so it doesn't fall through to other widgets
		bDraggedOver = false;
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SCollectionTreeItem::HandleBeginNameChange( const FText& OldText )
{
	TSharedPtr<FCollectionItem> CollectionItemPtr = CollectionItem.Pin();

	if ( CollectionItemPtr.IsValid() )
	{
		// If we get here via a context menu or input binding, bRenaming will already be set on the item.
		// If we got here by double clicking the editable text field, we need to set it now.
		CollectionItemPtr->bRenaming = true;

		OnBeginNameChange.ExecuteIfBound( CollectionItemPtr );
	}
}

void SCollectionTreeItem::HandleNameCommitted( const FText& NewText, ETextCommit::Type CommitInfo )
{
	TSharedPtr<FCollectionItem> CollectionItemPtr = CollectionItem.Pin();

	if ( CollectionItemPtr.IsValid() )
	{
		if ( CollectionItemPtr->bRenaming )
		{
			CollectionItemPtr->bRenaming = false;

			if ( OnNameChangeCommit.IsBound() )
			{
				FText WarningMessage;
				bool bIsCommitted = (CommitInfo != ETextCommit::OnCleared);
				if ( !OnNameChangeCommit.Execute(CollectionItemPtr, NewText.ToString(), bIsCommitted, WarningMessage) && bIsCommitted )
				{
					if (TSharedPtr<SWidget> PinnedParentWidget = ParentWidget.Pin())
					{
						// Failed to rename/create a collection, display a warning.
						ContentBrowserUtils::DisplayMessage(WarningMessage, CachedGeometry.GetLayoutBoundingRect(), PinnedParentWidget.ToSharedRef(), ContentBrowserUtils::EDisplayMessageType::Error);
					}
				}
			}				
		}
	}
}

bool SCollectionTreeItem::HandleVerifyNameChanged( const FText& NewText, FText& OutErrorMessage )
{
	TSharedPtr<FCollectionItem> CollectionItemPtr = CollectionItem.Pin();

	if (CollectionItemPtr.IsValid())
	{
		return !OnVerifyRenameCommit.IsBound() || OnVerifyRenameCommit.Execute(CollectionItemPtr, NewText.ToString(), CachedGeometry.GetLayoutBoundingRect(), OutErrorMessage);
	}

	return true;
}

FText SCollectionTreeItem::GetNameText() const
{
	TSharedPtr<FCollectionItem> CollectionItemPtr = CollectionItem.Pin();

	if ( CollectionItemPtr.IsValid() )
	{
		return FText::FromName(CollectionItemPtr->CollectionName);
	}
	else
	{
		return FText();
	}
}

FLinearColor SCollectionTreeItem::GetCollectionColor() const
{
	TSharedPtr<FCollectionItem> CollectionItemPtr = CollectionItem.Pin();

	if ( CollectionItemPtr.IsValid() )
	{
		return CollectionItemPtr->CollectionColor;
	}
	
	return CollectionViewUtils::GetDefaultColor();
}

FText SCollectionTreeItem::GetCollectionObjectCountText() const
{
	TSharedPtr<FCollectionItem> CollectionItemPtr = CollectionItem.Pin();

	if (CollectionItemPtr)
	{
		if (CollectionItemPtr->StorageMode == ECollectionStorageMode::Static)
		{
			return CollectionItemPtr->NumObjects > 999
				? NSLOCTEXT("ContentBrowser", "999+", "999+")
				: FText::AsNumber(CollectionItemPtr->NumObjects);
		}

		return NSLOCTEXT("ContentBrowser", "InfinitySymbol", "\u221E");
	}

	return FText::GetEmpty();
}

FText SCollectionTreeItem::GetCollectionWarningText() const
{
	TSharedPtr<FCollectionItem> CollectionItemPtr = CollectionItem.Pin();

	if (CollectionItemPtr)
	{
		switch (CollectionItemPtr->CurrentStatus)
		{
		case ECollectionItemStatus::IsOutOfDate:
			return NSLOCTEXT("ContentBrowser", "CollectionStatus_IsOutOfDate", "Collection is not at the latest revision");

		case ECollectionItemStatus::IsCheckedOutByAnotherUser:
			return NSLOCTEXT("ContentBrowser", "CollectionStatus_IsCheckedOutByAnotherUser", "Collection is checked out by another user");

		case ECollectionItemStatus::IsConflicted:
			return NSLOCTEXT("ContentBrowser", "CollectionStatus_IsConflicted", "Collection is conflicted - please use your external revision control provider to resolve this conflict");

		case ECollectionItemStatus::IsMissingSCCProvider:
			return NSLOCTEXT("ContentBrowser", "CollectionStatus_IsMissingSCCProvider", "Collection is missing its revision control provider - please check your revision control settings");

		case ECollectionItemStatus::HasLocalChanges:
			return NSLOCTEXT("ContentBrowser", "CollectionStatus_HasLocalChanges", "Collection has local unsaved or uncomitted changes");

		default:
			break;
		}
	}

	return FText::GetEmpty();
}

void SCollectionTreeItem::BuildToolTipInfo(const FOnBuildAssetTagItemToolTipInfoEntry& InCallback) const
{
	TSharedPtr<FCollectionItem> CollectionItemPtr = CollectionItem.Pin();

	if (CollectionItemPtr)
	{
		InCallback(LOCTEXT("CollectionShareTypeKey", "Share Type"), ECollectionShareType::ToText(CollectionItemPtr->CollectionType));
		InCallback(LOCTEXT("CollectionStorageModeKey", "Storage Mode"), ECollectionStorageMode::ToText(CollectionItemPtr->StorageMode));
		if (CollectionItemPtr->StorageMode == ECollectionStorageMode::Static)
		{
			InCallback(LOCTEXT("CollectionObjectCountKey", "Object Count"), FText::AsNumber(CollectionItemPtr->NumObjects));
		}
	}
}

#undef LOCTEXT_NAMESPACE
