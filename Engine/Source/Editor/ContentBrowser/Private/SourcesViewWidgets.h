// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTagItemTypes.h"
#include "AssetViewTypes.h"
#include "Delegates/Delegate.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Geometry.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SCompoundWidget.h"

class FDragDropEvent;
class FSlateRect;
class FString;
class FTreeItem;
class SEditableTextBox;
class SInlineEditableTextBlock;
class SWidget;
struct FCollectionItem;
struct FSlateBrush;
struct FSlateFontInfo;

template <>
struct TWidgetTypeTraits<class SAssetTreeItem>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** A single item in the asset tree. Represents a folder. */
class SAssetTreeItem : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_FourParams( FOnNameChanged, const TSharedPtr<FTreeItem>& /*TreeItem*/, const FString& /*InProposedName*/, const UE::Slate::FDeprecateVector2DParameter& /*MessageLocation*/, const ETextCommit::Type /*CommitType*/);
	DECLARE_DELEGATE_RetVal_ThreeParams( bool, FOnVerifyNameChanged, const TSharedPtr<FTreeItem>& /*TreeItem*/, const FString& /*InProposedName*/, FText& /*OutErrorMessage*/);

	SLATE_BEGIN_ARGS( SAssetTreeItem )
		: _TreeItem( TSharedPtr<FTreeItem>() )
		, _IsItemExpanded( false )
	{}

		/** Data for the folder this item represents */
		SLATE_ARGUMENT( TSharedPtr<FTreeItem>, TreeItem )

		/** Delegate for when the user commits a new name to the folder */
		SLATE_EVENT( FOnNameChanged, OnNameChanged )

		/** Delegate for when the user is typing a new name for the folder */
		SLATE_EVENT( FOnVerifyNameChanged, OnVerifyNameChanged )

		/** True when this item has children and is expanded */
		SLATE_ATTRIBUTE( bool, IsItemExpanded )

		/** The string in the title to highlight (used when searching folders) */
		SLATE_ATTRIBUTE( FText, HighlightText)

		SLATE_ATTRIBUTE( FSlateFontInfo, FontOverride)

		/** Callback to check if the widget is selected, should only be hooked up if parent widget is handling selection or focus. */
		SLATE_EVENT( FIsSelected, IsSelected )
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

	virtual ~SAssetTreeItem() override;

	//~ Begin SWidget
	virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	//~ End SWidget

protected:
	/** Returns the text of the folder name */
	virtual FText GetNameText() const;

	/** Handles verifying name changes */
	bool VerifyNameChanged(const FText& InName, FText& OutError) const;

	/** Handles committing a name change */
	void HandleNameCommitted( const FText& NewText, ETextCommit::Type /*CommitInfo*/ );

	/** Returns false if this folder is in the process of being created */
	bool IsReadOnly() const;

	/** Gets the brush used to draw the folder icon */
	const FSlateBrush* GetFolderIcon() const;

	/** Gets the color used to draw the folder icon */
	FSlateColor GetFolderColor() const;

	/** Returns the text to use for the folder tooltip */
	virtual FText GetToolTipText() const;

	/** Returns the image for the border around this item. Used for drag/drop operations */
	const FSlateBrush* GetBorderImage() const;

	/** Returns a widget that combines the results of calling all registered PathViewStateIconGenerators **/
	TSharedRef<SWidget> GenerateStateIcons();

	/** The data for this item */
	TWeakPtr<FTreeItem> TreeItem;

	/** The name of the asset as an editable text box */
	TSharedPtr<SEditableTextBox> EditableName;

	/** Delegate for when the user commits a new name to the folder */
	FOnNameChanged OnNameChanged;

	/** Delegate for when a user is typing a name for the folder */
	FOnVerifyNameChanged OnVerifyNameChanged;

	/** True when this item has children and is expanded */
	TAttribute<bool> IsItemExpanded;

	/** Delegate called to get the selection state of an asset path */
	FIsSelected IsSelected;

	/** True when a drag is over this item with a drag operation that we know how to handle. The operation itself may not be valid to drop. */
	bool bDraggedOver;

	/** What type of stuff does this folder hold */
	EFolderType FolderType;

	/** Widget to display the name of the asset item and allows for renaming */
	TSharedPtr<SInlineEditableTextBlock> InlineRenameWidget;

	/** Handle to the registered EnterEditingMode delegate. */
	FDelegateHandle EnterEditingModeDelegateHandle;
};

template <>
struct TWidgetTypeTraits<class SCollectionTreeItem>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** A single item in the collection tree. */
class SCollectionTreeItem : public SCompoundWidget
{
public:
	/** Delegates for when a collection is renamed. If returning false, OutWarningMessage will be displayed over the collection. */
	DECLARE_DELEGATE_OneParam( FOnBeginNameChange, const TSharedPtr<FCollectionItem>& /*Item*/);
	DECLARE_DELEGATE_RetVal_FourParams( bool, FOnNameChangeCommit, const TSharedPtr<FCollectionItem>& /*Item*/, const FString& /*NewName*/, bool /*bChangeConfirmed*/, FText& /*OutWarningMessage*/);
	DECLARE_DELEGATE_RetVal_FourParams( bool, FOnVerifyRenameCommit, const TSharedPtr<FCollectionItem>& /*Item*/, const FString& /*NewName*/, const FSlateRect& /*MessageAnchor*/, FText& /*OutErrorMessage*/)
	
	/** Delegates for when a collection item has something dropped into it */
	DECLARE_DELEGATE_RetVal_FourParams( bool, FOnValidateDragDrop, TSharedRef<FCollectionItem> /*CollectionItem*/, const FGeometry& /*Geometry*/, const FDragDropEvent& /*DragDropEvent*/, bool& /*OutIsKnownDragOperation*/ );
	DECLARE_DELEGATE_RetVal_ThreeParams( FReply, FOnHandleDragDrop, TSharedRef<FCollectionItem> /*CollectionItem*/, const FGeometry& /*Geometry*/, const FDragDropEvent& /*DragDropEvent*/ );

	SLATE_BEGIN_ARGS( SCollectionTreeItem )
		: _CollectionItem( TSharedPtr<FCollectionItem>() )
		, _ParentWidget()
	{}

		/** Data for the collection this item represents */
		SLATE_ARGUMENT( TSharedPtr<FCollectionItem>, CollectionItem )

		/** The parent widget */
		SLATE_ARGUMENT( TSharedPtr<SWidget>, ParentWidget )

		/** Delegate for when the user begins to rename the item */
		SLATE_EVENT( FOnBeginNameChange, OnBeginNameChange )

		/** Delegate for when the user commits a new name to the folder */
		SLATE_EVENT( FOnNameChangeCommit, OnNameChangeCommit )

		/** Delegate for when a collection name has been entered for an item to verify the name before commit */
		SLATE_EVENT( FOnVerifyRenameCommit, OnVerifyRenameCommit )

		/** Delegate to validate a drag drop operation on this collection item */
		SLATE_EVENT( FOnValidateDragDrop, OnValidateDragDrop )

		/** Delegate to handle a drag drop operation on this collection item */
		SLATE_EVENT( FOnHandleDragDrop, OnHandleDragDrop )

		/** Callback to check if the widget is selected, should only be hooked up if parent widget is handling selection or focus. */
		SLATE_EVENT( FIsSelected, IsSelected )

		/** True if the item is read-only. It will not be able to be renamed if read-only */
		SLATE_ATTRIBUTE( bool, IsReadOnly )

		/** Text to highlight for this item */
		SLATE_ATTRIBUTE( FText, HighlightText )

		/** True if the check box of the collection item is enabled */
		SLATE_ATTRIBUTE( bool, IsCheckBoxEnabled )

		/** Whether the check box of the collection item is currently in a checked state (if unset, no check box will be shown) */
		SLATE_ATTRIBUTE( ECheckBoxState, IsCollectionChecked )

		/** Delegate for when the checked state of the collection item check box is changed */
		SLATE_EVENT( FOnCheckStateChanged, OnCollectionCheckStateChanged )

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	/** True when a drag is over this item with a drag operation that we know how to handle. The operation itself may not be valid to drop. */
	bool IsDraggedOver() const
	{
		return bDraggedOver;
	}

private:
	/** Used by OnDragEnter, OnDragOver, and OnDrop to check and update the validity of the drag operation */
	bool ValidateDragDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, bool& OutIsKnownDragOperation ) const;

	/** Handles beginning a name change */
	void HandleBeginNameChange( const FText& OldText );

	/** Handles committing a name change */
	void HandleNameCommitted( const FText& NewText, ETextCommit::Type CommitInfo );

	/** Handles verifying a name change */
	bool HandleVerifyNameChanged( const FText& NewText, FText& OutErrorMessage );

	/** Returns the text of the collection name */
	FText GetNameText() const;

	/** Returns the color of the collection name */
	FLinearColor GetCollectionColor() const;

	/** Get the object count text for the current collection item */
	FText GetCollectionObjectCountText() const;

	/** Get the warning message for the current collection item */
	FText GetCollectionWarningText() const;

	/** Build the tooltip info for this collection */
	void BuildToolTipInfo(const FOnBuildAssetTagItemToolTipInfoEntry& InCallback) const;

private:
	/** A weak pointer to the parent widget. */
	TWeakPtr<SWidget> ParentWidget;

	/** The data for this item */
	TWeakPtr<FCollectionItem> CollectionItem;

	/** The name of the asset as an editable text box */
	TSharedPtr<SEditableTextBox> EditableName;

	/** True when a drag is over this item with a drag operation that we know how to handle. The operation itself may not be valid to drop. */
	bool bDraggedOver;

	/** Delegate to validate a drag drop operation on this collection item */
	FOnValidateDragDrop OnValidateDragDrop;

	/** Delegate to handle a drag drop operation on this collection item */
	FOnHandleDragDrop OnHandleDragDrop;

	/** The geometry as of the last frame. Used to open warning messages over the item */
	FGeometry CachedGeometry;

	/** Delegate for when the user starts to rename an item */
	FOnBeginNameChange OnBeginNameChange;

	/** Delegate for when the user commits a new name to the collection */
	FOnNameChangeCommit OnNameChangeCommit;

	/** Delegate for when a collection name has been entered for an item to verify the name before commit */
	FOnVerifyRenameCommit OnVerifyRenameCommit;
};
