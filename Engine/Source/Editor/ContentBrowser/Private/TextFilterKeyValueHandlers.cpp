// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextFilterKeyValueHandlers.h"

#include "TextFilterKeyValueHandler.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextFilterKeyValueHandlers)

class FTextFilterString;
struct FContentBrowserItem;

bool UTextFilterKeyValueHandlers::HandleTextFilterKeyValue(const FContentBrowserItem& InContentBrowserItem, const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode, bool& bOutIsMatch)
{
	return false;
}
