// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreFwd.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "TextFilterKeyValueHandlers.generated.h"

class FTextFilterString;
class UTextFilterKeyValueHandler;
struct FContentBrowserItem;

USTRUCT()
struct FTextFilterKeyValueHandlerEntry
{
	GENERATED_BODY()

	UPROPERTY(config)
	FName Key;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(config)
	TSoftClassPtr<UTextFilterKeyValueHandler> HandlerClass;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

UCLASS(transient, config = Editor)
class UE_DEPRECATED(5.5, "This type has been replaced by IAssetTextFilterHandler which must be manually instantiated and registered and must be implemented in a threadsafe way.")
UTextFilterKeyValueHandlers : public UObject
{
	GENERATED_BODY()
public:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(config)
	TArray<FTextFilterKeyValueHandlerEntry> TextFilterKeyValueHandlers;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	static bool HandleTextFilterKeyValue(const FContentBrowserItem& InContentBrowserItem, const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode, bool& bOutIsMatch);
};