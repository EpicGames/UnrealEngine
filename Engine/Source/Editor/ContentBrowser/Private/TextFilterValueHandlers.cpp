// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextFilterValueHandlers.h"

#include "TextFilterValueHandler.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextFilterValueHandlers)

class FTextFilterString;
struct FContentBrowserItem;

bool UTextFilterValueHandlers::HandleTextFilterValue(const FContentBrowserItem& InContentBrowserItem, const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode, bool& bOutIsMatch)
{

	return false;
}
