// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreFwd.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "TextFilterValueHandlers.generated.h"

class FTextFilterString;
class UTextFilterValueHandler;
struct FContentBrowserItem;

UCLASS(transient, config = Editor)
class UE_DEPRECATED(5.5, "This type has been replaced by IAssetTextFilterHandler which must be manually instantiated and registered and must be implemented in a threadsafe way.")
UTextFilterValueHandlers : public UObject
{
	GENERATED_BODY()
public:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(config)
	TArray<TSoftClassPtr<UTextFilterValueHandler>> TextFilterValueHandlers;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	static bool HandleTextFilterValue(const FContentBrowserItem& InContentBrowserItem, const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode, bool& bOutIsMatch);
};