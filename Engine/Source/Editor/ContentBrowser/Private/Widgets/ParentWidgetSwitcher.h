// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"

namespace UE::Editor::ContentBrowser::Private
{
	/** Takes a single widget and two or more containing widgets (SBox) to switch between, depending on a custom condition. */
	class FWidgetParentSwitcher
	{
	public:
		virtual ~FWidgetParentSwitcher() = default;

		FWidgetParentSwitcher(
	        const TSharedPtr<SWidget>& InSubjectWidget,
	        const TArray<TSharedPtr<SBox>>& InParentWidgets)
	        : SubjectWidget(InSubjectWidget)
	        , ParentWidgets(InParentWidgets)
	    {
	    }

	    /** Call to check and switch the widget's parent. Return true if switched. */
	    [[maybe_unused]] bool Update()
	    {
	        const int32 TargetParentIdx = GetParentIndex();
	        if (CurrentSubjectWidgetParentIdx == TargetParentIdx)
	        {
	            return false;
	        }

	        check(ParentWidgets.IsValidIndex(CurrentSubjectWidgetParentIdx));
	        check(ParentWidgets.IsValidIndex(TargetParentIdx));

			ParentWidgets[CurrentSubjectWidgetParentIdx]->SetContent(SNullWidget::NullWidget);
			ParentWidgets[TargetParentIdx]->SetContent(SubjectWidget.ToSharedRef());

			CurrentSubjectWidgetParentIdx = TargetParentIdx;

			return true;
	    }

		TSharedPtr<SWidget> GetParentWidget() const
		{
			check(ParentWidgets.IsValidIndex(CurrentSubjectWidgetParentIdx));
			return ParentWidgets[CurrentSubjectWidgetParentIdx];
		}

	protected:
		virtual int32 GetParentIndex() const = 0;

	protected:
	    TSharedPtr<SWidget> SubjectWidget;
	    int32 CurrentSubjectWidgetParentIdx = 0;
	    TArray<TSharedPtr<SBox>> ParentWidgets;
	};

	/** Switches parent widgets based on widget size */
	template <EOrientation Orientation>
	class TWidgetSizeParentSwitcher
	    : public FWidgetParentSwitcher
	{
	public:
		virtual ~TWidgetSizeParentSwitcher() override = default;

	    TWidgetSizeParentSwitcher(
	        const TSharedPtr<SWidget>& InSubjectWidget,
	        const TArray<TSharedPtr<SBox>>& InParentWidgets,
	        const TArray<FInt16Range>& InSwitchSizeRanges,
	        TUniqueFunction<float()>&& InSizeGetter)
	        : FWidgetParentSwitcher(InSubjectWidget, InParentWidgets)
			, SwitchSizeRanges(InSwitchSizeRanges)
			, SizeGetter(MoveTemp(InSizeGetter))
	    {
			check(SwitchSizeRanges.Num() == ParentWidgets.Num());
	    }

	protected:
		virtual int32 GetParentIndex() const override
		{
			const float CurrentSize = SizeGetter();
			int32 TargetIdx = CurrentSubjectWidgetParentIdx;

			// Returns -1 if less, 0 if in range, 1 if greater
			auto InRange = [&](const int32 Idx, const int32 Value)
			{
				const FInt16Range& Range = SwitchSizeRanges[Idx];
				if (Value <= Range.GetLowerBoundValue())
				{
					return -1;
				}

				if (Value > Range.GetUpperBoundValue())
				{
					return 1;
				}

				return 0;
			};

			// Starting from the existing widget, check each size range for the matching parent
			int32 NumIterations = 0;
			while (NumIterations < SwitchSizeRanges.Num())
			{
				const int32 InRangeResult = InRange(TargetIdx, FMath::FloorToInt(CurrentSize - UE_KINDA_SMALL_NUMBER));
				if (InRangeResult <= 0)
				{
					// Effectively clamp to first widget
					if (TargetIdx == 0 || InRangeResult == 0)
					{
						return TargetIdx;
					}

					// Otherwise offset TargetIdx
					TargetIdx -= 1;
				}
				else
				{
					// Effectively clamp to last widget
					if (TargetIdx == SwitchSizeRanges.Num() - 1)
					{
						return TargetIdx;
					}

					// Otherwise offset TargetIdx
					TargetIdx += 1;
				}

				++NumIterations;
			}

			checkNoEntry();
			return INDEX_NONE;
		}

	private:
		TArray<FInt16Range> SwitchSizeRanges;
		TUniqueFunction<float()> SizeGetter;
	};
}
