// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SAssetMenuIcon.h"

#include "AssetToolsModule.h"
#include "ClassIconFinder.h"
#include "ContentBrowserStyle.h"
#include "Misc/CommandLine.h"
#include "SlateOptMacros.h"
#include "Widgets/Colors/SColorBlock.h"

void SAssetMenuIcon::Construct(const FArguments& InArgs, const UClass* InAssetClass, const FName InIconOverride)
{
	check(InAssetClass);

	FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
	TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(InAssetClass);

	FLinearColor AssetColor = FLinearColor::White;
	if (AssetTypeActions.IsValid())
	{
		AssetColor = AssetTypeActions.Pin()->GetTypeColor();
	}

	if (UE::Editor::ContentBrowser::IsNewStyleEnabled())
	{
		constexpr float AssetLineHeight = 2.0f; // Height of the colored line at the base of the icon

		const FName ClassIconBrushOverride = InIconOverride;
		const FSlateBrush* ClassIcon = nullptr;
		if (ClassIconBrushOverride.IsNone())
		{
			ClassIcon = FSlateIconFinder::FindIconBrushForClass(InAssetClass);
		}
		else
		{
			// Instead of getting the override icon directly from the editor style here get it from the
			// FSlateIconFinder since it may have additional styles registered which can be searched by passing
			// it as a default with no class to search for.
			ClassIcon = FSlateIconFinder::FindIconBrushForClass(nullptr, ClassIconBrushOverride);
		}

		ChildSlot
		[
			SNew(SBox)
			.WidthOverride(static_cast<float>(InArgs._IconContainerSize.X))
			.HeightOverride(static_cast<float>(InArgs._IconContainerSize.Y))
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
					.BorderBackgroundColor(FSlateColor(FStyleColors::Background))
					.Padding(2.0f)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(InArgs._IconSize.X, InArgs._IconSize.X))
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(ClassIcon)
					]
				]

				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Bottom)
				.Padding(0)
				[
					SNew(SColorBlock)
					.Size(FVector2D(1.0f, AssetLineHeight)) // The X size is ignored/overridden by HAlign_Fill
					.Color(AssetColor)
				]
			]
		];
	}
	else
	{
		const FName ClassThumbnailBrushOverride = InIconOverride;
		const FSlateBrush* ClassThumbnail = nullptr;
		if (ClassThumbnailBrushOverride.IsNone())
		{
			ClassThumbnail = FClassIconFinder::FindThumbnailForClass(InAssetClass);
		}
		else
		{
			// Instead of getting the override thumbnail directly from the editor style here get it from the
			// ClassIconFinder since it may have additional styles registered which can be searched by passing
			// it as a default with no class to search for.
			ClassThumbnail = FClassIconFinder::FindThumbnailForClass(nullptr, ClassThumbnailBrushOverride);
		}

		ChildSlot
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
				SNew(SBox)
				.WidthOverride(static_cast<float>(InArgs._IconContainerSize.X))
				.HeightOverride(static_cast<float>(InArgs._IconContainerSize.Y))
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("AssetThumbnail.AssetBackground"))
					.Padding(2.0f)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(SImage)
						.Image(ClassThumbnail)
					]
				]
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Bottom)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("WhiteBrush"))
				.BorderBackgroundColor(AssetColor)
				.Padding(FMargin(0, FMath::Max(FMath::CeilToFloat(InArgs._IconSize.X * 0.025f), 3.0f), 0, 0))
			]
		];
	}
}
