// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SContentBrowserFavoriteItem.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandInfo.h"
#include "SPrimaryButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ContentBrowserFavoriteItem"

TSharedRef<SWindow> SContentBrowserFavoriteItem::CreateAndShowWindow(const FArguments& InArguments, const FContentBrowserFavoriteItem& InItemToEdit)
{
	TSharedRef<SWindow> NewWindow = SNew(SWindow)
		.Title(LOCTEXT("EditFavoriteItemWindow", "Edit Favorite Item"))
		.HasCloseButton(true)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.SizingRule(ESizingRule::FixedSize)
		.ClientSize(FVector2D(400, 150));

	const TSharedRef<SContentBrowserFavoriteItem> Content = SArgumentNew(InArguments, SContentBrowserFavoriteItem, InItemToEdit);
	Content->ParentWindowWeak = NewWindow;

	NewWindow->SetContent(Content);
	FSlateApplication::Get().AddWindow(NewWindow);

	return NewWindow;
}

FText SContentBrowserFavoriteItem::GetAliasText() const
{
	return FText::FromString(FavoriteItemEdited.Alias);
}

void SContentBrowserFavoriteItem::OnAliasTextChanged(const FText& InText)
{
	FavoriteItemEdited.Alias = InText.ToString();
	bAnyChanges = true;
}

void SContentBrowserFavoriteItem::Construct(const FArguments& InArgs, const FContentBrowserFavoriteItem& InItemToEdit)
{
	FavoriteItemEdited = InItemToEdit;
	CommandOptions = InArgs._CommandOptions;
	OnConfirmClicked = InArgs._OnConfirm;
	OnCancelClicked = InArgs._OnCancel;
	bAnyChanges = false;
	
	// Empty choice
	CommandOptions.Add(MakeShared<FUICommandInfo>(NAME_None));
	const TSharedPtr<FUICommandInfo>* SelectedCommandPtr = CommandOptions.FindByPredicate([this, CommandName = FavoriteItemEdited.ShortcutCommandName](const TSharedPtr<FUICommandInfo>& InCommand)
	{
		return CommandName == InCommand->GetCommandName();
	});

	SelectedCommand = SelectedCommandPtr ? *SelectedCommandPtr : nullptr;

	constexpr float HPadding = 20.f;
	constexpr float VPadding = 10.f;

	ChildSlot
	.HAlign(HAlign_Fill)
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
		[
			SNew(SVerticalBox)
			// Path (read-only)
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.Padding(HPadding, VPadding, HPadding, 0.f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.Padding(0.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PathText", "Path"))
					.ToolTipText(LOCTEXT("PathTooltip", "Path of this favorite item"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(2.f)
				.Padding(0.f)
				[
					SNew(STextBlock)
					.Text(FText::FromString(FavoriteItemEdited.Path))
				]
			]
			// Alias
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.Padding(HPadding, VPadding, HPadding, 0.f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.Padding(0.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AliasText", "Alias"))
					.ToolTipText(LOCTEXT("AliasTooltip", "Alias of this favorite item"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(2.f)
				.Padding(0.f)
				[
					SNew(SEditableTextBox)
					.Text(this, &SContentBrowserFavoriteItem::GetAliasText)
					.OnTextChanged(this, &SContentBrowserFavoriteItem::OnAliasTextChanged)
				]
			]
			// Shortcut
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.Padding(HPadding, VPadding, HPadding, 0.f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.Padding(0.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ShortcutText", "Shortcut"))
					.ToolTipText(LOCTEXT("ShortcutTooltip", "Keyboard shortcut assigned to navigate to this favorite item"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(2.f)
				.Padding(0.f)
				[
					SNew(SComboBox<TSharedPtr<FUICommandInfo>>)
					.OptionsSource(&CommandOptions)
					.InitiallySelectedItem(SelectedCommand)
					.OnGenerateWidget(this, &SContentBrowserFavoriteItem::OnCommandGenerateWidget)
					.OnSelectionChanged(this, &SContentBrowserFavoriteItem::OnCommandSelectionChanged)
					[
						SNew(STextBlock)
						.Text(this, &SContentBrowserFavoriteItem::GetItemCommandLabel)
					]
				]
			]
			// Buttons
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Bottom)
			.Padding(HPadding, VPadding, HPadding, 0.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(5.f)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.Text(LOCTEXT("CancelButton", "Cancel"))
					.OnClicked(this, &SContentBrowserFavoriteItem::OnCancelButtonClicked)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5.f)
				.HAlign(HAlign_Right)
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("ConfirmButton", "Confirm"))
					.OnClicked(this, &SContentBrowserFavoriteItem::OnConfirmButtonClicked)
				]
			]
		]
	];
}

TSharedRef<SWidget> SContentBrowserFavoriteItem::OnCommandGenerateWidget(TSharedPtr<FUICommandInfo> InCommand)
{
	return SNew(STextBlock).Text(GetCommandLabel(InCommand));
}

void SContentBrowserFavoriteItem::OnCommandSelectionChanged(TSharedPtr<FUICommandInfo> InCommand, ESelectInfo::Type InSelectType)
{
	SelectedCommand = InCommand;
	FavoriteItemEdited.ShortcutCommandName = InCommand ? InCommand->GetCommandName() : NAME_None;
	bAnyChanges = true;
}

FText SContentBrowserFavoriteItem::GetItemCommandLabel() const
{
	return GetCommandLabel(SelectedCommand);
}

FText SContentBrowserFavoriteItem::GetCommandLabel(TSharedPtr<FUICommandInfo> InCommand) const
{
	if (!InCommand || InCommand->GetCommandName().IsNone())
	{
		return LOCTEXT("NoFavoriteShortcutLabel", "No shortcut");
	}

	return InCommand->GetInputText();
}

FReply SContentBrowserFavoriteItem::OnConfirmButtonClicked() const
{
	OnConfirmClicked.ExecuteIfBound(FavoriteItemEdited, bAnyChanges);
	if (const TSharedPtr<SWindow> ParentWindow = ParentWindowWeak.Pin())
	{
		ParentWindow->RequestDestroyWindow();
	}
	return FReply::Handled();
}

FReply SContentBrowserFavoriteItem::OnCancelButtonClicked() const
{
	OnCancelClicked.ExecuteIfBound();
	if (const TSharedPtr<SWindow> ParentWindow = ParentWindowWeak.Pin())
	{
		ParentWindow->RequestDestroyWindow();
	}
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
