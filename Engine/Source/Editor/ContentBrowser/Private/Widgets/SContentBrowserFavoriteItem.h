// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "ContentBrowserConfig.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/SCompoundWidget.h"

class FReply;
class FUICommandInfo;
class SEditableTextBox;
class SWindow;

/** 
 * Content browser favorite item dialog form to edit options
 */
class SContentBrowserFavoriteItem : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_TwoParams(FOnConfirmClicked, const FContentBrowserFavoriteItem& /* Item */, bool /** AnyChanges */);
	DECLARE_DELEGATE(FOnCancelClicked);

	SLATE_BEGIN_ARGS(SContentBrowserFavoriteItem)
		{}
		/** Commands to pick from for this item */
		SLATE_ARGUMENT(TArray<TSharedPtr<FUICommandInfo>>, CommandOptions)
		/** Delegate for when the confirm button is clicked */
		SLATE_EVENT(FOnConfirmClicked, OnConfirm)
		/** Delegate for when the cancel button is clicked */
		SLATE_EVENT(FOnCancelClicked, OnCancel)
    SLATE_END_ARGS()

	/** Create a window with this widget and shows it */
	static TSharedRef<SWindow> CreateAndShowWindow(const FArguments& InArguments, const FContentBrowserFavoriteItem& InItemToEdit);

    void Construct(const FArguments& InArgs, const FContentBrowserFavoriteItem& InItemToEdit);

protected:
	TSharedRef<SWidget> OnCommandGenerateWidget(TSharedPtr<FUICommandInfo> InCommand);
	void OnCommandSelectionChanged(TSharedPtr<FUICommandInfo> InCommand, ESelectInfo::Type InSelectType);
	FText GetItemCommandLabel() const;
	FText GetCommandLabel(TSharedPtr<FUICommandInfo> InCommand) const;
	
	FText GetAliasText() const;
	void OnAliasTextChanged(const FText& InText);

	/** Called when cancel button is clicked */
	FReply OnConfirmButtonClicked() const;

	/** Called when cancel button is clicked */
	FReply OnCancelButtonClicked() const;
	
	TArray<TSharedPtr<FUICommandInfo>> CommandOptions;
	TSharedPtr<FUICommandInfo> SelectedCommand;

	FOnConfirmClicked OnConfirmClicked;
	FOnCancelClicked OnCancelClicked;
	
	/** Current item being edited */
	FContentBrowserFavoriteItem FavoriteItemEdited;
	
	/** Self containing window */
	TWeakPtr<SWindow> ParentWindowWeak;
	
	/** Whether any changes have been done to this item */
	bool bAnyChanges = false;
};