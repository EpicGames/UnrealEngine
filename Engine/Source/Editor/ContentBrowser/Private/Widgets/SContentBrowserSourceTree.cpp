// Copyright Epic Games, Inc. All Rights Reserved.

#include "SContentBrowserSourceTree.h"

#include "ContentBrowserStyle.h"
#include "Framework/Layout/IScrollableWidget.h"
#include "Misc/ConfigCacheIni.h"
#include "SlateOptMacros.h"
#include "SourcesSearch.h"
#include "SSearchToggleButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SInvalidationPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

namespace UE::Editor::ContentBrowser::Private
{
	constexpr float MinBodyHeight = 88.0f;

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

	void SContentBrowserSourceTree::FSlot::Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
	{
		TSlotBase<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));

		check(EntryWidget.IsValid() || InArgs._AreaWidget.IsValid());

		EntryWidget = InArgs._AreaWidget;

		ExpandedSizeRule = InArgs._ExpandedSizeRule.Get(SSplitter::ESizeRule::FractionOfParent);

		if (InArgs._Visibility.IsSet())
		{
			EntryVisibility = InArgs._Visibility;
		}
		else
		{
			EntryVisibility = EVisibility::Visible;
		}

		if (InArgs._HeaderHeight.IsSet())
		{
			HeaderHeight = InArgs._HeaderHeight.GetValue();
		}
		else
		{
			HeaderHeight =
				IsNewStyleEnabled()
				? 36.0f
				: 26.0f + 3.0f;
		}

		SlotSize = InitialSlotSize = InArgs._Size.Get(0.5f);
	}

	SSplitter::ESizeRule SContentBrowserSourceTree::FSlot::GetExpandedSizeRule() const
	{
		// If the slot is empty, we size to content (to enforce a fixed size and disallow resizing)
		return EntryWidget->IsExpanded() && IsVisible()
			? EntryWidget->IsEmpty()
				? EmptyExpandedSizeRule : ExpandedSizeRule
			: SSplitter::ESizeRule::SizeToContent;
	}

	SSplitter::ESizeRule SContentBrowserSourceTree::FSlot::GetEmptyExpandedSizeRule() const
	{
		return EmptyExpandedSizeRule;
	}

	float SContentBrowserSourceTree::FSlot::GetMinHeight() const
	{
		return IsVisible()
			? EntryWidget->IsExpanded() ? HeaderHeight + MinBodyHeight : HeaderHeight
			: 0.0f;
	}

	float SContentBrowserSourceTree::FSlot::GetHeaderHeight() const
	{
		return HeaderHeight;
	}

	float SContentBrowserSourceTree::FSlot::GetInitialSlotSize() const
	{
		return InitialSlotSize;
	}

	float SContentBrowserSourceTree::FSlot::GetSlotSize() const
	{
 		return SlotSize;
	}

	TSharedPtr<SContentBrowserSourceTreeArea> SContentBrowserSourceTree::FSlot::GetEntryWidget() const
	{
		return EntryWidget;
	}

	bool SContentBrowserSourceTree::FSlot::IsVisible() const
	{
		return EntryVisibility.Get() == EVisibility::Visible;
	}

	void SContentBrowserSourceTree::FSlot::OnSlotResized(float InNewSize)
	{
		SlotSize = InNewSize;
	}

	SContentBrowserSourceTree::FSlot::FSlotArguments SContentBrowserSourceTree::Slot()
	{
		return FSlot::FSlotArguments(MakeUnique<FSlot>());
	}

	SContentBrowserSourceTree::FScopedWidgetSlotArguments SContentBrowserSourceTree::AddSlot(const int32 InAtIndex)
	{
		return FScopedWidgetSlotArguments{ MakeUnique<FSlot>(), Slots, InAtIndex,
			[this, HostSplitter = Splitter](const FSlot* InSlotAdded, int32 InSlotIdx)
			{
				// When the slot is added, also add it to the splitter
				HostSplitter->AddSlot(InSlotIdx)
				.Value(InSlotAdded->GetInitialSlotSize())
				.SizeRule(this, &SContentBrowserSourceTree::GetExpandedSizeRule, InSlotAdded->GetEntryWidget())
				.MinSize(this, &SContentBrowserSourceTree::GetMinHeight, InSlotAdded->GetEntryWidget())
				[
					InSlotAdded->GetEntryWidget().ToSharedRef()
				];

				// ... and re-calculate total header height
				UpdateTotalHeaderHeight();
			}};
	}

	int32 SContentBrowserSourceTree::RemoveSlot(const TSharedRef<SWidget>& InSlotWidget)
	{
		if (!Splitter.IsValid())
		{
			return INDEX_NONE;
		}

		if (Slots.Remove(InSlotWidget) != INDEX_NONE)
		{
			const int32 RemovedIndex = Splitter->RemoveSlot(InSlotWidget);

			UpdateTotalHeaderHeight();

			return RemovedIndex;
		}

		return INDEX_NONE;
	}

	int32 SContentBrowserSourceTree::RemoveSlotAt(int32 InIndex)
	{
		if (!Splitter.IsValid())
		{
			return INDEX_NONE;
		}

		if (Slots.IsValidIndex(InIndex))
		{
			Slots.RemoveAt(InIndex);

			Splitter->RemoveAt(InIndex);

			UpdateTotalHeaderHeight();

			return InIndex;
		}

		return INDEX_NONE;
	}

	void SContentBrowserSourceTree::ClearChildren()
	{
		Slots.Empty();
		Splitter->ClearChildren();

		UpdateTotalHeaderHeight();
	}

	int32 SContentBrowserSourceTree::NumSlots() const
	{
		return Slots.Num();
	}

	bool SContentBrowserSourceTree::IsValidSlotIndex(int32 InIndex) const
	{
		return Slots.IsValidIndex(InIndex);
	}

	float SContentBrowserSourceTree::GetMinHeight(TSharedPtr<SContentBrowserSourceTreeArea> InEntryWidget) const
	{
		for (int32 SlotIdx = 0; SlotIdx < Slots.Num(); ++SlotIdx)
		{
			if (Slots[SlotIdx].GetEntryWidget() == InEntryWidget)
			{
				return Slots[SlotIdx].GetMinHeight();
			}
		}

		return 0.0f;
	}

	float SContentBrowserSourceTree::GetTotalHeaderHeight() const
	{
		return TotalHeaderHeight;
	}

	SSplitter::ESizeRule SContentBrowserSourceTree::GetExpandedSizeRule(TSharedPtr<SContentBrowserSourceTreeArea> InEntryWidget) const
	{
		for (int32 SlotIdx = 0; SlotIdx < Slots.Num(); ++SlotIdx)
		{
			if (Slots[SlotIdx].GetEntryWidget() == InEntryWidget)
			{
				return Slots[SlotIdx].GetExpandedSizeRule();
			}
		}

		return SSplitter::FractionOfParent;
	}

	float SContentBrowserSourceTree::GetSlotSize(const int32 InSlotIdx) const
	{
		if (Slots.IsValidIndex(InSlotIdx))
		{
			return Slots[InSlotIdx].GetSlotSize();
		}

		return 0.0f;
	}

	void SContentBrowserSourceTree::OnSlotResized(float InNewSize, const int32 InSlotIdx)
	{
		if (Slots.IsValidIndex(InSlotIdx))
		{
			Slots[InSlotIdx].OnSlotResized(InNewSize);
		}
	}

	void SContentBrowserSourceTree::UpdateTotalHeaderHeight()
	{
		TotalHeaderHeight = 0.0f;
		for (int32 SlotIdx = 0; SlotIdx < Slots.Num(); ++SlotIdx)
		{
			TotalHeaderHeight += Slots[SlotIdx].GetHeaderHeight();
		}
	}

	void SContentBrowserSourceTree::Construct(const FArguments& InArgs)
	{
		using namespace UE::ContentBrowser::Private;

		constexpr float SourceTreeSectionPadding = 2.0f;
		constexpr float SourceTreeSectionHandleSize = 8.0f;
		constexpr float SourceTreeHeaderHeight = 32.0f;

		SAssignNew(Splitter, SSplitter)
		.Style(&FContentBrowserStyle::Get().GetWidgetStyle<FSplitterStyle>("ContentBrowser.Splitter"))
		.Clipping(EWidgetClipping::ClipToBounds)
		.PhysicalSplitterHandleSize(SourceTreeSectionPadding)
		.HitDetectionSplitterHandleSize(SourceTreeSectionHandleSize)
		.Orientation(EOrientation::Orient_Vertical)
		.MinimumSlotHeight(SourceTreeHeaderHeight);

		for (const FSlot::FSlotArguments& Slot : InArgs._Slots)
		{
			AddSlot()
			.AreaWidget(Slot._AreaWidget)
			.HeaderHeight(SourceTreeHeaderHeight)
			.Size(Slot._Size)
			.Visibility(Slot._Visibility);
		}

		const TAttribute<float> TotalHeaderHeightAttribute =
			TAttribute<float>::Create(
				TAttribute<float>::FGetter::CreateSP(
					this, &SContentBrowserSourceTree::GetTotalHeaderHeight));

		ChildSlot
		.HAlign(HAlign_Fill)
	    .VAlign(VAlign_Fill)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.MinHeight(TotalHeaderHeightAttribute)
			.FillHeight(1.0f)
			[
				Splitter.ToSharedRef()
			]

			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Bottom)
			.AutoHeight()
			[
				SNullWidget::NullWidget
			]
		];
	}

	void SContentBrowserSourceTreeArea::Construct(
		const FArguments& InArgs,
		const FName InId,
		const TSharedPtr<FSourcesSearch>& InSearch,
		TSharedRef<IScrollableWidget> InBody)
	{
		using namespace UE::ContentBrowser::Private;

		check(!InId.IsNone());

		Id = InId;
		bIsEmpty = InArgs._IsEmpty;
		bExpandedByDefault = InArgs._ExpandedByDefault;
		OnExpansionChanged = InArgs._OnExpansionChanged;
		BodyScrollableWidget = InBody;

		const float HorizontalPadding =
			UE::Editor::ContentBrowser::IsNewStyleEnabled()
			? 10.0f
			: 4.0f;

		const float VerticalHeaderPadding =
			UE::Editor::ContentBrowser::IsNewStyleEnabled()
			? HorizontalPadding - 6.0f
			: 0.0f;

		const float SearchButtonRightPadding =
			UE::Editor::ContentBrowser::IsNewStyleEnabled()
			? 0.0f
			: 4.0f;

		constexpr float EmptyBodyLabelPadding = 8.0f;

		const FMargin HeaderPadding = FMargin(HorizontalPadding, VerticalHeaderPadding);
		const FMargin ExpandableAreaPadding = FMargin(0.f, 1.f, 0.f , 0.f);

		TSharedPtr<SWidget> SearchButtonWidget = SNullWidget::NullWidget;
		if (InSearch.IsValid())
		{
			Search = InSearch;
			SearchButtonWidget =
				SAssignNew(SearchToggleButton, SSearchToggleButton, Search->GetWidget())
				.Visibility(this, &SContentBrowserSourceTreeArea::GetHeaderSearchActionVisibility)
				.OnSearchBoxShown(FOnSearchBoxShown::CreateSPLambda(this, [this]() { SetExpanded(true); }));
		}

		TAttribute<int32> BodyContentIndexAttribute;
		if (UE::Editor::ContentBrowser::IsNewStyleEnabled()
			&& (InArgs._IsEmpty.IsSet() || InArgs._IsEmpty.IsBound()))
		{
			BodyContentIndexAttribute = MakeAttributeSPLambda(
				this,
				[this]()
				{
					return IsEmpty() ? 1 : 0;
				});
		}
		else
		{
			BodyContentIndexAttribute.Set(0);
		}

		ChildSlot
		[
			SAssignNew(ExpandableArea, SExpandableArea)
			.Style(&FContentBrowserStyle::Get().GetWidgetStyle<FExpandableAreaStyle>("ContentBrowser.AssetTreeExpandableArea"))
			.BorderImage(FContentBrowserStyle::Get().GetBrush("ContentBrowser.AssetTreeHeaderBrush"))
			.BodyBorderImage(FContentBrowserStyle::Get().GetBrush("ContentBrowser.AssetTreeBodyBrush"))
			.HeaderPadding(HeaderPadding)
			.Visibility(InArgs._Visibility)
			.Padding(ExpandableAreaPadding)
			.AllowAnimatedTransition(true)
			.OnAreaExpansionChanged(this, &SContentBrowserSourceTreeArea::OnAreaExpansionChanged)
			.HeaderContent()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Visibility(EVisibility::HitTestInvisible) // Allows click-through to the expander button
					.Text(InArgs._Label)
					.TextStyle(FAppStyle::Get(), "ButtonText")
					.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Right)
				.AutoWidth()
				.Padding(0.0f)
				[
					InArgs._HeaderContent.Widget
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoWidth()
				.Padding(FMargin(HorizontalPadding, 0.0f, SearchButtonRightPadding, 0.0f))
				[
					SearchButtonWidget.ToSharedRef()
				]
			]
			.BodyContent()
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex(BodyContentIndexAttribute)

				+ SWidgetSwitcher::Slot()
				.Padding(0)
				[
					SNew(SVerticalBox)

					// Search bar (if applicable)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						// Should blend in visually with the header but technically acts like part of the body
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
						.Padding(FMargin(HorizontalPadding, 2.0f))
						[
							Search->GetWidget()
						]
					]

					+ SVerticalBox::Slot()
					.Padding(FMargin(0, 1))
					[
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
						SNew(SInvalidationPanel)
						.UseDynamicInvalidation(true)
						[
#endif
							// Surround scrollable with a scrollbox (adds drop shadows)
							SNew(SScrollBorder, InBody)
							[
								BodyScrollableWidget->GetScrollWidget()
							]
							.Style(InArgs._bEnableShadowBoxStyle ? InArgs._ShadowBoxStyle : &FCoreStyle::Get().GetWidgetStyle<FScrollBorderStyle>("ScrollBorder"))
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
						]
#endif
					]
				]

				+ SWidgetSwitcher::Slot()
				.Padding(0)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("Brushes.Recessed"))
					.Padding(0)
					[
						SNew(SBox)
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Top)
						.HeightOverride(MinBodyHeight)
						.Padding(EmptyBodyLabelPadding)
						[
							SNew(SRichTextBlock)
							.Text(InArgs._EmptyBodyLabel)
							.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("RichTextBlock.Italic"))
							.AutoWrapText(true)
							.Justification(ETextJustify::Center)
							.DecoratorStyleSet(&FAppStyle::Get())
							+ SRichTextBlock::ImageDecorator()
						]
					]
				]
			]
		];
	}

	bool SContentBrowserSourceTreeArea::IsExpanded() const
	{
		return ExpandableArea->IsExpanded();
	}

	void SContentBrowserSourceTreeArea::SetExpanded(bool bInExpanded)
	{
		ExpandableArea->SetExpanded(bInExpanded);
	}

	bool SContentBrowserSourceTreeArea::IsEmpty() const
	{
		return bIsEmpty.Get();
	}

	void SContentBrowserSourceTreeArea::SaveSettings(const FString& InIniFilename, const FString& InIniSection, const FString& InSettingsString) const
	{
		const FString& IdString = Id.ToString();

		GConfig->SetBool(*InIniSection, *(InSettingsString + TEXT(".") + IdString + TEXT("AreaExpanded")), IsExpanded(), InIniFilename);

		if (Search.IsValid())
		{
			GConfig->SetBool(*InIniSection, *(InSettingsString + TEXT(".") + IdString + TEXT("SearchAreaExpanded")), SearchToggleButton->IsExpanded(), InIniFilename);
		}
	}

	void SContentBrowserSourceTreeArea::LoadSettings(const FString& InIniFilename, const FString& InIniSection, const FString& InSettingsString)
	{
		const FString& IdString = Id.ToString();

		bool bAreaExpanded = bExpandedByDefault;
		GConfig->GetBool(*InIniSection, *(InSettingsString + TEXT(".") + IdString + TEXT("AreaExpanded")), bAreaExpanded, InIniFilename);
		SetExpanded(bAreaExpanded);

		if (Search.IsValid())
		{
			bool bSearchAreaExpanded = false;
			GConfig->GetBool(*InIniSection, *(InSettingsString + TEXT(".") + IdString + TEXT("SearchAreaExpanded")), bSearchAreaExpanded, InIniFilename);
			SearchToggleButton->SetExpanded(bSearchAreaExpanded);
		}
	}

	void SContentBrowserSourceTreeArea::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		if (BodyScrollableWidget.IsValid())
		{
			BodyScrollableWidget->GetScrollWidget()->OnDragEnter(MyGeometry, DragDropEvent);
		}
	}

	void SContentBrowserSourceTreeArea::OnDragLeave(const FDragDropEvent& DragDropEvent)
	{
		if (BodyScrollableWidget.IsValid())
		{
			BodyScrollableWidget->GetScrollWidget()->OnDragLeave(DragDropEvent);
		}
	}

	FReply SContentBrowserSourceTreeArea::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		if (BodyScrollableWidget.IsValid())
		{
			return BodyScrollableWidget->GetScrollWidget()->OnDrop(MyGeometry, DragDropEvent);
		}
		return FReply::Unhandled();
	}

	TSharedPtr<SSearchToggleButton> SContentBrowserSourceTreeArea::GetSearchToggleButton() const
	{
		return SearchToggleButton;
	}

	EVisibility SContentBrowserSourceTreeArea::GetHeaderSearchActionVisibility() const
	{
		return EVisibility::Visible;
	}

	void SContentBrowserSourceTreeArea::OnAreaExpansionChanged(bool bInIsExpanded)
	{
		if (SearchToggleButton.IsValid() && !bInIsExpanded)
		{
			SearchToggleButton->SetExpanded(false);
		}

		OnExpansionChanged.ExecuteIfBound(bInIsExpanded);
	}

	END_SLATE_FUNCTION_BUILD_OPTIMIZATION
}

#undef LOCTEXT_NAMESPACE
