// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Framework/Layout/IScrollableWidget.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SCompoundWidget.h"

class FSourcesSearch;
class SSearchToggleButton;

namespace UE::Editor::ContentBrowser::Private
{
	class SContentBrowserSourceTree;
	class SContentBrowserSourceTreeArea;
}

template<>
struct TWidgetTypeTraits<UE::Editor::ContentBrowser::Private::SContentBrowserSourceTree>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

template<>
struct TWidgetTypeTraits<UE::Editor::ContentBrowser::Private::SContentBrowserSourceTreeArea>
{
	static constexpr bool SupportsInvalidation() { return true; }
};


namespace UE::Editor::ContentBrowser::Private
{
	/** The Content Browser Source Tree, containing "Favorites", etc. */
	class SContentBrowserSourceTree : public SCompoundWidget
	{
		using Super = SCompoundWidget;

	public:
		class FSlot : public TSlotBase<FSlot>
		{
		public:
			SLATE_SLOT_BEGIN_ARGS(FSlot, TSlotBase<FSlot>)
				SLATE_ARGUMENT(TSharedPtr<SContentBrowserSourceTreeArea>, AreaWidget)
				SLATE_ARGUMENT(TOptional<float>, Size)
				SLATE_ARGUMENT(TOptional<SSplitter::ESizeRule>, ExpandedSizeRule)
				SLATE_ATTRIBUTE(EVisibility, Visibility)
				SLATE_ARGUMENT(TOptional<float>, HeaderHeight)
			SLATE_SLOT_END_ARGS()

			void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs);

		public:
			SSplitter::ESizeRule GetExpandedSizeRule() const;
			SSplitter::ESizeRule GetEmptyExpandedSizeRule() const;
			float GetMinHeight() const;
			float GetHeaderHeight() const;
			float GetInitialSlotSize() const;
			float GetSlotSize() const;
			TSharedPtr<SContentBrowserSourceTreeArea> GetEntryWidget() const;

			bool IsVisible() const;

			void OnSlotResized(float InNewSize);

		private:
			SSplitter::ESizeRule ExpandedSizeRule = SSplitter::ESizeRule::FractionOfParent;
			SSplitter::ESizeRule EmptyExpandedSizeRule = SSplitter::ESizeRule::FractionOfParent;
			float HeaderHeight = 36.0f;
			float InitialSlotSize = 0.0f;
			float SlotSize = 0.0f; // The current slot height
			TSharedPtr<SContentBrowserSourceTreeArea> EntryWidget = nullptr;
			TAttribute<EVisibility> EntryVisibility;
		};

	public:
		/** @return Add a new FSlot(). */
		static FSlot::FSlotArguments Slot();

		SLATE_BEGIN_ARGS(SContentBrowserSourceTree)
		{ }
			SLATE_SLOT_ARGUMENT(FSlot, Slots)
		SLATE_END_ARGS()

		SContentBrowserSourceTree()
			: Slots(this)
			, TotalHeaderHeight(0.0f)
		{
		}

		/**
		 * Constructs this widget with InArgs.
		 */
		void Construct(const FArguments& InArgs);

		TSharedPtr<SSplitter> GetSplitter() const { return Splitter; }

		using FScopedWidgetSlotArguments = TPanelChildren<FSlot>::FScopedWidgetSlotArguments;

		FScopedWidgetSlotArguments AddSlot(const int32 InAtIndex = INDEX_NONE);

		/** Removes a slot from this panel which contains the specified SWidget
		 *
		 * @param InSlotWidget The widget to match when searching through the slots
		 * @returns The index in the children array where the slot was removed and INDEX_NONE if no slot was found matching the widget
		 */
		int32 RemoveSlot(const TSharedRef<SWidget>& InSlotWidget);

		/** Removes a slot from this panel at the specified slot index
		 *
		 * @param InIndex The index of the slot to remove
		 * @returns The index in the children array where the slot was removed and INDEX_NONE if there was no slot at that index
		 */
		int32 RemoveSlotAt(int32 InIndex);

		/** Removes all children from the panel. */
		void ClearChildren();

		/** @return the number of slots. */
		int32 NumSlots() const;

		/** @return if it's a valid index slot index. */
		bool IsValidSlotIndex(int32 InIndex) const;

	private:
		float GetMinHeight(TSharedPtr<SContentBrowserSourceTreeArea> InEntryWidget) const;
		float GetTotalHeaderHeight() const;
		SSplitter::ESizeRule GetExpandedSizeRule(TSharedPtr<SContentBrowserSourceTreeArea> InEntryWidget) const;
		float GetSlotSize(const int32 InSlotIdx) const;
		void OnSlotResized(float InNewSize, const int32 InSlotIdx);
		void UpdateTotalHeaderHeight();

	private:
		TSharedPtr<SSplitter> Splitter;
		TPanelChildren<FSlot> Slots;

		/** Cached total header height, recalculated one Slot Add/Remove. */
		float TotalHeaderHeight;
	};

	/** Represents a single item in the source tree view, ie. "Favorites". */
	class SContentBrowserSourceTreeArea	: public SCompoundWidget
	{
		using Super = SCompoundWidget;

	public:
		SLATE_BEGIN_ARGS(SContentBrowserSourceTreeArea)
			: _IsEmpty(false)
			, _bEnableShadowBoxStyle(false)
		{ }
			/** Label to display in the header, ie. "Favorites" */
			SLATE_ATTRIBUTE(FText, Label)

			/** (Optional) content displayed to the right of the header label. */
			SLATE_NAMED_SLOT(FArguments, HeaderContent)

			/** (Optional) bind to notify when this widget should switch to an empty state. */
			SLATE_ATTRIBUTE(bool, IsEmpty)

			/** (Optional) label to display in the body area when the body is empty, shown as RichText. */
			SLATE_ATTRIBUTE(FText, EmptyBodyLabel)

			/** (Optional) default state for area expansion. */
			SLATE_ARGUMENT(bool, ExpandedByDefault)

			/** (Optional) override for STreeView ScrollBorder style */
			SLATE_STYLE_ARGUMENT(FScrollBorderStyle, ShadowBoxStyle)
			
			/** (Optional) If true, ScrollBorder will use the style set in BodyScrollBorderStyle */
			SLATE_ARGUMENT(bool, bEnableShadowBoxStyle)

			/** Called when the area is expanded or collapsed */
			SLATE_EVENT(FOnBooleanValueChanged, OnExpansionChanged)
		SLATE_END_ARGS()

		/**
		 * Constructs this widget with InArgs.
		 * @param Id		Used to identify the role of this widget (Favorites, etc.) for config settings, etc.
		 * @param InSearch  (Optional) Search for this area and it's contained items.
		 * @param InBody	The Body widget, which must implement IScrollableWidget
		 */
		void Construct(const FArguments& InArgs, const FName InId, const TSharedPtr<FSourcesSearch>& InSearch, TSharedRef<IScrollableWidget> InBody);

		bool IsExpanded() const;
		void SetExpanded(bool bInExpanded);

		/** Returns true if the contents is "Empty", regardless of expansion state. */
		bool IsEmpty() const;

		/** Saves all persistent settings to config. */
		void SaveSettings(const FString& InIniFilename, const FString& InIniSection, const FString& InSettingsString) const;

		/** Loads settings from config based on the browser's InstanceName. */
		void LoadSettings(const FString& InIniFilename, const FString& InIniSection, const FString& InSettingsString);

		bool HasSearch() const { return Search.IsValid(); }
		
		virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
		virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
		virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

		TSharedPtr<SSearchToggleButton> GetSearchToggleButton() const;

	private:
		EVisibility GetHeaderSearchActionVisibility() const;

		void OnAreaExpansionChanged(bool bInIsExpanded);

	private:
		FName Id;

		TAttribute<bool> bIsEmpty;

		/** The (optional) source search. */
		TSharedPtr<FSourcesSearch> Search = nullptr;

		/** Toggle button for showing/hiding the search area, only visible if Search IsValid. */
		TSharedPtr<SSearchToggleButton> SearchToggleButton = nullptr;

		TSharedPtr<IScrollableWidget> BodyScrollableWidget;

		TSharedPtr<SExpandableArea> ExpandableArea;

		FOnBooleanValueChanged OnExpansionChanged;

		bool bExpandedByDefault = false;
	};
}
