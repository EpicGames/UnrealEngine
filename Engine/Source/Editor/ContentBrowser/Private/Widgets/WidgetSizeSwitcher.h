// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"

namespace UE::Editor::ContentBrowser::Private
{
	/**
	 * Returns a size override to bind to based on a single subject widget's desired size, clamped between a provided size range.
	 * If a reference widget is provided, it's (allotted, not desired) size is used for the range max value.
	 * Axis is 0 for x/width, 1 for y/height.
	 */
	template <EAxis::Type Axis>
	class TWidgetDesiredSizeSwitcher
	{
		static_assert(Axis == EAxis::X || Axis == EAxis::Y, "Axis must be 0 (X/Width) or 1 (Y/Height).");

	public:
		virtual ~TWidgetDesiredSizeSwitcher() = default;

		TWidgetDesiredSizeSwitcher(
			const TSharedPtr<SWidget>& InSubjectWidget,
			const TSharedPtr<SWidget>& InMaxSizeReferenceWidget,
			const FInt16Range& InSizeRange)
			: SubjectWidget(InSubjectWidget)
			, MaxSizeReferenceWidget(InMaxSizeReferenceWidget)
			, SizeRange(InSizeRange)
		{
		}

		void SetMaxSizeReferenceWidget(const TSharedPtr<SWidget>& InWidget)
		{
			if (InWidget != MaxSizeReferenceWidget)
			{
				MaxSizeReferenceWidget = InWidget;
			}
		}

		void SetSizeRange(const FInt16Range& InRange)
		{
			if (SizeRange != InRange)
			{
				SizeRange = InRange;
			}
		}

		float GetDesiredSizeOverride()
		{
			const int16 CurrentSubjectWidgetDesiredSize = static_cast<int16>(FMath::FloorToInt(SubjectWidget->GetDesiredSize()[Axis - 1] - UE_KINDA_SMALL_NUMBER));
			const int16 MaxSize =
				MaxSizeReferenceWidget.IsValid()
				? static_cast<int16>(FMath::FloorToInt(MaxSizeReferenceWidget->GetPaintSpaceGeometry().Size[Axis - 1] - UE_KINDA_SMALL_NUMBER))
				: SizeRange.GetUpperBound().GetValue();

			// Check if the subject or reference widget (max size) has changed
			if (CurrentSubjectWidgetDesiredSize == LastSubjectWidgetDesiredSize
				&& MaxSize == LastMaxSize)
			{
				return LastDesiredSizeOverride;
			}

			LastMaxSize = MaxSize;
			LastSubjectWidgetDesiredSize = CurrentSubjectWidgetDesiredSize;

			const int16 MinSize = SizeRange.GetLowerBound().GetValue();

			LastDesiredSizeOverride = LastSubjectWidgetDesiredSize > MinSize ? MaxSize : MinSize;

			return LastDesiredSizeOverride;
		}

	protected:
		TSharedPtr<SWidget> SubjectWidget;
		TSharedPtr<SWidget> MaxSizeReferenceWidget;
		FInt16Range SizeRange;

		int16 LastMaxSize = INDEX_NONE;
		float LastDesiredSizeOverride = -1.0f;
		int16 LastSubjectWidgetDesiredSize = INDEX_NONE;
	};
}
