// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IContentBrowserSingleton.h"

#include "Misc/StringBuilder.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Misc/TextFilterUtils.h"

class FAssetTextFilterContext;
class ICollectionContainer;
class FCompiledAssetTextFilter;

/** 
 * Class handling text filtering for content browser views.
 * Unlike FFrontendFilter, guarantees that filtering can be performed concurrently.
 * @see FAssetTextFilterContext used to share the settings of this filter across multiple parallel workers.
 */
class FAssetTextFilter : public TSharedFromThis<FAssetTextFilter>
{
public:
    CONTENTBROWSER_API FAssetTextFilter();
 	CONTENTBROWSER_API ~FAssetTextFilter();

	// Cannot be moved or copied because of bound delegates.
	FAssetTextFilter(FAssetTextFilter&& Other) = delete;
	FAssetTextFilter& operator=(FAssetTextFilter&& Other) = delete;

	FAssetTextFilter(const FAssetTextFilter& Other) = delete;
	FAssetTextFilter& operator=(const FAssetTextFilter& Other) = delete;

public:
	/**
	 * Create a compiled filter which can safely be used on other threads.
	 * Combines the primary text filter with custom saved filters. 
	 */
	TSharedPtr<FCompiledAssetTextFilter> Compile();

	/** Returns true if the filter contains no primary text and no custom saved queries */
	bool IsEmpty() const;

	/** Provides a set of saved filters/queries to be performed asynchronously alongside the main text filtering. */
	void SetCustomTextFilters(TArray<FText> InQueries);

	/** Returns the unsanitized and unsplit filter terms */
	CONTENTBROWSER_API FText GetRawFilterText() const;

	/** Set the Text to be used as the Filter's restrictions */
	CONTENTBROWSER_API void SetRawFilterText(const FText& InFilterText);

	/** Get the last error returned from lexing or compiling the current filter text */
	CONTENTBROWSER_API FText GetFilterErrorText() const;

	/** If bIncludeClassName is true, the text filter will include an asset's class name in the search */
	CONTENTBROWSER_API void SetIncludeClassName(const bool InIncludeClassName);

	/** If bIncludeAssetPath is true, the text filter will match against full Asset path */
	CONTENTBROWSER_API void SetIncludeAssetPath(const bool InIncludeAssetPath);

    /** Returns the last value set with SetIncludeAssetPath */
	CONTENTBROWSER_API bool GetIncludeAssetPath() const;

	/** If bIncludeCollectionNames is true, the text filter will match against collection names as well */
	CONTENTBROWSER_API void SetIncludeCollectionNames(const bool InIncludeCollectionNames);

    /** Returns the last value set with SetIncludeCollectionNames */
	CONTENTBROWSER_API bool GetIncludeCollectionNames() const;

	/** Delegate to bind for when the effective text filter changes so filtering can be re-run. */
	FSimpleMulticastDelegate& OnChanged() { return ChangedEvent; }

private:
	/** Handles an on collection container created event */
	void HandleCollectionContainerCreated(const TSharedRef<ICollectionContainer>& CollectionContainer);

	/** Handles an on collection container destroyed event */
	void HandleCollectionContainerDestroyed(const TSharedRef<ICollectionContainer>& CollectionContainer);

	/** Handles an on collection container is hidden changed event */
	void HandleIsHiddenChanged(ICollectionContainer& CollectionContainer, bool bIsHidden);

	/** Handles an on collection created event */
	void HandleCollectionCreated(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection);

	/** Handles an on collection destroyed event */
	void HandleCollectionDestroyed(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection);

	/** Handles an on collection renamed event */
	void HandleCollectionRenamed(ICollectionContainer& CollectionContainer, const FCollectionNameType& OriginalCollection, const FCollectionNameType& NewCollection);

	/** Handles an on collection updated event */
	void HandleCollectionUpdated(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection);

	void BroadcastChangedEvent() const { ChangedEvent.Broadcast(); }

	/** An array of collection containers that are being monitored */
	TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;

	/** An array of dynamic collections that are being referenced by the current query. These should be tested against each asset when it's looking for collections that contain it */
	TArray<FCollectionRef> ReferencedDynamicCollections;

	/** 
	 * Expression evaluator that can be used to perform complex text filter queries.
	 * When CustomTextFilters is empty, this filter can be used as-is, otherwise it is necessary to compiled a combined evaluator.
	 */
	FTextFilterExpressionEvaluator TextFilterExpressionEvaluator;

	/** Additional queries from saved user filters. These and any primary text filter are combined with AND semantics. */
	TArray<FText> CustomTextFilters;

	/** Delegate handles */
	FDelegateHandle OnCollectionContainerCreatedHandle;
	FDelegateHandle OnCollectionContainerDestroyedHandle;

	struct FCollectionContainerHandles
	{
		FDelegateHandle OnIsHiddenChangedHandle;
		FDelegateHandle OnCollectionCreatedHandle;
		FDelegateHandle OnCollectionDestroyedHandle;
		FDelegateHandle OnCollectionRenamedHandle;
		FDelegateHandle OnCollectionUpdatedHandle;
	};
	TArray<FCollectionContainerHandles> CollectionContainerHandles;
    
    // Filter options
    bool bIncludeClassName = false;
    bool bIncludeAssetPath = false;
    bool bIncludeCollectionNames = false;

	bool bReferencedDynamicCollectionsDirty = true;

	FSimpleMulticastDelegate ChangedEvent;
};


/** 
 * Context object for parallel filtering
 * Allows reuse of memory when filtering in ParallelForWithTaskContext
 */
class FCompiledAssetTextFilter : private ITextFilterExpressionContext, public TSharedFromThis<FCompiledAssetTextFilter>
{
	typedef TRemoveReference<FAssetFilterType>::Type* FAssetFilterTypePtr;
    friend class FAssetTextFilter;

	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FCompiledAssetTextFilter(
		FPrivateToken,
		TSharedRef<const FTextFilterExpressionEvaluator> InSharedEvaluator,
		TSharedPtr<const TArray<FCollectionRef>> InReferencedDynamicCollections,
		TSharedPtr<const TArray<TSharedPtr<ICollectionContainer>>> InCollectionContainers,
		bool InIncludeClassName,
		bool InIncludeAssetPath,
		bool InIncludeCollectionNames);
    virtual ~FCompiledAssetTextFilter() = default;

    FCompiledAssetTextFilter(FCompiledAssetTextFilter&&) = default;
    FCompiledAssetTextFilter& operator=( FCompiledAssetTextFilter&&) = default;

	/** 
	 * Clone a copy of this which shares filtering data but can be used on a different thread to the original. 
	 * Each FCompiledAssetTextFilter should only be used on a single thread at once. 
	 */
	FCompiledAssetTextFilter CloneForThreading();

	/**
     * Non-const method to check an item against the filter, using member fields to amortize memory allocations for
	 * many such checks 
     */
	bool PassesFilter(FAssetFilterType InItem);
	bool PassesFilter(FAssetFilterType InItem, FStringView InItemVersePath);

	/** Test the given value against the strings extracted from the current item */
	virtual bool TestBasicStringExpression(
		const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override;

	/** Perform a complex expression test for the current item */
	virtual bool TestComplexExpression(
		const FName& InKey,
		const FTextFilterString& InValue,
		const ETextFilterComparisonOperation InComparisonOperation,
		const ETextFilterTextComparisonMode InTextComparisonMode) const override;
private:
    // Private copying for explicit clone operation
    FCompiledAssetTextFilter(FCompiledAssetTextFilter&) = default;
    FCompiledAssetTextFilter& operator=(const FCompiledAssetTextFilter&) = default;

    /** Compiled text query */
    TSharedRef<const FTextFilterExpressionEvaluator> Evaluator;

	/** Dynamic collections referenced by the compiled query */
	TSharedPtr<const TArray<FCollectionRef>> ReferencedDynamicCollections;

	/** Cached Collection containers */
	TSharedPtr<const TArray<TSharedPtr<ICollectionContainer>>> CollectionContainers;

    // Filter options
    bool bIncludeClassName = false;
    bool bIncludeAssetPath = false;
    bool bIncludeCollectionNames = false;
    
	// State for PassesFilter

	bool bIsFile = false;

	/** Pointer to the asset we're currently filtering */
	FAssetFilterTypePtr AssetPtr;
    
    /** Shared buffer for text keys to search */
    FString TextBuffer;

	/** Recyclable text filter string so reduce allocations  */
	mutable FTextFilterString TextFilterString;

	/** Display name of the current asset */
	FStringView AssetDisplayName;
	/** Full path of the current asset */
	FStringView AssetFullPath;
	/** Virtual path of the current asset */
	FStringView AssetVirtualPath;
	/** The export text name of the current asset */
	FStringView AssetExportTextPath;
	/** Verse path of the current asset */
	FStringView AssetVersePath;

	/** Names of the collections that the current asset is in */
	TArray<FName> AssetCollectionNames;
};

/**
 * Interface that can be implemented to extend text filtering in the content browser.
 *
 * Objects implementing this interface must be manually added and removed from participation in text filtering with
 * RegisterHandler and UnregisterHandler to allow synchronization with async text filtering.
 *
 * Objects implementing this interface must be able to have HandleTextFilterValue and HandleTextFilterKeyValue called on
 * any thread in between RegisterHandler and UnregisterHandler
 */
class IAssetTextFilterHandler
{
public:
	CONTENTBROWSER_API virtual ~IAssetTextFilterHandler() = default;

	/**
	 * Implement this function to handle basic text matching.
	 *
	 * @param InContentBrowserItem The item to be filtered.
	 * @param InValue The text string the user wants to filter items against.
	 * @param InTextComparisonMode How to compare text e.g. prefix/suffix/contains
	 * @param bOutIsMatch Whether the item matches the user's query
	 * @return true If this handler provided an answer for whether the item matches. bOutIsMatch may be true or false in this case.
	 * @return false If this handler did not decide whether this item was filtered, and other handlers or the built-in implementation should continue.
	 */
	virtual bool HandleTextFilterValue(const FContentBrowserItem& InContentBrowserItem,
		const FTextFilterString& InValue,
		const ETextFilterTextComparisonMode InTextComparisonMode,
		bool& bOutIsMatch) = 0;

	/**
	 * Implement this function to handle complex tests of keys against values (e.g. class=actor)
	 *
	 * @param InContentBrowserItem The item to be filtered.
	 * @param InKey The key identifying what data from InContentBrowserItem to compare against.
	 * @param InValue The user's provided text to compare the item against.
	 * @param InComparisonOperation Comparison operator for numeric comparisons.
	 * @param InTextComparisonMode How to compare text e.g. prefix/suffix/contains
	 * @param bOutIsMatch Whether the item matches the user's query
	 * @return true If this handler provided an answer for whether the item matches. bOutIsMatch may be true or false in this case.
	 * @return false If this handler did not decide whether this item was filtered, and other handlers or the built-in implementation should continue.
	 */
	virtual bool HandleTextFilterKeyValue(const FContentBrowserItem& InContentBrowserItem,
		const FName& InKey,
		const FTextFilterString& InValue,
		const ETextFilterComparisonOperation InComparisonOperation,
		const ETextFilterTextComparisonMode InTextComparisonMode,
		bool& bOutIsMatch) = 0;

	/** Call on your derived instance of IAssetTextFilterHandler to have it participate in asset text filtering. */
	CONTENTBROWSER_API void RegisterHandler();
	/** Call on your derived instance of IAssetTextFilterHandler to remove it from asset text filtering. */
	CONTENTBROWSER_API void UnregisterHandler();
};
