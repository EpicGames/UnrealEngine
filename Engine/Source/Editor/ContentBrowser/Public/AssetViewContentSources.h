// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CollectionManagerTypes.h"

struct FAssetViewContentSources
{
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnEnumerateCustomSourceItemDatas, TFunctionRef<bool(class FContentBrowserItemData&&)>)

	FOnEnumerateCustomSourceItemDatas OnEnumerateCustomSourceItemDatas;
	bool bIncludeVirtualPaths = true;

	FAssetViewContentSources() = default;

	CONTENTBROWSER_API explicit FAssetViewContentSources(FName InVirtualPath);
	CONTENTBROWSER_API explicit FAssetViewContentSources(TArray<FName> InVirtualPaths);

	CONTENTBROWSER_API explicit FAssetViewContentSources(FCollectionRef InCollection);
	CONTENTBROWSER_API explicit FAssetViewContentSources(TArray<FCollectionRef> InCollections);

	CONTENTBROWSER_API FAssetViewContentSources(TArray<FName> InVirtualPaths, TArray<FCollectionRef> InCollections);

	FAssetViewContentSources(const FAssetViewContentSources&) = default;
	FAssetViewContentSources(FAssetViewContentSources&&) = default;

	FAssetViewContentSources& operator=(const FAssetViewContentSources&) = default;
	FAssetViewContentSources& operator=(FAssetViewContentSources&&) = default;

	bool IsEmpty() const
	{
		return VirtualPaths.IsEmpty() && Collections.IsEmpty();
	}

	bool HasVirtualPaths() const
	{
		return !VirtualPaths.IsEmpty();
	}

	const TArray<FName>& GetVirtualPaths() const
	{
		return VirtualPaths;
	}

	CONTENTBROWSER_API void SetVirtualPath(FName InVirtualPath);
	CONTENTBROWSER_API void SetVirtualPaths(const TArray<FName>& InVirtualPaths);

	bool HasCollections() const
	{
		return !Collections.IsEmpty();
	}

	const TArray<FCollectionRef>& GetCollections() const
	{
		return Collections;
	}

	CONTENTBROWSER_API void SetCollection(const FCollectionRef& InCollection);
	CONTENTBROWSER_API void SetCollections(const TArray<FCollectionRef>& InCollections);

	bool IsIncludingVirtualPaths() const
	{
		return bIncludeVirtualPaths;
	}

	CONTENTBROWSER_API bool IsDynamicCollection() const;

	CONTENTBROWSER_API void Reset();

private:
	void SanitizeCollections();

	TArray<FName> VirtualPaths;
	TArray<FCollectionRef> Collections;
};
