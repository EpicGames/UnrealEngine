// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StringView.h"
#include "ContentBrowserDelegates.h"
#include "CoreMinimal.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "Widgets/Views/SHeaderRow.h"

#define UE_API CONTENTBROWSER_API

class FAssetViewItem;
class FString;
struct FAssetViewCustomColumn;

class FAssetViewSortManager
{
public:
	/** Constructor */
	UE_API FAssetViewSortManager();

	/** Reset the sort mode back to default */
	UE_API void ResetSort();

	/** Sorts a list using the current ColumnId and Mode. Supply a MajorityAssetType to help discover sorting type (numerical vs alphabetical) */
	UE_API void SortList(TArray<TSharedPtr<FAssetViewItem>>& AssetItems, const FName& MajorityAssetType, const TArray<FAssetViewCustomColumn>& CustomColumns) const;

	/** Exports the list of asset items to CSV, in order and with the listed columns */
	UE_API void ExportColumnsToCSV(TArray<TSharedPtr<FAssetViewItem>>& AssetItems, TArray<FName>& ColumnList, const TArray<FAssetViewCustomColumn>& CustomColumns, FString& OutString) const;

	/** Sets the sort mode based on the column that was clicked, returns true if newly assigned */
	UE_API bool SetOrToggleSortColumn(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId);

	/**	Sets the column to sort */
	UE_API void SetSortColumnId(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId);

	/**	Sets the current sort mode */
	UE_API void SetSortMode(const EColumnSortPriority::Type InSortPriority, const EColumnSortMode::Type InSortMode);

	/** Gets the current sort mode */
	EColumnSortMode::Type GetSortMode(const EColumnSortPriority::Type InSortPriority) const { check(InSortPriority < EColumnSortPriority::Max); return SortMode[InSortPriority]; }

	/** Gets the current sort column id */
	FName GetSortColumnId(const EColumnSortPriority::Type InSortPriority) const { check(InSortPriority < EColumnSortPriority::Max); return SortColumnId[InSortPriority]; }

	/** Refresh a custom column if needed.  If found, returns true with TagType parameter set */
	UE_API bool FindAndRefreshCustomColumn(const TArray<TSharedPtr<FAssetViewItem>>& AssetItems, const FName ColumnName, const TArray<FAssetViewCustomColumn>& CustomColumns, UObject::FAssetRegistryTag::ETagType& TagType) const;

public:
	/** The names of non-type specific columns in the columns view. */
	static UE_API const FName NameColumnId;
	static UE_API const FName ClassColumnId;
	static UE_API const FName PathColumnId;
	static UE_API const FName DiskSizeColumnId;
	static UE_API const FName AssetAccessSpecifierColumnId;

	// The revision control column. NOTE: This column currently doesn't support sorting, but is wired through the Sort Manager so the feature can be added in the future
	static UE_API const FName RevisionControlColumnId;

private:
	/** The name of the column that is currently used for sorting. */
	FName SortColumnId[EColumnSortPriority::Max];

	/** Whether the sort is ascending or descending. */
	EColumnSortMode::Type SortMode[EColumnSortPriority::Max];
};

#undef UE_API
