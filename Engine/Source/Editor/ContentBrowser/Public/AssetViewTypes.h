// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "ContentBrowserItem.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Templates/Tuple.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

#include <atomic>

enum class EAssetAccessSpecifier : uint8;
class FContentBrowserItemData;
class FString;
class FText;
struct FAssetViewCustomColumn;

enum class EFolderType : uint8
{
	Normal,
	CustomVirtual, // No corresponding on-disk path, used for organization in the content browser
	PluginRoot,    // Root content folder of a plugin
	Code,
	Developer,
};

ENUM_CLASS_FLAGS(EFolderType)

/** An item (folder or file) displayed in the asset view */
class FAssetViewItem
{
public:
	FAssetViewItem(int32 Index);

	explicit FAssetViewItem(int32 Index, FContentBrowserItem&& InItem);
	explicit FAssetViewItem(int32 Index, const FContentBrowserItem& InItem);

	explicit FAssetViewItem(int32 Index, FContentBrowserItemData&& InItemData);
	explicit FAssetViewItem(int32 Index, const FContentBrowserItemData& InItemData);

	// When recycling an object, clear the item data and replace it with the given data.
	void ResetItemData(int32 OldIndex, int32 Index, FContentBrowserItemData InItemData);

	void AppendItemData(const FContentBrowserItem& InItem);

	void AppendItemData(const FContentBrowserItemData& InItemData);

	void RemoveItemData(const FContentBrowserItem& InItem);

	void RemoveItemData(const FContentBrowserItemData& InItemData);

	void RemoveItemData(const FContentBrowserMinimalItemData& InItemKey);

	/** Configures the type of path GetAssetPathText should return. */
	UE_INTERNAL void ConfigureAssetPathText(bool bInShowingContentVersePath);

	/** Caches the Verse path to use when filtering items. */
	UE_INTERNAL void CacheVersePathTextForTextFiltering();

	/** Gets the asset class name to use when viewing. */
	UE_INTERNAL FText GetAssetClassText() const;

	/** Gets the asset path to use when viewing or sorting the path column. */
	UE_INTERNAL const FText& GetAssetPathText() const;

	/** Gets the Verse path to use when filtering items. Requires a call to CacheVersePathTextForTextFiltering first. */
	UE_INTERNAL const FText& GetVersePathTextForTextFiltering() const;

	/** Clear cached custom column data */
	void ClearCachedCustomColumns();

	/**
	 * Updates cached custom column data (only does something for files) 
	 * @param bUpdateExisting If true, only updates existing columns, if false only adds missing columns 
	 */
	void CacheCustomColumns(TArrayView<const FAssetViewCustomColumn> CustomColumns, const bool bUpdateSortData, const bool bUpdateDisplayText, const bool bUpdateExisting);
	
	/** Get the display value of a custom column on this item */
	bool GetCustomColumnDisplayValue(const FName ColumnName, FText& OutText) const;

	/** Get the value (and optionally also the type) of a custom column on this item */
	bool GetCustomColumnValue(const FName ColumnName, FString& OutString, UObject::FAssetRegistryTag::ETagType* OutType = nullptr) const;

	/** Get the value (and optionally also the type) of a named tag on this item */
	bool GetTagValue(const FName Tag, FString& OutString, UObject::FAssetRegistryTag::ETagType* OutType = nullptr) const;

	/**
	 * Query the EAssetAccessSpecifier of the item.
	 *
	 * @param OutAssetAccessSpecifier The EAssetAccessSpecifier reference that will be filled.
	 *
	 * @return True if OutAssetAccessSpecifier was filled.
	 */
	bool GetItemAssetAccessSpecifier(EAssetAccessSpecifier& OutAssetAccessSpecifier) const;
	
	/**
	 * Query a readable FText of the EAssetAccessSpecifier for the item.
	 
	 * @return If the item has a modifiable EAssetAccessSpecifier, then a readable text version. Otherwise blank.
	 */
	FText GetItemAssetAccessSpecifierText() const;

	/**
	 * Query the icon brush style name to use for the EAssetAccessSpecifier of the item.

	 * @return If the item has a modifiable EAssetAccessSpecifier, then the style name of the brush to use. Otherwise NAME_None.
	 */
	FName GetItemAssetAccessSpecifierIconStyleName(const TCHAR* InSizeSuffix = TEXT("")) const;

	/**
	 * Query the item's ability to modify its EAssetAccessSpecifier.
	 *
	 * @return True if the item has an EAssetAccessSpecifier that can be modified.
	 */
	bool CanModifyItemAssetAccessSpecifier() const;

	 /** Returns the brush style name to use for a public asset access specifier */
	static FName GetPublicAssetAccessSpecifierIconStyleName(const TCHAR* InSizeSuffix = TEXT(""));

	/** Returns the brush style name to use for a private asset access specifier */
	static FName GetPrivateAssetAccessSpecifierIconStyleName(const TCHAR* InSizeSuffix = TEXT(""));

	/** Returns the brush style name to use for an Epic Internal asset access specifier */
	static FName GetEpicInternalAssetAccessSpecifierIconStyleName(const TCHAR* InSizeSuffix = TEXT(""));

	/** Get the underlying Content Browser item */
	CONTENTBROWSER_API const FContentBrowserItem& GetItem() const;

	bool IsFolder() const;

	bool IsFile() const;

	bool IsTemporary() const;

	// Called when the view explicitly wants to notify widgets of changes
	// Not called during bulk rebuilds when the view will be re-populated even if items are being recycled
	// Updates cached paths then raises the OnItemDataChanged event.
	void BroadcastItemDataChanged();

	/** Get the event fired when the data for this item changes */
	FSimpleMulticastDelegate& OnItemDataChanged();

	/** Get the event fired whenever a rename is requested */
	FSimpleDelegate& OnRenameRequested();

	/** Get the event fired whenever a rename is canceled */
	FSimpleDelegate& OnRenameCanceled();

	/** Helper function to turn an item into a string for debugging */
	static FString ItemToString_Debug(TSharedPtr<FAssetViewItem> AssetItem);

private:
	void CacheAssetPathText();
	void CacheVersePathText() const;

	/** Underlying Content Browser item data */
	FContentBrowserItem Item;

	/**
	 * Index at which this is stored in the asset view's item collection.
	 * Can be used to detect an item being added to the collection twice by mistake.
	 */
	std::atomic<int32> Index;

	bool bShowingContentVersePath;
	mutable bool bHasCachedVersePathText;
	FText CachedAssetPathText;
	mutable FText CachedVersePathText;

	/** An event to fire when the data for this item changes */
	FSimpleMulticastDelegate ItemDataChangedEvent;

	/** Broadcasts whenever a rename is requested */
	FSimpleDelegate RenameRequestedEvent;

	/** Broadcasts whenever a rename is canceled */
	FSimpleDelegate RenameCanceledEvent;

	/** Map of values/types for custom columns */
	TMap<FName, TTuple<FString, UObject::FAssetRegistryTag::ETagType>> CachedCustomColumnData;
	
	/** Map of display text for custom columns */
	TMap<FName, FText> CachedCustomColumnDisplayText;
};
