// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "AssetThumbnail.h"
#include "AssetViewSortManager.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "ContentBrowserDelegates.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Geometry.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SlateFwd.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STileView.h"

class FAssetViewItem;
class FDragDropEvent;
class FSlateRect;
class FString;
class IToolTip;
class SAssetListItem;
class SAssetTileItem;
class SInlineEditableTextBlock;
class SLayeredImage;
class SVerticalBox;
struct FKeyEvent;
struct FPointerEvent;
struct FSlateBrush;
template <typename ItemType> class SListView;

DECLARE_DELEGATE_ThreeParams( FOnRenameBegin, const TSharedPtr<FAssetViewItem>& /*AssetItem*/, const FString& /*OriginalName*/, const FSlateRect& /*MessageAnchor*/)
DECLARE_DELEGATE_FourParams( FOnRenameCommit, const TSharedPtr<FAssetViewItem>& /*AssetItem*/, const FString& /*NewName*/, const FSlateRect& /*MessageAnchor*/, ETextCommit::Type /*CommitType*/ )
DECLARE_DELEGATE_RetVal_FourParams( bool, FOnVerifyRenameCommit, const TSharedPtr<FAssetViewItem>& /*AssetItem*/, const FText& /*NewName*/, const FSlateRect& /*MessageAnchor*/, FText& /*OutErrorMessage*/)
DECLARE_DELEGATE_OneParam( FOnItemDestroyed, const TSharedPtr<FAssetViewItem>& /*AssetItem*/);

class SAssetListItem;
class SAssetTileItem;

struct FAssetViewItemHelper
{
public:
	static TSharedRef<SWidget> CreateListItemContents(SAssetListItem* const InListItem, const TSharedRef<SWidget>& InThumbnail, FName& OutItemShadowBorder, const FMargin& InFolderOverlayPadding = FMargin(0.f));
	static TSharedRef<SWidget> CreateTileItemContents(SAssetTileItem* const InTileItem, const TSharedRef<SWidget>& InThumbnail, FName& OutItemShadowBorder, const FMargin& InFolderOverlayPadding = FMargin(0.f), bool bShowAssetAccessSpecifier = false);

private:
	template <typename T>
	static TSharedRef<SWidget> CreateListTileItemContents(T* const InTileOrListItem, const TSharedRef<SWidget>& InThumbnail, FName& OutItemShadowBorder, const FMargin& InFolderOverlayPadding, bool bShowAssetAccessSpecifier);
};


template <>
struct TWidgetTypeTraits<class SAssetTileView>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** The tile view mode of the asset view */
class SAssetTileView : public STileView<TSharedPtr<FAssetViewItem>>
{
public:
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
};


template <>
struct TWidgetTypeTraits<class SAssetListView>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** The list view mode of the asset view */
class SAssetListView : public SListView<TSharedPtr<FAssetViewItem>>
{
public:
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
};


template <>
struct TWidgetTypeTraits<class SAssetColumnView>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** The columns view mode of the asset view */
class SAssetColumnView : public SListView<TSharedPtr<FAssetViewItem>>
{
public:
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
};


template <>
struct TWidgetTypeTraits<class SAssetViewItem>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** A base class for all asset view items */
class SAssetViewItem : public SCompoundWidget
{
	friend class SAssetViewItemToolTip;

public:

	SLATE_BEGIN_ARGS( SAssetViewItem )
		: _ShouldAllowToolTip(true)
		, _ThumbnailEditMode(false)
		, _ShowAssetAccessSpecifier(false)
		{}

		/** Data for the asset this item represents */
	SLATE_ARGUMENT(TSharedPtr<FAssetViewItem>, AssetItem)

		/** Delegate for when an asset name has entered a rename state */
		SLATE_EVENT(FOnRenameBegin, OnRenameBegin)

		/** Delegate for when an asset name has been entered for an item that is in a rename state */
		SLATE_EVENT(FOnRenameCommit, OnRenameCommit)

		/** Delegate for when an asset name has been entered for an item to verify the name before commit */
		SLATE_EVENT(FOnVerifyRenameCommit, OnVerifyRenameCommit)

		/** Called when any asset item is destroyed. Used in thumbnail management */
		SLATE_EVENT(FOnItemDestroyed, OnItemDestroyed)

		/** If false, the tooltip will not be displayed */
		SLATE_ATTRIBUTE(bool, ShouldAllowToolTip)

		/** If true, display the thumbnail edit mode UI */
		SLATE_ATTRIBUTE(bool, ThumbnailEditMode)

		/** If true, display the item's EAssetAccessSpecifier */
		SLATE_ARGUMENT(bool, ShowAssetAccessSpecifier)

		/** The string in the title to highlight (used when searching by string) */
		SLATE_ATTRIBUTE(FText, HighlightText)

		/** Delegate to call (if bound) to check if it is valid to get a custom tooltip for this view item */
		SLATE_EVENT(FOnIsAssetValidForCustomToolTip, OnIsAssetValidForCustomToolTip)

		/** Delegate to call (if bound) to get a custom tooltip for this view item */
		SLATE_EVENT(FOnGetCustomAssetToolTip, OnGetCustomAssetToolTip)

		/** Delegate for when an item is about to show a tool tip */
		SLATE_EVENT(FOnVisualizeAssetToolTip, OnVisualizeAssetToolTip)

		/** Delegate for when an item's tooltip is about to close */
		SLATE_EVENT(FOnAssetToolTipClosing, OnAssetToolTipClosing)

		/** Delegate for getting the selection state of this item */ 
		SLATE_ARGUMENT(FIsSelected, IsSelected)

	SLATE_END_ARGS()

	/** Virtual destructor */
	virtual ~SAssetViewItem();

	/** Performs common initialization logic for all asset view items */
	void Construct( const FArguments& InArgs );

	/** NOTE: Any functions overridden from the base widget classes *must* also be overridden by SAssetColumnViewRow and forwarded on to its internal item */
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	virtual TSharedPtr<IToolTip> GetToolTip() override;
	virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent, const TSharedRef<SWidget>& InParentWidget);
	virtual bool OnVisualizeTooltip( const TSharedPtr<SWidget>& TooltipContent ) override;
	virtual void OnToolTipClosing() override;

	/** Returns the color this item should be tinted with */
	virtual FSlateColor GetAssetColor() const;

	/** Get the border image to display */
	virtual const FSlateBrush* GetBorderImage() const;

	/** Get the name text to be displayed for this item */
	FText GetNameText() const;

protected:
	/** Check to see if the name should be read-only */
	bool IsNameReadOnly() const;

	/** Handles starting a name change */
	virtual void HandleBeginNameChange( const FText& OriginalText );

	/** Handles committing a name change */
	virtual void HandleNameCommitted( const FText& NewText, ETextCommit::Type CommitInfo );

	/** Handles verifying a name change */
	virtual bool HandleVerifyNameChanged( const FText& NewText, FText& OutErrorMessage );

	/** Handles committing a name change */
	virtual void OnAssetDataChanged();

	/** Notification for when the dirty flag changes */
	virtual void DirtyStateChanged();

	/** Gets the name of the class of this asset */
	FText GetAssetClassText() const;

	/** Gets the brush for the dirty indicator image */
	const FSlateBrush* GetDirtyImage() const;

	/** Generates the source control icon widget */
	TSharedRef<SWidget> GenerateSourceControlIconWidget();

	/** Generate a widget to inject extra external state indicator on the asset. */
	TSharedRef<SWidget> GenerateExtraStateIconWidget(TAttribute<float> InMaxExtraStateIconWidth) const;

	/** Generate a widget to inject extra external state indicator on the asset tooltip. */
	TSharedRef<SWidget> GenerateExtraStateTooltipWidget() const;

	/** Generate a widget to inject extra external state indicator on the asset tooltip. Need to be TSharedPtr for AssetThumbnail */
	TSharedPtr<SWidget> GenerateExtraStateTooltipWidgetNewStyle() const;

	/** Gets the visibility for the thumbnail edit mode UI */
	EVisibility GetThumbnailEditModeUIVisibility() const;

	/** Creates a tooltip widget for this item */
	TSharedRef<SWidget> CreateToolTipWidget() const;

	/** Gets the visibility of the source control text block in the tooltip */
	EVisibility GetSourceControlTextVisibility() const;

	/** Gets the text for the source control text block in the tooltip */
	FText GetSourceControlText() const;

	/** Gets the text detailing all the modified external packages (and their contained objects) */
	FText GetExternalPackagesText() const;

	/** Helper function for CreateToolTipWidget. Gets the user description for the asset, if it exists. */
	FText GetAssetUserDescription() const;

	/** Helper function for CreateToolTipWidget. Adds a key value pair to the info box of the tooltip */
	void AddToToolTipInfoBox(const TSharedRef<SVerticalBox>& InfoBox, const FText& Key, const FText& Value, bool bImportant) const;

	/** Updates the bPackageDirty flag */
	void UpdateDirtyState();

	/** Returns true if the item is dirty. */
	bool IsDirty() const;

	/** Update the source control state of this item if required */
	void UpdateSourceControlState(float InDeltaTime);
	
	/** Cache the display tags for this item */
	void CacheDisplayTags();

	/** Whether this item is a folder */
	bool IsFolder() const;

	/** Delegate handler for when the source control provider changes */
	void HandleSourceControlProviderChanged(class ISourceControlProvider& OldProvider, class ISourceControlProvider& NewProvider);

	/** Delegate handler for when source control state changes */
	void HandleSourceControlStateChanged();

	/** Returns the width at which the name label will wrap the name */
	virtual float GetNameTextWrapWidth() const { return 0.0f; }

	/** Caches the ExternalPackagesText */
	void CacheDirtyExternalPackageInfo();

	/** If true it will expand the tooltip by default */
	bool ShouldExpandTooltip() const;

	/** Returns the SBox that will hold the AssetAccessSpecifierIcon */
	TSharedRef<SWidget> GetAssetAccessSpecifierIconBox() const;

	/** Renders the appropriate AssetAccessSpecifierIcon based on the EAssetAccessSpecifier of the asset */
	void UpdateAssetAccessSpecifierIcon();

	/** Gets the AssetAccessSpecifier as FText. */
	FText GetAssetAccessSpecifierText() const;

protected:
	/** Data for a cached display tag for this item (used in the tooltip, and also as the display string in column views) */
	struct FTagDisplayItem
	{
		FTagDisplayItem(FName InTagKey, FText InDisplayKey, FText InDisplayValue, const bool InImportant)
			: TagKey(InTagKey)
			, DisplayKey(MoveTemp(InDisplayKey))
			, DisplayValue(MoveTemp(InDisplayValue))
			, bImportant(InImportant)
		{
		}

		FName TagKey;
		FText DisplayKey;
		FText DisplayValue;
		bool bImportant;
	};

	TSharedPtr< SInlineEditableTextBlock > InlineRenameWidget;

	TSharedPtr<STextBlock> ClassTextWidget;

	/** The data for this item */
	TSharedPtr<FAssetViewItem> AssetItem;

	/** The cached display tags for this item */
	TArray<FTagDisplayItem> CachedDisplayTags;

	/** Delegate for when an asset name has entered a rename state */
	FOnRenameBegin OnRenameBegin;

	/** Delegate for when an asset name has been entered for an item that is in a rename state */
	FOnRenameCommit OnRenameCommit;

	/** Delegate for when an asset name has been entered for an item to verify the name before commit */
	FOnVerifyRenameCommit OnVerifyRenameCommit;

	/** Called when any asset item is destroyed. Used in thumbnail management */
	FOnItemDestroyed OnItemDestroyed;

	/** Called to test if it is valid to make a custom tool tip for that asset */
	FOnIsAssetValidForCustomToolTip OnIsAssetValidForCustomToolTip;

	/** Called if bound to get a custom asset item tooltip */
	FOnGetCustomAssetToolTip OnGetCustomAssetToolTip;

	/** Called if bound when about to show a tooltip */
	FOnVisualizeAssetToolTip OnVisualizeAssetToolTip;

	/** Called if bound when a tooltip is closing */
	FOnAssetToolTipClosing OnAssetToolTipClosing;
	
	/** Delegate for getting the selection state of this item */ 
	FIsSelected IsSelected;

	FGeometry LastGeometry;

	/** If false, the tooltip will not be displayed */
	TAttribute<bool> ShouldAllowToolTip;

	/** If true, display the thumbnail edit mode UI */
	TAttribute<bool> ThumbnailEditMode;

	/** The substring to be highlighted in the name and tooltip path */
	TAttribute<FText> HighlightText;

	/** Cached brushes for the dirty state */
	const FSlateBrush* AssetDirtyBrush;

	/** Cached flag describing if the item is dirty */
	bool bItemDirty;

	/** Flag indicating whether we have requested initial source control state */
	bool bSourceControlStateRequested;

	/** Delay timer before we request a source control state update, to prevent spam */
	float SourceControlStateDelay;

	/** True when a drag is over this item with a drag operation that we know how to handle. The operation itself may not be valid to drop. */
	bool bDraggedOver;

	/** Widget for the source control state */
	TSharedPtr<SLayeredImage> SCCStateWidget;

	/** Thumbnail tooltip widget used for asset if no given custom widget is passed */
	TSharedPtr<IToolTip> ThumbnailWidgetITooltip;

	/** Whether the item has a valid scc state brush (not empty) */
	bool bHasCCStateBrush = false;

	bool bShouldSaveExternalPackages = false;
	FString CachedDirtyPackagesList;

	/** Delegate handle for the HandleSourceControlStateChanged function callback */
	FDelegateHandle SourceControlStateChangedDelegateHandle;

	/** Whether to show the icon / text for the item's EAssetAccessSpecifier */
	bool bShowAssetAccessSpecifier = false;

	/** Widget holding the EAssetAccessSpecifier as an icon */
	TSharedPtr<SBox> AssetAccessSpecifierIconBox;

	/** Widget rendering the EAssetAccessSpecifier as text */
	TSharedPtr<STextBlock> AssetAccessSpecifierText;
};


template <>
struct TWidgetTypeTraits<class SAssetListItem>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** An item in the asset list view */
class SAssetListItem : public SAssetViewItem
{
	friend struct FAssetViewItemHelper;

public:
	SLATE_BEGIN_ARGS( SAssetListItem )
		: _ThumbnailPadding(0)
		, _ThumbnailLabel( EThumbnailLabel::ClassName )
		, _CurrentThumbnailSize(EThumbnailSize::Medium)
		, _ThumbnailHintColorAndOpacity( FLinearColor( 0.0f, 0.0f, 0.0f, 0.0f ) )
		, _ItemHeight(16)
		, _ShouldAllowToolTip(true)
		, _ThumbnailEditMode(false)
		, _AllowThumbnailHintLabel(false)
		, _ShowAssetAccessSpecifier(false)
		{}

		/** The handle to the thumbnail this item should render */
		SLATE_ARGUMENT( TSharedPtr<FAssetThumbnail>, AssetThumbnail)

		/** Data for the asset this item represents */
		SLATE_ARGUMENT( TSharedPtr<FAssetViewItem>, AssetItem )

		/** How much padding to allow around the thumbnail */
		SLATE_ARGUMENT( float, ThumbnailPadding )

		/** The contents of the label displayed on the thumbnail */
		SLATE_ARGUMENT( EThumbnailLabel::Type, ThumbnailLabel )

		/** Current size of the thumbnail that was generated */
		SLATE_ATTRIBUTE(EThumbnailSize, CurrentThumbnailSize)

		/**  */
		SLATE_ATTRIBUTE( FLinearColor, ThumbnailHintColorAndOpacity )

		/** The height of the list item */
		SLATE_ATTRIBUTE( float, ItemHeight )

		/** Delegate for when an asset name has entered a rename state */
		SLATE_EVENT( FOnRenameBegin, OnRenameBegin )

		/** Delegate for when an asset name has been entered for an item that is in a rename state */
		SLATE_EVENT( FOnRenameCommit, OnRenameCommit )

		/** Delegate for when an asset name has been entered for an item to verify the name before commit */
		SLATE_EVENT( FOnVerifyRenameCommit, OnVerifyRenameCommit )

		/** Called when any asset item is destroyed. Used in thumbnail management */
		SLATE_EVENT( FOnItemDestroyed, OnItemDestroyed )

		/** If false, the tooltip will not be displayed */
		SLATE_ATTRIBUTE( bool, ShouldAllowToolTip )

		/** The string in the title to highlight (used when searching by string) */
		SLATE_ATTRIBUTE( FText, HighlightText )

		/** If true, the thumbnail in this item can be edited */
		SLATE_ATTRIBUTE( bool, ThumbnailEditMode )

		/** Whether the thumbnail should ever show it's hint label */
		SLATE_ARGUMENT( bool, AllowThumbnailHintLabel )

		/** If true, display the item's EAssetAccessSpecifier */
		SLATE_ARGUMENT( bool, ShowAssetAccessSpecifier)

		/** Whether the item is selected in the view */
		SLATE_ARGUMENT(FIsSelected, IsSelected)

		/** Whether the item is selected in the view without anything else being selected*/
		SLATE_ARGUMENT(FIsSelected, IsSelectedExclusively)

		/** Delegate to call (if bound) to check if it is valid to get a custom tooltip for this view item */
		SLATE_EVENT(FOnIsAssetValidForCustomToolTip, OnIsAssetValidForCustomToolTip)

		/** Delegate to request a custom tool tip if necessary */
		SLATE_EVENT(FOnGetCustomAssetToolTip, OnGetCustomAssetToolTip)

		/* Delegate to signal when the item is about to show a tooltip */
		SLATE_EVENT(FOnVisualizeAssetToolTip, OnVisualizeAssetToolTip)

		/** Delegate for when an item's tooltip is about to close */
		SLATE_EVENT( FOnAssetToolTipClosing, OnAssetToolTipClosing )

	SLATE_END_ARGS()

	/** Destructor */
	~SAssetListItem();

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

	/** Handles committing a name change */
	virtual void OnAssetDataChanged() override;

	/** Handles realtime thumbnails */
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Handles realtime thumbnails */
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	/** Whether the widget should allow primitive tools to be displayed */
	bool CanDisplayPrimitiveTools() const { return false; }

	/** Generates a widget for a particular column */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName, FIsSelected InIsSelected);

protected:
	/** Generic ThumbnailSize for the Thumbnail */
	int32 GetGenericThumbnailSize() const;

	/** Get if AssetStatus should be visible */
	bool ShouldAllowAssetStatusOverlay(bool bForceGenericThumbnail) const;

	/** Get if AssetAction should be visible */
	bool ShouldAllowAssetActionOverlay(bool bForceGenericThumbnail) const;

	/** Get if the AssetAction indicator should be visible */
	bool ShouldAllowAssetActionOverlayIndicator(bool bForceGenericThumbnail) const;

private:
	/** New Style columns generation */
	TSharedRef<SWidget> GenerateWidgetForColumnNewStyle(const FName& ColumnName, FIsSelected InIsSelected);
	
	/** Get the expected width of an extra state icon. */
	float GetExtraStateIconWidth() const;

	/** Returns the max size to be used by extra state icons. */
	FOptionalSize GetExtraStateIconMaxSize() const;

	/** Returns the size of the state icon box widget (i.e dirty image, scc)*/
	FOptionalSize GetStateIconImageSize() const;

	/** Returns the size of the thumbnail widget */
	FOptionalSize GetThumbnailBoxSize() const;

	/** Get the text color for the columns */
	FSlateColor GetColumnTextColor(FIsSelected InIsSelected) const;
	
	/** Gets the path to this asset */
	FText GetAssetPathText() const;

	/** Gets the value for the specified tag in this asset */
	FText GetAssetTagText(FName InTag) const;

	/** Get the index to use for the Thumbnail WidgetSwitcher, based on the ThumbnailSize */
	int32 GetWidgetSwitcherIndex() const;

	/** Get the visibility of the dirty badge in the list view based on the current size */
	EVisibility GetDirtyBadgeVisibility() const;

private:
	/** The handle to the thumbnail that this item is rendering */
	TSharedPtr<FAssetThumbnail> AssetThumbnail;

	/** The height allowed for this item */
	TAttribute<float> ItemHeight;

	/** The padding for the thumbnail */
	float ThumbnailPadding;

	/** The actual widget for the thumbnail */
	TSharedPtr<SWidget> ThumbnailWidget;

	/** The string in the title to highlight (used when searching by string) */
	TAttribute<FText> HighlightText;

	/** Whether the item is selected in the view without anything else being selected*/
	FIsSelected IsSelectedExclusively;
	
	/** Current thumbnail size when this widget was generated */
	TAttribute<EThumbnailSize> CurrentThumbnailSize;

	/** Path text widget, need to be updated when the AssetData changes */
	TSharedPtr<STextBlock> PathTextWidget;
};


template <>
struct TWidgetTypeTraits<class SAssetListViewRow>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SAssetListViewRow : public SMultiColumnTableRow< TSharedPtr<FAssetViewItem> >
{
public:
	SLATE_BEGIN_ARGS( SAssetListViewRow )
		: _Padding(FMargin(0.f))
		{}
		
		SLATE_STYLE_ARGUMENT( FTableRowStyle, Style )
		SLATE_EVENT( FOnDragDetected, OnDragDetected )
		SLATE_ARGUMENT( TSharedPtr<SAssetListItem>, AssetListItem )
		SLATE_ATTRIBUTE( FMargin, Padding )

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		this->AssetListItem = InArgs._AssetListItem;
		ensure(this->AssetListItem.IsValid());

		SMultiColumnTableRow< TSharedPtr<FAssetViewItem> >::Construct( 
			FSuperRowType::FArguments()
				.Style(InArgs._Style)
				.Padding(InArgs._Padding)
				.OnDragDetected(InArgs._OnDragDetected), 
			InOwnerTableView);
		Content = this->AssetListItem;
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		if ( this->AssetListItem.IsValid() )
		{
			return this->AssetListItem->GenerateWidgetForColumn(ColumnName, FIsSelected::CreateSP( this, &SAssetListViewRow::IsSelectedExclusively ));
		}
		else
		{
			return SNew(STextBlock) .Text( NSLOCTEXT("AssetView", "ListViewInvalidColumnId", "Invalid Column Item") );
		}
		
	}

	virtual FVector2D GetRowSizeForColumn(const FName& InColumnName) const override
	{
		const TSharedRef<SWidget>* ColumnWidget = GetWidgetFromColumnId(InColumnName);

		if (ColumnWidget != nullptr)
		{
			return (*ColumnWidget)->GetDesiredSize();
		}

		return FVector2D::ZeroVector; 
	}

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override
	{
		this->AssetListItem->Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}

	virtual TSharedPtr<IToolTip> GetToolTip() override
	{
		return AssetListItem->GetToolTip();
	}

	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		AssetListItem->OnDragEnter(MyGeometry, DragDropEvent);
	}

	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override
	{
		AssetListItem->OnDragLeave(DragDropEvent);
	}

	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		return AssetListItem->OnDragOver(MyGeometry, DragDropEvent);
	}

	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		return AssetListItem->OnDrop(MyGeometry, DragDropEvent, AsShared());
	}

	virtual bool OnVisualizeTooltip(const TSharedPtr<SWidget>& TooltipContent) override
	{
		// We take the content from the asset column item during construction,
		// so let the item handle the tooltip callback
		return AssetListItem->OnVisualizeTooltip(TooltipContent);
	}

	virtual void OnToolTipClosing() override
	{
		AssetListItem->OnToolTipClosing();
	}

	TSharedPtr<SAssetListItem> AssetListItem;
};

template <>
struct TWidgetTypeTraits<class SAssetTileItem>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** An item in the asset tile view */
class SAssetTileItem : public SAssetViewItem
{
	friend struct FAssetViewItemHelper;

public:
	SLATE_BEGIN_ARGS( SAssetTileItem )
		: _ThumbnailPadding(0)
		/** Default Size of the Small Thumbnail Size */
		, _ThumbnailDimension(64)
		, _ThumbnailLabel( EThumbnailLabel::ClassName )
		, _ThumbnailHintColorAndOpacity( FLinearColor( 0.0f, 0.0f, 0.0f, 0.0f ) )
		, _AllowThumbnailHintLabel(true)
		, _CurrentThumbnailSize(EThumbnailSize::Medium)
		, _ItemWidth(16)
		, _ShouldAllowToolTip(true)
		, _ShowType(true)
		, _ThumbnailEditMode(false)
		, _ShowAssetAccessSpecifier(false)
	{}

		/** The handle to the thumbnail this item should render */
		SLATE_ARGUMENT( TSharedPtr<FAssetThumbnail>, AssetThumbnail)

		/** Data for the asset this item represents */
		SLATE_ARGUMENT( TSharedPtr<FAssetViewItem>, AssetItem )

		/** How much padding to allow around the thumbnail */
		SLATE_ARGUMENT( float, ThumbnailPadding )

		/** The Actual Thumbnail dimension based on the  */
		SLATE_ATTRIBUTE(float, ThumbnailDimension)

		/** The contents of the label displayed on the thumbnail EThumbnailSize */
		SLATE_ARGUMENT( EThumbnailLabel::Type, ThumbnailLabel )

		/** Thumbnail Color for Hint */
		SLATE_ATTRIBUTE( FLinearColor, ThumbnailHintColorAndOpacity )

		/** Whether the thumbnail should ever show it's hint label */
		SLATE_ARGUMENT( bool, AllowThumbnailHintLabel )

		/** Current size of the thumbnail that was generated */
		SLATE_ATTRIBUTE(EThumbnailSize, CurrentThumbnailSize)

		/** The width of the item */
		SLATE_ATTRIBUTE( float, ItemWidth )

		/** Delegate for when an asset name has entered a rename state */
		SLATE_EVENT( FOnRenameBegin, OnRenameBegin )

		/** Delegate for when an asset name has been entered for an item that is in a rename state */
		SLATE_EVENT( FOnRenameCommit, OnRenameCommit )

		/** Delegate for when an asset name has been entered for an item to verify the name before commit */
		SLATE_EVENT( FOnVerifyRenameCommit, OnVerifyRenameCommit )

		/** Called when any asset item is destroyed. Used in thumbnail management */
		SLATE_EVENT( FOnItemDestroyed, OnItemDestroyed )

		/** If false, the tooltip will not be displayed */
		SLATE_ATTRIBUTE( bool, ShouldAllowToolTip )

		/** If false, will not show type */
		SLATE_ARGUMENT( bool, ShowType )

		/** The string in the title to highlight (used when searching by string) */
		SLATE_ATTRIBUTE( FText, HighlightText )

		/** If true, the thumbnail in this item can be edited */
		SLATE_ATTRIBUTE( bool, ThumbnailEditMode )

		/** If true, display the item's EAssetAccessSpecifier */
		SLATE_ARGUMENT(bool, ShowAssetAccessSpecifier)

		/** Whether the item is selected in the view */
		SLATE_ARGUMENT(FIsSelected, IsSelected)

		/** Whether the item is selected in the view without anything else being selected*/
		SLATE_ARGUMENT(FIsSelected, IsSelectedExclusively)

		/** Delegate to call (if bound) to check if it is valid to get a custom tooltip for this view item */
		SLATE_EVENT(FOnIsAssetValidForCustomToolTip, OnIsAssetValidForCustomToolTip)

		/** Delegate to request a custom tool tip if necessary */
		SLATE_EVENT(FOnGetCustomAssetToolTip, OnGetCustomAssetToolTip)

		/* Delegate to signal when the item is about to show a tooltip */
		SLATE_EVENT(FOnVisualizeAssetToolTip, OnVisualizeAssetToolTip)

		/** Delegate for when an item's tooltip is about to close */
		SLATE_EVENT( FOnAssetToolTipClosing, OnAssetToolTipClosing )

	SLATE_END_ARGS()

	/** Destructor */
	~SAssetTileItem();

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

	/** Handles committing a name change */
	virtual void OnAssetDataChanged() override;

	/** Handles realtime thumbnails */
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Handles realtime thumbnails */
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	/** Whether the widget should allow primitive tools to be displayed */
	bool CanDisplayPrimitiveTools() const { return true; }

	/** Get the border image to display */
	virtual const FSlateBrush* GetBorderImage() const;

	static void InitializeAssetNameHeights();
	static float GetRegularFontHeight() { return RegularFontHeight; }
	static float GetSmallFontHeight() { return SmallFontHeight; }

protected:
	/** Called when requesting renaming */
	virtual void OnRequestingRename();

	/** SAssetViewItem interface */
	virtual float GetNameTextWrapWidth() const override { return LastGeometry.GetLocalSize().X - 15.f; }

	/** Get the expected width of an extra state icon. */
	float GetExtraStateIconWidth() const;

	/** Returns the max size to be used by extra state icons. */
	FOptionalSize GetExtraStateIconMaxSize() const;

	/** Returns the size of the state icon box widget (i.e dirty image, scc)*/
	FOptionalSize GetStateIconImageSize() const;

	/** Returns the size of the thumbnail widget */
	FOptionalSize GetThumbnailBoxSize() const;

	/** Returns the width of the tiny folder name overlay widget */
	FOptionalSize GetTinyFolderNameWidth() const;

	/** Gets the visibility of the folder name overlay on the folder icon */
	EVisibility GetTinyFolderNameOverlayVisibility() const;

	/** Gets the visibility of the asset class label in thumbnails */
	EVisibility GetAssetClassLabelVisibility() const;

	/** Gets the color of the asset class label in thumbnails */
	FSlateColor GetAssetClassLabelTextColor() const;

	/** Returns the font to use for the thumbnail label */
	FSlateFontInfo GetThumbnailFont() const;

	const FSlateBrush* GetFolderSlotBackgroundImage() const;
	const FSlateBrush* GetFolderBackgroundImage() const;
	const FSlateBrush* GetFolderBackgroundShadowImage() const;

	const FSlateBrush* GetNameAreaBackgroundImage() const;
	const FSlateBrush* GetAssetAreaOverlayBackgroundImage() const;
	FSlateColor GetNameAreaTextColor() const;

	FOptionalSize GetNameAreaMaxDesiredHeight() const;

	/** Generic ThumbnailSize for the Thumbnail */
	int32 GetGenericThumbnailSize() const;

	/** Gets the visibility of the SCC icons */
	EVisibility GetSCCIconVisibility() const;

	/** Get the visibility for the name area, collapsed if Tiny */
	EVisibility GetNameAreaVisibility() const;

	/** Return the size of the class text at the bottom, based on the current ThumbnailSize */
	FOptionalSize GetSourceControlIconSize() const;

private:
	/** Thumbnail Widget or Folder image holder */
	TSharedPtr<SWidget> Thumbnail;

	/** If false, the tooltip will not be displayed */
	bool bShowType;

	/** The handle to the thumbnail that this item is rendering */
	TSharedPtr<FAssetThumbnail> AssetThumbnail;

	/** The width of the item. Used to enforce a square thumbnail. */
	TAttribute<float> ItemWidth;

	/** Actual size that the thumbnail should have */
	TAttribute<float> ThumbnailDimension;

	/** Max name height for each thumbnail size */
	static float AssetNameHeights[(int32)EThumbnailSize::MAX];

	/** Regular thumbnail font size */
	static float RegularFontHeight;

	/** Small thumbnail font size */
	static float SmallFontHeight;

	/** The padding allotted for the thumbnail */
	float ThumbnailPadding;

	/** Current thumbnail size when this widget was generated */
	TAttribute<EThumbnailSize> CurrentThumbnailSize;
};

template <>
struct TWidgetTypeTraits<class SAssetColumnItem>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** An item in the asset column view */
class SAssetColumnItem : public SAssetViewItem
{
public:
	SLATE_BEGIN_ARGS( SAssetColumnItem ) 
		: _ShowAssetAccessSpecifier(false)
		{}

		/** Data for the asset this item represents */
		SLATE_ARGUMENT( TSharedPtr<FAssetViewItem>, AssetItem )

		/** If true, display the item's EAssetAccessSpecifier */
		SLATE_ARGUMENT(bool, ShowAssetAccessSpecifier)

		/** Delegate for when an asset name has entered a rename state */
		SLATE_EVENT( FOnRenameBegin, OnRenameBegin )

		/** Delegate for when an asset name has been entered for an item that is in a rename state */
		SLATE_EVENT( FOnRenameCommit, OnRenameCommit )

		/** Delegate for when an asset name has been entered for an item to verify the name before commit */
		SLATE_EVENT( FOnVerifyRenameCommit, OnVerifyRenameCommit )

		/** Called when any asset item is destroyed. Used in thumbnail management, though it may be used for more so It is in column items for consistency. */
		SLATE_EVENT( FOnItemDestroyed, OnItemDestroyed )

		/** The string in the title to highlight (used when searching by string) */
		SLATE_ATTRIBUTE( FText, HighlightText )

		/** Delegate to call (if bound) to check if it is valid to get a custom tooltip for this view item */
		SLATE_EVENT(FOnIsAssetValidForCustomToolTip, OnIsAssetValidForCustomToolTip)

		/** Delegate to request a custom tool tip if necessary */
		SLATE_EVENT( FOnGetCustomAssetToolTip, OnGetCustomAssetToolTip)

		/* Delegate to signal when the item is about to show a tooltip */
		SLATE_EVENT( FOnVisualizeAssetToolTip, OnVisualizeAssetToolTip)

		/** Delegate for when an item's tooltip is about to close */
		SLATE_EVENT( FOnAssetToolTipClosing, OnAssetToolTipClosing )

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

	/** Generates a widget for a particular column */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName, FIsSelected InIsSelected );

	/** Handles committing a name change */
	virtual void OnAssetDataChanged() override;

private:
	/** Gets the path to this asset */
	FText GetAssetPathText() const;

	/** Gets the value for the specified tag in this asset */
	FText GetAssetTagText(FName Tag) const;

	/** Get the text color for the columns */
	FSlateColor GetColumnTextColor(FIsSelected InIsSelected) const;

private:
	TAttribute<FText> HighlightText;

	TSharedPtr<STextBlock> ClassText;
	TSharedPtr<STextBlock> PathText;
};


template <>
struct TWidgetTypeTraits<class SAssetColumnViewRow>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SAssetColumnViewRow : public SMultiColumnTableRow< TSharedPtr<FAssetViewItem> >
{
public:
	SLATE_BEGIN_ARGS( SAssetColumnViewRow )
		{}

		SLATE_EVENT( FOnDragDetected, OnDragDetected )
		SLATE_ARGUMENT( TSharedPtr<SAssetColumnItem>, AssetColumnItem )

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		this->AssetColumnItem = InArgs._AssetColumnItem;
		ensure(this->AssetColumnItem.IsValid());

		SMultiColumnTableRow< TSharedPtr<FAssetViewItem> >::Construct( 
			FSuperRowType::FArguments()
				.Style(FAppStyle::Get(), "ContentBrowser.AssetListView.ColumnListTableRow")
				.OnDragDetected(InArgs._OnDragDetected), 
			InOwnerTableView);
		Content = this->AssetColumnItem;
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		if ( this->AssetColumnItem.IsValid() )
		{
			return this->AssetColumnItem->GenerateWidgetForColumn(ColumnName, FIsSelected::CreateSP( this, &SAssetColumnViewRow::IsSelectedExclusively ));
		}
		else
		{
			return SNew(STextBlock) .Text( NSLOCTEXT("AssetView", "InvalidColumnId", "Invalid Column Item") );
		}
		
	}

	virtual FVector2D GetRowSizeForColumn(const FName& InColumnName) const override
	{
		const TSharedRef<SWidget>* ColumnWidget = GetWidgetFromColumnId(InColumnName);

		if (ColumnWidget != nullptr)
		{
			return (*ColumnWidget)->GetDesiredSize();
		}

		return FVector2D::ZeroVector; 
	}

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override
	{
		this->AssetColumnItem->Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	}

	virtual TSharedPtr<IToolTip> GetToolTip() override
	{
		return AssetColumnItem->GetToolTip();
	}

	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		AssetColumnItem->OnDragEnter(MyGeometry, DragDropEvent);
	}

	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override
	{
		AssetColumnItem->OnDragLeave(DragDropEvent);
	}

	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		return AssetColumnItem->OnDragOver(MyGeometry, DragDropEvent);
	}

	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
	{
		return AssetColumnItem->OnDrop(MyGeometry, DragDropEvent, AsShared());
	}

	virtual bool OnVisualizeTooltip(const TSharedPtr<SWidget>& TooltipContent) override
	{
		// We take the content from the asset column item during construction,
		// so let the item handle the tooltip callback
		return AssetColumnItem->OnVisualizeTooltip(TooltipContent);
	}

	virtual void OnToolTipClosing() override
	{
		AssetColumnItem->OnToolTipClosing();
	}

	TSharedPtr<SAssetColumnItem> AssetColumnItem;
};
