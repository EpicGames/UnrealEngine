// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "IAssetTypeActions.h"
#include "Input/Reply.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Widgets/SToolTip.h"
#include "Containers/ArrayView.h"
#include "ContentBrowserDataLegacyBridge.h"

class FContentBrowserItemData;
class FContentBrowserItemDataAttributeValue;
struct FARFilter;
struct FCollectionNameType;
struct FContentBrowserItem;
class FContentBrowserItemTemporaryContext;
class FUICommandList;

/** Called when the user has committed a rename of one or more items */
DECLARE_DELEGATE_OneParam(FOnContentBrowserItemRenameCommitted, TArrayView<const FContentBrowserItem> /*Items*/);

/** Delegate called when item selection is changed */
DECLARE_DELEGATE_TwoParams(FOnContentBrowserItemSelectionChanged, const FContentBrowserItem& /*SelectedItem*/, ESelectInfo::Type /*SelectInfo*/);

/** Called when the user double clicks, presses enter, or presses space on a Content Browser item */
DECLARE_DELEGATE_TwoParams(FOnContentBrowserItemsActivated, TArrayView<const FContentBrowserItem> /*ActivatedItems*/, EAssetTypeActivationMethod::Type /*ActivationMethod*/);

/** Called when the user drags one or more Content Browser items */
DECLARE_DELEGATE_OneParam(FOnContentBrowserItemsDragged, TArrayView<const FContentBrowserItem> /*Items*/);

/** Called to request the menu when right clicking on an selection of items (maybe be a mix of folders and files) */
DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SWidget>, FOnGetContentBrowserItemContextMenu, TArrayView<const FContentBrowserItem> /*SelectedItems*/);

/** Called when a collection is selected in the collections view */
DECLARE_DELEGATE_OneParam( FOnCollectionSelected, const FCollectionNameType& /*SelectedCollection*/);

/** Called to retrieve the tooltip for the specified asset */
DECLARE_DELEGATE_RetVal_OneParam( TSharedRef< SToolTip >, FConstructToolTipForAsset, const FAssetData& /*Asset*/);

/** Called to check if an asset should be filtered out by external code. Return true to exclude the asset from the view. */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnShouldFilterAsset, const FAssetData& /*AssetData*/);

/** Called to check if an item should be filtered out by external code. Return true to exclude the item from the view. */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnShouldFilterItem, const FContentBrowserItem& /*AssetItem*/);

/** Called to check if an asset tag should be display in details view. Return false to exclude the asset from the view. */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnShouldDisplayAssetTag, FName /*AssetType*/, FName /*TagName*/);

/** Fired when the search box text changes. Allows embedding widgets to observe the search text without replacing the built-in search behavior. */
DECLARE_DELEGATE_OneParam(FOnSearchTextChanged, const FText& /*InSearchText*/)

/** Called to clear the selection of the specified assetdata or all selection if an invalid assetdata is passed */
DECLARE_DELEGATE( FClearSelectionDelegate );

/** Called when thumbnail scale changes and the thumbnail scale is bound to a delegate */
DECLARE_DELEGATE_OneParam( FOnThumbnailScaleChanged, const float /*NewScale*/);

/** Called to retrieve an array of the currently selected asset data */
DECLARE_DELEGATE_RetVal( TArray< FAssetData >, FGetCurrentSelectionDelegate );

/** Called to retrieve an array of the currently selected asset data */
DECLARE_DELEGATE_OneParam(FSyncToAssetsDelegate, const TArray< FAssetData >& /*AssetData*/);

/** Called to force the asset view to refresh */
DECLARE_DELEGATE_OneParam(FRefreshAssetViewDelegate, bool /*UpdateSources*/);

/** Called to save the asset view settings (column widths, sort order, hidden columns, etc.) */
DECLARE_DELEGATE(FSaveAssetViewSettingsDelegate);

/** Called to load/restore the asset view settings */
DECLARE_DELEGATE(FLoadAssetViewSettingsDelegate);

/** Called to set a new filter for an existing asset picker */
DECLARE_DELEGATE_OneParam(FSetARFilterDelegate, const FARFilter& /*NewFilter*/);

/** A pointer to an existing delegate that, when executed, will set the filter an the asset picker after it is created. */
DECLARE_DELEGATE_OneParam(FSetPathPickerPathsDelegate, const TArray<FString>& /*NewPaths*/);

/** Called when an asset is selected in the asset view */
DECLARE_DELEGATE_OneParam(FOnAssetSelected, const FAssetData& /*AssetData*/);

/** Called when the user double clicks, presses enter, or presses space on an asset */
DECLARE_DELEGATE_TwoParams(FOnAssetsActivated, const TArray<FAssetData>& /*ActivatedAssets*/, EAssetTypeActivationMethod::Type /*ActivationMethod*/);

/** Called when asset(s) are dragged in the asset view */
DECLARE_DELEGATE_OneParam(FOnAssetsDragged, const TArray<FAssetData>& /*DraggedAssets*/);

/** Called when the user changes which columns are hidden in the asset column/list view */
DECLARE_DELEGATE_OneParam(FOnColumnVisibilityChanged, const TArray<FString>& /*HiddenColumnIds*/);

/** Called when an asset is double clicked in the asset view */
DECLARE_DELEGATE_OneParam(FOnAssetDoubleClicked, const FAssetData& /*AssetData*/);

/** Called when enter is pressed on an asset in the asset view */
DECLARE_DELEGATE_OneParam(FOnAssetEnterPressed, const TArray<FAssetData>& /*SelectedAssets*/);

/** Called when a new folder is starting to be created */
DECLARE_DELEGATE_OneParam(FOnCreateNewFolder, const FContentBrowserItemTemporaryContext& /*NewItemContext*/);

/** Called when one or more folders are being deleted */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnDeleteFolders, const TArray<FContentBrowserItemPath>& /*PathsToDelete*/);

/** Called to request the menu when right clicking on an asset */
DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SWidget>, FOnGetAssetContextMenu, const TArray<FAssetData>& /*SelectedAssets*/);

/** Called when a path is selected in the path picker */
DECLARE_DELEGATE_OneParam(FOnPathSelected, const FString& /*Path*/);

/** Called when registering a custom command/keybinding for the content browser */
DECLARE_DELEGATE_TwoParams(FOnContentBrowserGetSelection, TArray<FAssetData>& /*SelectedAssets*/, TArray<FString>& /*SelectedPaths*/);
DECLARE_DELEGATE_TwoParams(FContentBrowserCommandExtender, TSharedRef<FUICommandList> /*CommandList*/, FOnContentBrowserGetSelection /*GetSelectionDelegate*/);

/** Called to request the menu when right clicking on a path */
DECLARE_DELEGATE_RetVal(TSharedRef<FExtender>, FContentBrowserMenuExtender);
DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<FExtender>, FContentBrowserMenuExtender_SelectedAssets, const TArray<FAssetData>& /*SelectedAssets*/);
DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<FExtender>, FContentBrowserMenuExtender_SelectedPaths, const TArray<FString>& /*SelectedPaths*/);

/** Called to check if should show menu entries to set asset public/private */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnContentBrowserCanShowSetAssetAccessMenu, const FStringView /*AssetPath*/, const FAssetData& /*Asset*/);

/** Called to request the menu when right clicking on an asset */
DECLARE_DELEGATE_RetVal_ThreeParams(TSharedPtr<SWidget>, FOnGetFolderContextMenu, const TArray<FString>& /*SelectedPaths*/, FContentBrowserMenuExtender_SelectedPaths /*MenuExtender*/, FOnCreateNewFolder /*CreationDelegate*/);

/** Called to see if it is valid to request a custom asset item tooltip */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnIsAssetValidForCustomToolTip, FAssetData& /*AssetData*/);

/** Called to request a custom asset item tooltip */
DECLARE_DELEGATE_RetVal_OneParam( TSharedRef<SToolTip>, FOnGetCustomAssetToolTip, FAssetData& /*AssetData*/);

/** Called to get string/text value for a custom column, will get converted as necessary */
DECLARE_DELEGATE_RetVal_TwoParams(FString, FOnGetCustomAssetColumnData, FAssetData& /*AssetData*/, FName /*ColumnName*/);
DECLARE_DELEGATE_RetVal_TwoParams(FText, FOnGetCustomAssetColumnDisplayText, FAssetData& /*AssetData*/, FName /*ColumnName*/);

/** Called to generate extra state information icons or tooltips on asset items. */
DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnGenerateAssetViewExtraStateIndicators, const FAssetData& /*AssetData*/);

/** Called to generate state information icons for path view items. */
DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SWidget>, FPathViewStateIconGenerator, const FContentBrowserItem& /*ContentBrowserItem*/);

/** Called to get ContentBrowserItemData attribute values. */
UE_EXPERIMENTAL(5.7, "FGetItemDataAttribute is experimental while a newer system is in active development.")
DECLARE_DELEGATE_RetVal_FourParams(bool, FGetItemDataAttribute, const FContentBrowserItemData& /*InItem*/, const bool /*InIncludeMetaData*/, const FName /*InAttributeKey*/, FContentBrowserItemDataAttributeValue& /*OutAttributeValue*/);

/** Called when an asset item visualizes its tooltip */
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnVisualizeAssetToolTip, const TSharedPtr<SWidget>& /*ToolTipContent*/, FAssetData& /*AssetData*/);

/** Called from the Asset Dialog when a non-modal dialog is closed or cancelled */
DECLARE_DELEGATE(FOnAssetDialogCancelled);

/** Called when an asset item's tooltip is closing */
DECLARE_DELEGATE( FOnAssetToolTipClosing );

/** Called from the Asset Dialog when assets are chosen in non-modal Open dialogs */
DECLARE_DELEGATE_OneParam(FOnAssetsChosenForOpen, const TArray<FAssetData>& /*SelectedAssets*/);

/** Called from the Asset Dialog when an asset name is chosen in non-modal Save dialogs */
DECLARE_DELEGATE_OneParam(FOnObjectPathChosenForSave, const FString& /*ObjectPath*/);

/** Called when custom behavior is needed for allowing folders to toggle their private content edit state */
DECLARE_DELEGATE_RetVal_OneParam(bool, FIsFolderShowPrivateContentToggleableDelegate, const FStringView /*VirtualFolderPath*/);

/** Called when a custom folder color is set on a folder */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnSetFolderColor, const FString& /*FolderPath*/)

/** Called when an item is added or removed from favorites */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnFavoritesChanged, const FContentBrowserItemPath& /*ItemPath*/, bool /* bAdded */)

/** Contains the delegates used to handle a custom drag-and-drop in the asset view */
struct FAssetViewDragAndDropExtender
{
	struct FPayload
	{
		FPayload(TSharedPtr<FDragDropOperation> InDragDropOp, const TArray<FName>& InPackagePaths, const TArray<FCollectionRef>& InCollectionSources);

		UE_DEPRECATED(5.6, "Use the FCollectionRef overload instead.")
		FPayload(TSharedPtr<FDragDropOperation> InDragDropOp, const TArray<FName>& InPackagePaths, const TArray<FCollectionNameType>& InCollections);

		TSharedPtr<FDragDropOperation> DragDropOp;

	private:
		TArray<FCollectionRef> TempCollectionSources;
		TArray<FCollectionNameType> TempCollections;

	public:
		const TArray<FName>& PackagePaths;
		const TArray<FCollectionRef>& CollectionSources;

		UE_DEPRECATED(5.6, "Use CollectionSources instead.")
		const TArray<FCollectionNameType> Collections;
	};

	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnDropDelegate, const FPayload&);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnDragOverDelegate, const FPayload&);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnDragLeaveDelegate, const FPayload&);

	FAssetViewDragAndDropExtender(FOnDropDelegate InOnDropDelegate)
		: OnDropDelegate(MoveTemp(InOnDropDelegate))
		, OnDragOverDelegate()
		, OnDragLeaveDelegate()
	{ }

	FAssetViewDragAndDropExtender(FOnDropDelegate InOnDropDelegate, FOnDragOverDelegate InOnDragOverDelegate)
		: OnDropDelegate(MoveTemp(InOnDropDelegate))
		, OnDragOverDelegate(MoveTemp(InOnDragOverDelegate))
		, OnDragLeaveDelegate()
	{ }

	FAssetViewDragAndDropExtender(FOnDropDelegate InOnDropDelegate, FOnDragOverDelegate InOnDragOverDelegate, FOnDragLeaveDelegate InOnDragLeaveDelegate)
		: OnDropDelegate(MoveTemp(InOnDropDelegate))
		, OnDragOverDelegate(MoveTemp(InOnDragOverDelegate))
		, OnDragLeaveDelegate(MoveTemp(InOnDragLeaveDelegate))
	{ }

	FOnDropDelegate OnDropDelegate;
	FOnDragOverDelegate OnDragOverDelegate;
	FOnDragLeaveDelegate OnDragLeaveDelegate;
};

/** Struct to define a custom column for the asset view */
struct FAssetViewCustomColumn
{
	/** Internal name of the column */
	FName ColumnName;

	/** Display name of the column */
	FText DisplayName;

	/** Tooltip for the column */
	FText TooltipText;

	/** Type of column, used for sorting */
	UObject::FAssetRegistryTag::ETagType DataType;

	/** Delegate to get String value for this column, used for sorting and internal use */
	FOnGetCustomAssetColumnData OnGetColumnData;

	/** Delegate to get Text value for this column, used to actually display */
	FOnGetCustomAssetColumnDisplayText OnGetColumnDisplayText;

	FAssetViewCustomColumn()
		: DataType(UObject::FAssetRegistryTag::TT_Alphabetical) 
	{ }

	FAssetViewCustomColumn(FName InColumnName, const FText& InDisplayName, const FText& InTooltipText, UObject::FAssetRegistryTag::ETagType InDataType, const FOnGetCustomAssetColumnData& InOnGetColumnData, const FOnGetCustomAssetColumnDisplayText& InOnGetColumnDisplayText = FOnGetCustomAssetColumnDisplayText())
		: ColumnName(InColumnName)
		, DisplayName(InDisplayName)
		, TooltipText(InTooltipText)
		, DataType(InDataType)
		, OnGetColumnData(InOnGetColumnData)
		, OnGetColumnDisplayText(InOnGetColumnDisplayText)
	{ }
};
