// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "FrontendFilterBase.h"
#include "ContentBrowserFrontEndFilterExtension.generated.h"

// Override this class in order to make an additional front-end filter available in the Content Browser
UCLASS(MinimalAPI, Abstract)
class UContentBrowserFrontEndFilterExtension : public UObject
{
	GENERATED_BODY()

public:
	// Override this method to add new frontend filters
	virtual void AddFrontEndFilterExtensions(TSharedPtr<class FFrontendFilterCategory> DefaultCategory, TArray< TSharedRef<class FFrontendFilter> >& InOutFilterList) const { }
};
