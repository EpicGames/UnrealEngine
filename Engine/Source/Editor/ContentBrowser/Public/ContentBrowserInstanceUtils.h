// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"

#define UE_API CONTENTBROWSER_API

namespace ContentBrowserInstanceUtils
{
	/** Gets a list of names for all existing content browser configs */
	UE_API TArray<FName> GetConfigNames();

	/** Saves all content browser configs */
	UE_API void SaveAllConfigs();

	/** Gets whether we are allowed to display plugin content
	 * @param InstanceName Instance name of the content browser
	 */
	UE_API bool GetShowPluginContent(const FName& InstanceName);

	/** Sets whether we are allowed to display plugin content
	 * @param InstanceName Instance name of the content browser to be modified
	 * @param bInShowPluginContent True to display plugin content
	 * @param bInSaveConfig True to save the config after modification
	 */
	UE_API void SetShowPluginContent(const FName& InstanceName, const bool bInShowPluginContent, const bool bInSaveConfig = true);
		
	/** Gets whether we are allowed to display localized content
	 * @param InstanceName Instance name of the content browser
	 */
	UE_API bool GetShowLocalizedContent(const FName& InstanceName);

	/** Sets whether we are allowed to display localized content
	 * @param InstanceName Instance name of the content browser to be modified
	 * @param bInShowLocalizedContent True to display localized content
	 * @param bInSaveConfig True to save the config after modification
	 */
	UE_API void SetShowLocalizedContent(const FName& InstanceName, const bool bInShowLocalizedContent, const bool bInSaveConfig = true);

	/** Gets whether we are allowed to display engine content
	 * @param InstanceName Instance name of the content browser
	 */
	UE_API bool GetShowEngineContent(const FName& InstanceName);

	/** Sets whether we are allowed to display engine content
	 * @param InstanceName Instance name of the content browser to be modified
	 * @param bInShowEngineContent True to display engine content
	 * @param bInSaveConfig True to save the config after modification
	 */
	UE_API void SetShowEngineContent(const FName& InstanceName, const bool bInShowEngineContent, const bool bInSaveConfig = true);

	/** Gets whether we are allowed to display developer content
	 * @param InstanceName Instance name of the content browser
	 */
	UE_API bool GetShowDeveloperContent(const FName& InstanceName);

	/** Sets whether we are allowed to display developer content
	 * @param InstanceName Instance name of the content browser to be modified
	 * @param bInShowDeveloperContent True to display developer content
	 * @param bInSaveConfig True to save the config after modification
	 */
	UE_API void SetShowDeveloperContent(const FName& InstanceName, const bool bInShowDeveloperContent, const bool bInSaveConfig = true);

	/** Gets whether we are allowed to display cpp folders
	 * @param InstanceName Instance name of the content browser
	 */
	UE_API bool GetShowCppFolders(const FName& InstanceName);

	/** Sets whether we are allowed to display cpp folders
	 * @param InstanceName Instance name of the content browser to be modified
	 * @param bInShowCppFolders True to display cpp folders
	 * @param bInSaveConfig True to save the config after modification
	 */
	UE_API void SetShowCppFolders(const FName& InstanceName, const bool bInShowCppFolders, const bool bInSaveConfig = true);

	/** Gets whether we are allowed to display favorites
	 * @param InstanceName Instance name of the content browser
	 */
	UE_API bool GetShowFavorites(const FName& InstanceName);

	/** Sets whether we are allowed to display favorites
	 * @param InstanceName Instance name of the content browser to be modified
	 * @param bInShowFavorites True to display favorites
	 * @param bInSaveConfig True to save the config after modification
	 */
	UE_API void SetShowFavorites(const FName& InstanceName, const bool bInShowFavorites, const bool bInSaveConfig = true);

	/** Gets whether we are allowed to expand favorites
	 * @param InstanceName Instance name of the content browser
	 */
	UE_API bool GetFavoritesExpanded(const FName& InstanceName);

	/** Sets whether we are allowed to expand favorites
	 * @param InstanceName Instance name of the content browser to be modified
	 * @param bInFavoritesExpanded True to expand favorites
	 * @param bInSaveConfig True to save the config after modification
	 */
	UE_API void SetFavoritesExpanded(const FName& InstanceName, const bool bInFavoritesExpanded, const bool bInSaveConfig = true);

	/** Gets whether we are allowed to expand sources
	 * @param InstanceName Instance name of the content browser
	 */
	UE_API bool GetSourcesExpanded(const FName& InstanceName);

	/** Sets whether we are allowed to expand sources
	 * @param InstanceName Instance name of the content browser to be modified
	 * @param bInSourcesExpanded True to expand sources
	 * @param bInSaveConfig True to save the config after modification
	 */
	UE_API void SetSourcesExpanded(const FName& InstanceName, const bool bInSourcesExpanded, const bool bInSaveConfig = true);

	/** Gets whether we are allowed to filter recursively when a filter is applied
	 * @param InstanceName Instance name of the content browser
	 */
	UE_API bool GetFilterRecursively(const FName& InstanceName);

	/** Sets whether we are allowed to filter recursively when a filter is applied
	 * @param InstanceName Instance name of the content browser to be modified
	 * @param bInFilterRecursively True to filter recursively
	 * @param bInSaveConfig True to save the config after modification
	 */
	UE_API void SetFilterRecursively(const FName& InstanceName, const bool bInFilterRecursively, const bool bInSaveConfig = true);

	/** Gets whether we are allowed to display folders
	 * @param InstanceName Instance name of the content browser
	 */
	UE_API bool GetShowFolders(const FName& InstanceName);

	/** Sets whether we are allowed to display folders
	 * @param InstanceName Instance name of the content browser to be modified
	 * @param bInShowFolders True to display folders
	 * @param bInSaveConfig True to save the config after modification
	 */
	UE_API void SetShowFolders(const FName& InstanceName, const bool bInShowFolders, const bool bInSaveConfig = true);

	/** Gets whether we are allowed to display empty folders
	 * @param InstanceName Instance name of the content browser
	 */
	UE_API bool GetShowEmptyFolders(const FName& InstanceName);

	/** Sets whether we are allowed to display empty folders
	 * @param InstanceName Instance name of the content browser to be modified
	 * @param bInShowEmptyFolders True to display empty folders
	 * @param bInSaveConfig True to save the config after modification
	 */
	UE_API void SetShowEmptyFolders(const FName& InstanceName, const bool bInShowEmptyFolders, const bool bInSaveConfig = true);

	/** Gets whether we are allowed to search in asset class names
	 * @param InstanceName Instance name of the content browser
	 */
	UE_API bool GetSearchClasses(const FName& InstanceName);

	/** Sets whether we are allowed to search in asset class names
	 * @param InstanceName Instance name of the content browser to be modified
	 * @param bInSearchClasses True to search in asset class names
	 * @param bInSaveConfig True to save the config after modification
	 */
	UE_API void SetSearchClasses(const FName& InstanceName, const bool bInSearchClasses, const bool bInSaveConfig = true);

	/** Gets whether we are allowed to search asset paths (instead of asset name only)
	 * @param InstanceName Instance name of the content browser
	 */
	UE_API bool GetSearchAssetPaths(const FName& InstanceName);

	/** Sets whether we are allowed to search asset paths (instead of asset name only)
	 * @param InstanceName Instance name of the content browser to be modified
	 * @param bInSearchAssetPaths True to search asset paths
	 * @param bInSaveConfig True to save the config after modification
	 */
	UE_API void SetSearchAssetPaths(const FName& InstanceName, const bool bInSearchAssetPaths, const bool bInSaveConfig = true);

	/** Gets whether we are allowed to search for collection names
	 * @param InstanceName Instance name of the content browser
	 */
	UE_API bool GetSearchCollections(const FName& InstanceName);

	/** Sets whether we are allowed to search for collection names
	 * @param InstanceName Instance name of the content browser to be modified
	 * @param bInSearchCollections True to search for collection names
	 * @param bInSaveConfig True to save the config after modification
	 */
	UE_API void SetSearchCollections(const FName& InstanceName, const bool bInSearchCollections, const bool bInSaveConfig = true);
}

#undef UE_API
