// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeCategories.h"
#include "Containers/Array.h"
#include "ContentBrowserDelegates.h"
#include "CoreMinimal.h"
#include "Algo/SelectRandomWeighted.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "ContentBrowserMenuContexts.generated.h"

#define UE_API CONTENTBROWSER_API

class FAssetViewSortManager;
class FAssetContextMenu;
class IAssetTypeActions;
class SAssetView;
class SContentBrowser;
class SFilterList;
class UClass;
class UAssetDefinition;
struct FContentBrowserConfig;
struct FFrame;
struct FToolMenuSection;
struct FToolMenuContext;

enum class EIncludeSubclasses : uint8
{
	No,
	Yes
};

UCLASS(MinimalAPI)
class UContentBrowserAssetContextMenuContext : public UObject
{
	GENERATED_BODY()

public:

	TWeakPtr<FAssetContextMenu> AssetContextMenu;

	UE_DEPRECATED(5.2, "IAssetTypeActions (CommonAssetTypeActions) is being phased out too much menu related stuff was built into their design that forced loading assets.  Please use UAssetDefinition (CommonAssetDefinition).")
	TWeakPtr<IAssetTypeActions> CommonAssetTypeActions;
	
	UPROPERTY()
	TObjectPtr<const UAssetDefinition> CommonAssetDefinition;

	/**
	 * The currently selected assets in the content browser.
	 */
	UPROPERTY(BlueprintReadOnly, Category="Tool Menus")
	TArray<FAssetData> SelectedAssets;

	UPROPERTY()
	TObjectPtr<UClass> CommonClass;

	UPROPERTY()
	bool bCanBeModified;

	UPROPERTY()
	bool bCanView;

	UPROPERTY()
	bool bHasCookedPackages;

	UPROPERTY(BlueprintReadOnly, Category = ContentBrowser)
	bool bContainsUnsupportedAssets = true;

	UE_DEPRECATED(5.5, "GetSelectedObjects has been deprecated.  We no longer automatically load assets on right click.  Please use SelectedAssets and determine whatever you need for your context menu options without actually loading the assets.  When you finally need all or a subset of the selected assets use LoadSelectedAssets or LoadSelectedAssetsIf")
	UFUNCTION(BlueprintCallable, Category="Tool Menus", meta=(DeprecatedFunction, DeprecationMessage = "GetSelectedObjects has been deprecated.  We no longer automatically load assets on right click.  If you can work without loading the assets, please use SelectedAssets.  Otherwise call LoadSelectedObjects"))
	TArray<UObject*> GetSelectedObjects() const
	{
		return LoadSelectedObjectsIfNeeded();
	}

	template<typename ExpectedAssetType>
	TArray<ExpectedAssetType*> GetSelectedObjectsInMemory() const
	{
		TArray<ExpectedAssetType*> Result;
		Result.Reserve(SelectedAssets.Num());
		for (const FAssetData& Asset : SelectedAssets)
		{
			if (Asset.IsInstanceOf(ExpectedAssetType::StaticClass()))
			{
				if (UObject* AssetObject = Asset.FastGetAsset(false))
				{
					if (ExpectedAssetType* AssetObjectTyped = Cast<ExpectedAssetType>(AssetObject))
					{
						Result.Add(AssetObjectTyped);
					}
				}
			}
		}
		return Result;
	}

	/**
	 * Loads the selected assets (if needed) which is based on AssetViewUtils::LoadAssetsIfNeeded, this exists primarily
	 * for backwards compatability.  Reliance on a black box to determine 'neededness' is not recommended, this function
	 * will likely be deprecated a few versions after GetSelectedObjects.
	 */
	UFUNCTION(BlueprintCallable, Category="Tool Menus")
	UE_API TArray<UObject*> LoadSelectedObjectsIfNeeded() const;

	/**
	 * Loads all the selected assets and returns an array of the objects.
	 */
	UFUNCTION(BlueprintCallable, Category="Tool Menus")
	TArray<UObject*> LoadSelectedObjects(TSet<FName> LoadTags) const
	{
		return LoadSelectedObjects<UObject>(LoadTags);
	}

	/**
	 * Loads all the selected assets and returns an array of the ExpectedAssetType.
	 */
	template<typename ExpectedAssetType>
	TArray<ExpectedAssetType*> LoadSelectedObjects(TSet<FName> LoadTags = {}) const
	{
		return LoadSelectedObjectsIf<ExpectedAssetType>(LoadTags, [](const FAssetData& AssetData){ return true; });
	}

	/**
	 * Loads the selected assets if the PredicateFilter returns true, and returns an array of the objects.
     */
	template<typename ExpectedAssetType>
    TArray<ExpectedAssetType*> LoadSelectedObjectsIf(TFunctionRef<bool(const FAssetData& AssetData)> PredicateFilter) const
    {
		return LoadSelectedObjectsIf<ExpectedAssetType>({}, PredicateFilter);
    }

	/**
	 * Loads the selected assets if the PredicateFilter returns true, and returns an array of the objects.
	 */
	template<typename ExpectedAssetType>
	TArray<ExpectedAssetType*> LoadSelectedObjectsIf(TSet<FName> LoadTags, TFunctionRef<bool(const FAssetData& AssetData)> PredicateFilter) const
	{
		TArray<ExpectedAssetType*> Result;
		Result.Reserve(SelectedAssets.Num());
		for (const FAssetData& Asset : SelectedAssets)
		{
			if (Asset.IsInstanceOf(ExpectedAssetType::StaticClass()))
			{
				if (PredicateFilter(Asset))
				{
					if (UObject* AssetObject = Asset.GetAsset(LoadTags))
					{
						if (ExpectedAssetType* AssetObjectTyped = Cast<ExpectedAssetType>(AssetObject))
						{
							Result.Add(AssetObjectTyped);
						}
					}
				}
			}
		}
		return Result;
	}

	/**
	 * Loads the first selected valid asset and returns it.
	 */
	template<typename ExpectedAssetType>
	ExpectedAssetType* LoadFirstSelectedObject(const TSet<FName>& LoadTags = {}) const
	{   	
		for (const FAssetData& Asset : SelectedAssets)
		{
			if (Asset.IsInstanceOf(ExpectedAssetType::StaticClass()))
			{
				if (ExpectedAssetType* ExpectedType = Cast<ExpectedAssetType>(Asset.GetAsset(LoadTags)))
				{
					return ExpectedType;
				}
			}	
		}
    
		return nullptr;
	}

	/**
	 * Returns a filtered array of assets that are of the desired class and potentially any subclasses.
	 */
	UE_API TArray<FAssetData> GetSelectedAssetsOfType(const UClass* AssetClass, EIncludeSubclasses IncludeSubclasses = EIncludeSubclasses::Yes) const;
	
	/**
	 * Sometimes you want to write actions that will only operate on a singular selected asset, in those cases you
	 * can use the following function which will only return a live ptr if it's an instance of that asset type, and
	 * only one thing is selected.
	 */
	UE_API const FAssetData* GetSingleSelectedAssetOfType(const UClass* AssetClass, EIncludeSubclasses IncludeSubclasses = EIncludeSubclasses::Yes) const;

	/**
	 * Get the selected assets as an array of SoftObjectPtr<T>'s.
	 */
	template<typename ExpectedAssetType>
	TArray<TSoftObjectPtr<ExpectedAssetType>> GetSelectedAssetSoftObjects(EIncludeSubclasses IncludeSubclasses = EIncludeSubclasses::Yes) const
	{
		TArray<TSoftObjectPtr<ExpectedAssetType>> Result;
		Result.Reserve(SelectedAssets.Num());
		if (IncludeSubclasses == EIncludeSubclasses::Yes)
		{
			Algo::TransformIf(SelectedAssets, Result,
				[](const FAssetData& AssetData) { return AssetData.IsInstanceOf(ExpectedAssetType::StaticClass()); },
				[](const FAssetData& AssetData) { return TSoftObjectPtr<ExpectedAssetType>(AssetData.ToSoftObjectPath()); }
			);
		}
		else
		{
			Algo::TransformIf(SelectedAssets, Result,
				[](const FAssetData& AssetData) { return AssetData.GetClass() == ExpectedAssetType::StaticClass(); },
				[](const FAssetData& AssetData) { return TSoftObjectPtr<ExpectedAssetType>(AssetData.ToSoftObjectPath()); }
			);
		}

		return Result;
	}

	/** Read-only access to the list of currently selected items */
	UE_API const TArray<FContentBrowserItem>& GetSelectedItems() const;

	/**
	 * Finds the Content Browser MenuContext from a Menu or Section, and returns the context provided there are some
	 * selected assets.
	 */
	template<typename MenuOrSectionType>
	static const UContentBrowserAssetContextMenuContext* FindContextWithAssets(const MenuOrSectionType& MenuOrSection)
	{
		const UContentBrowserAssetContextMenuContext* Context = MenuOrSection.template FindContext<UContentBrowserAssetContextMenuContext>();
		if (!Context || Context->SelectedAssets.IsEmpty())
		{
			return nullptr;
		}

		return Context;
	}

	template<typename MenuOrSectionType>
	static int32 GetNumAssetsSelected(const MenuOrSectionType& MenuOrSection)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuOrSection))
		{
			return Context->SelectedAssets.Num();
		}
		return 0;
	}

	template<typename ExpectedType, typename MenuOrSectionType>
	static ExpectedType* LoadSingleSelectedAsset(const MenuOrSectionType& MenuOrSection)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuOrSection))
		{
			if (ensure(Context->SelectedAssets.Num() == 1))
			{
				return Cast<ExpectedType>(Context->SelectedAssets[0].GetAsset());
			}
		}
		return nullptr;
	}
};

UCLASS(MinimalAPI)
class UContentBrowserAssetViewContextMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<SContentBrowser> OwningContentBrowser;
	TWeakPtr<SAssetView> AssetView;
};

UCLASS(MinimalAPI)
class UContentBrowserAssetSortingContextMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<SContentBrowser> OwningContentBrowser;
	TWeakPtr<SAssetView> AssetView;
	TWeakPtr<FAssetViewSortManager> AssetViewSortManager;
};

UCLASS(MinimalAPI)
class UContentBrowserMenuContext : public UObject
{
	GENERATED_BODY()

public:

	TWeakPtr<SContentBrowser> ContentBrowser;
};

UCLASS(MinimalAPI)
class UContentBrowserFolderContext : public UContentBrowserMenuContext
{
	GENERATED_BODY()

public:

	UPROPERTY(BlueprintReadOnly, Category = "ContentBrowser")
	bool bCanBeModified;

	UPROPERTY(BlueprintReadOnly, Category = "ContentBrowser")
	bool bNoFolderOnDisk;

	UPROPERTY(BlueprintReadOnly, Category = "ContentBrowser")
	int32 NumAssetPaths;

	UPROPERTY(BlueprintReadOnly, Category = "ContentBrowser")
	int32 NumClassPaths;

	UPROPERTY(BlueprintReadOnly, Category = "ContentBrowser")
	TArray<FString> SelectedPackagePaths;

	FOnCreateNewFolder OnCreateNewFolder;

	const TArray<FString>& GetSelectedPackagePaths() const { return SelectedPackagePaths; }
};

UCLASS(MinimalAPI)
class UContentBrowserAddNewContextMenuContext : public UObject
{
	GENERATED_BODY()

public:

	TWeakPtr<SContentBrowser> ContentBrowser;
};

UCLASS(MinimalAPI)
class UContentBrowserToolbarMenuContext : public UObject
{
	GENERATED_BODY()

public:
	UE_API FName GetCurrentPath() const;

	UE_API bool CanWriteToCurrentPath() const;

	TWeakPtr<SContentBrowser> ContentBrowser;
	TWeakPtr<SAssetView> AssetView;
	const FContentBrowserConfig* ContentBrowserConfig = nullptr;
};

class UToolMenu;

namespace UE::ContentBrowser
{
	CONTENTBROWSER_API UToolMenu* ExtendToolMenu_AssetContextMenu(UClass* AssetClass);
	CONTENTBROWSER_API UToolMenu* ExtendToolMenu_AssetContextMenu(TSoftClassPtr<UObject> AssetSoftClass);
	CONTENTBROWSER_API TArray<UToolMenu*> ExtendToolMenu_AssetContextMenus(TConstArrayView<UClass*> AssetClasses);
}

#undef UE_API
