// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "ContentBrowserDelegates.h"
#include "CoreMinimal.h"
#include "AssetTypeCategories.h"
#include "IAssetTools.h"
#include "Delegates/Delegate.h"
#include "Experimental/ContentBrowserSearchOverride.h"
#include "Experimental/ContentBrowserViewExtender.h"
#include "HAL/PlatformCrt.h"
#include "MRUFavoritesList.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

class FContentBrowserPluginFilter;
class FMainMRUFavoritesList;
class FString;
class FText;
class IContentBrowserSingleton;
struct FARFilter;
struct FAssetData;

/** Extra state generator that adds an icon and a corresponding legend entry on an asset. */
class FAssetViewExtraStateGenerator
{
public:
	FAssetViewExtraStateGenerator(FOnGenerateAssetViewExtraStateIndicators InIconGenerator, FOnGenerateAssetViewExtraStateIndicators InToolTipGenerator)
		: IconGenerator(MoveTemp(InIconGenerator))
		, ToolTipGenerator(MoveTemp(InToolTipGenerator))
		, Handle(FDelegateHandle::GenerateNewHandle)
	{}

	/** Delegate called to generate an extra icon on an asset view. */
	FOnGenerateAssetViewExtraStateIndicators IconGenerator;
	
	/** Delegate called to generate an extra tooltip on an asset view. */
	FOnGenerateAssetViewExtraStateIndicators ToolTipGenerator;

private:
	/** The handle to this extra state generator. */
	FDelegateHandle Handle;

	friend class FContentBrowserModule;
};

// Workflow event when a collection is created
struct FCollectionCreatedTelemetryEvent
{
	static inline constexpr FGuid TelemetryID = FGuid(0x2F9C8896, 0xCB2C402B, 0xB8D6DCF1, 0xE3F22D41);
	
	double DurationSec = 0.0;
	ECollectionShareType::Type CollectionShareType = ECollectionShareType::CST_All;
};

// Workflow event when a set of collections are deleted
struct FCollectionsDeletedTelemetryEvent
{
	static inline constexpr FGuid TelemetryID = FGuid(0x1362ABC8, 0x6CDD4FF9, 0xB4B0E06C, 0x2CA418DC);
	
	double DurationSec = 0.0;
	int32 CollectionsDeleted = 0;
};

enum class ECollectionTelemetryAssetAddedWorkflow : int32
{
	ContextMenu = 0,
	DragAndDrop = 1
};

// Workflow event when assets are added to a collection
struct FAssetAddedToCollectionTelemetryEvent
{
	static inline constexpr FGuid TelemetryID = FGuid(0x3676C84E, 0xFEB74E21, 0x99FF6FB8, 0x622431D2);
	
	double DurationSec;
	ECollectionShareType::Type CollectionShareType;
	uint32 NumAdded;
	ECollectionTelemetryAssetAddedWorkflow Workflow;
};

enum class ECollectionTelemetryAssetRemovedWorkflow : int32
{
	ContextMenu = 0
};

// Workflow event when assets are removed from a collection
struct FAssetRemovedFromCollectionTelemetryEvent
{
	static inline constexpr FGuid TelemetryID = FGuid(0xF7660FA4, 0x744F44F5, 0x9B647690, 0x3C5689BD);
	
	double DurationSec;
	ECollectionShareType::Type CollectionShareType;
	uint32 NumRemoved;
	ECollectionTelemetryAssetRemovedWorkflow Workflow;
};

/**
 * Content browser module
 */
class FContentBrowserModule : public IModuleInterface
{

public:

	/**  */
	DECLARE_MULTICAST_DELEGATE_TwoParams( FOnFilterChanged, const FARFilter& /*NewFilter*/, bool /*bIsPrimaryBrowser*/ );
	/** */
	DECLARE_MULTICAST_DELEGATE_TwoParams( FOnSearchBoxChanged, const FText& /*InSearchText*/, bool /*bIsPrimaryBrowser*/ );
	/** */
	DECLARE_MULTICAST_DELEGATE_TwoParams( FOnAssetSelectionChanged, const TArray<FAssetData>& /*NewSelectedAssets*/, bool /*bIsPrimaryBrowser*/ );
	/** */
	DECLARE_MULTICAST_DELEGATE_OneParam( FOnSourcesViewChanged, bool /*bExpanded*/ );
	/** */
	DECLARE_MULTICAST_DELEGATE_OneParam( FOnAssetPathChanged, const FString& /*NewPath*/ );
	/** */
	DECLARE_DELEGATE_OneParam( FAddPathViewPluginFilters, TArray<TSharedRef<FContentBrowserPluginFilter>>& /*Filters*/ );
	/** */
	DECLARE_MULTICAST_DELEGATE_OneParam( FOnContentBrowserSettingChanged, FName /*PropertyName*/);
	/** */
	DECLARE_DELEGATE_OneParam( FDefaultSelectedPathsDelegate, TArray<FName>& /*VirtualPaths*/ );
	/** */
	DECLARE_DELEGATE_OneParam( FDefaultPathsToExpandDelegate, TArray<FName>& /*VirtualPaths*/ );
	/** Delegate that creates an instance of the custom view extender */
	DECLARE_DELEGATE_RetVal(TSharedPtr<IContentBrowserViewExtender>, FCreateViewExtender);
	
	/**
	 * Called right after the plugin DLL has been loaded and the plugin object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the plugin is unloaded, right before the plugin object is destroyed.
	 */
	virtual void ShutdownModule();

	/** Gets the content browser singleton */
	virtual IContentBrowserSingleton& Get() const;

	/**
	 * Add a generator to add extra state functionality to the content browser's assets.
	 * @param Generator the delegates that add functionality. 
	 * @return FDelegateHandle the handle to the extra state generator.
	 */
	virtual FDelegateHandle AddAssetViewExtraStateGenerator(const FAssetViewExtraStateGenerator& Generator);

	/**
	 * Remove an asset view extra state generator.
	 * @param GeneratorHandle the extra state generator's handle.
	 */
	virtual void RemoveAssetViewExtraStateGenerator(const FDelegateHandle& GeneratorHandle);

	/** Delegates to be called to extend the content browser menus */
	virtual TArray<FContentBrowserMenuExtender_SelectedPaths>& GetAllAssetContextMenuExtenders() {return AssetContextMenuExtenders;}
	virtual TArray<FContentBrowserMenuExtender_SelectedPaths>& GetAllPathViewContextMenuExtenders() {return PathViewContextMenuExtenders;}
	virtual TArray<FContentBrowserMenuExtender>& GetAllCollectionListContextMenuExtenders() {return CollectionListContextMenuExtenders;}
	virtual TArray<FContentBrowserMenuExtender>& GetAllCollectionViewContextMenuExtenders() {return CollectionViewContextMenuExtenders;}
	virtual TArray<FContentBrowserMenuExtender_SelectedAssets>& GetAllAssetViewContextMenuExtenders() {return AssetViewContextMenuExtenders;}
	virtual TArray<FContentBrowserMenuExtender>& GetAllAssetViewViewMenuExtenders() {return AssetViewViewMenuExtenders;}
	virtual TArray<FPathViewStateIconGenerator>& GetAllPathViewStateIconGenerators() {return PathViewStateIconGenerators;}

	/** Delegates to call to extend the command/keybinds for content browser */
	virtual TArray<FContentBrowserCommandExtender>& GetAllContentBrowserCommandExtenders() { return ContentBrowserCommandExtenders; }

	/** Delegates to be called to add extra state indicators on the asset view */
	virtual const TArray<FAssetViewExtraStateGenerator>& GetAllAssetViewExtraStateGenerators() { return AssetViewExtraStateGenerators; }

	/** Delegates to be called to extend the drag-and-drop support of the asset view */
	virtual TArray<FAssetViewDragAndDropExtender>& GetAssetViewDragAndDropExtenders() { return AssetViewDragAndDropExtenders; }

	/** Delegates to be called to extend list of content browser Plugin Filters*/
	virtual TArray<FAddPathViewPluginFilters>& GetAddPathViewPluginFilters() { return PathViewPluginFilters; }

	/** Delegates to call to provide the value for a ContentBrowserItemData's attribute */
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	UE_EXPERIMENTAL(5.7, "GetAllItemDataAttributeDelegates() is experimental while a newer system is in active development.")
	virtual TArray<FGetItemDataAttribute>& GetAllItemDataAttributeDelegates() { return ItemDataAttributeDelegates; }
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	/** Register a delegate that creates a custom view that can show up inside SAssetView in the Content Browser */
	virtual void SetContentBrowserViewExtender(const FCreateViewExtender& InCreateViewExtender);
	/** Get the currently registered custom view */
	virtual FCreateViewExtender GetContentBrowserViewExtender();

	/** Delegate accessors */
	FOnFilterChanged& GetOnFilterChanged() { return OnFilterChanged; } 
	FOnSearchBoxChanged& GetOnSearchBoxChanged() { return OnSearchBoxChanged; } 
	FOnAssetSelectionChanged& GetOnAssetSelectionChanged() { return OnAssetSelectionChanged; } 
	FOnSourcesViewChanged& GetOnSourcesViewChanged() { return OnSourcesViewChanged; }
	FOnAssetPathChanged& GetOnAssetPathChanged() { return OnAssetPathChanged; }
	FOnContentBrowserSettingChanged& GetOnContentBrowserSettingChanged() { return OnContentBrowserSettingChanged; }
	FOnDeleteFolders& GetOnDeleteFolders() { return OnDeleteFolders; }
	FOnContentBrowserCanShowSetAssetAccessMenu& GetOnContentBrowserCanShowSetAssetAccessMenu() { return OnContentBrowserCanShowSetAssetAccessMenu; }

	/** Override list of paths to select by default */
	FDefaultSelectedPathsDelegate& GetDefaultSelectedPathsDelegate() { return DefaultSelectedPathsDelegate; }

	/** Override list of paths to expand by default */
	FDefaultPathsToExpandDelegate& GetDefaultPathsToExpandDelegate() { return DefaultPathsToExpandDelegate; }

	FMainMRUFavoritesList* GetRecentlyOpenedAssets() const;

	static const FName NumberOfRecentAssetsName;

	CONTENTBROWSER_API void RegisterDynamicTagAssetClass(const FTopLevelAssetPath& InClassPathName);
	CONTENTBROWSER_API void UnregisterDynamicTagAssetClass(const FTopLevelAssetPath& InClassPathName);
	CONTENTBROWSER_API bool IsDynamicTagAssetClass(const FTopLevelAssetPath& InClassPathName) const;
	
	UE_DEPRECATED(5.8, "Use the new version of this function, which uses a FTopLevelAssetPath")
	void AddDynamicTagAssetClass(const FName& InName) {}
	UE_DEPRECATED(5.8, "Use the new version of this function, which uses a FTopLevelAssetPath")
	void RemoveDynamicTagAssetClass(const FName& InName) {}
	UE_DEPRECATED(5.8, "Use the new version of this function, which uses a FTopLevelAssetPath")
	bool IsDynamicTagAssetClass(const FName& InName) { return false; }

	FOnSetFolderColor& GetOnSetFolderColor()
	{
		return OnSetFolderColor;
	}

	struct FWizard
	{
		FName WizardIdentifier;
		FText DisplayName;
		FText Description;
		FSlateIcon Icon;
		FAdvancedAssetCategory CategoryPath;
		FSimpleDelegate OnOpen;
	};

	CONTENTBROWSER_API void RegisterWizard(const FWizard& InWizard);
	CONTENTBROWSER_API void UnregisterWizard(const FName WizardIdentifier);
	CONTENTBROWSER_API const TArray<FWizard>& GetWizards() const;


	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	UE_EXPERIMENTAL(5.8, "Asset Search Override is experimental.")
	CONTENTBROWSER_API bool RegisterAssetSearchOverride(TUniquePtr<UE::Editor::ContentBrowser::Extension::IAssetSearchOverride>&& InOverride);
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	UE_EXPERIMENTAL(5.8, "Asset Search Override is experimental.")
	CONTENTBROWSER_API void UnregisterAssetSearchOverride();
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	UE_EXPERIMENTAL(5.8, "Asset Search Override is experimental.")
	CONTENTBROWSER_API UE::Editor::ContentBrowser::Extension::IAssetSearchOverride* GetActiveAssetSearchOverride() const;
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

private:
	TArray<FWizard> Wizards;
	
	/** Handle changes to content browser settings */
	void ContentBrowserSettingChanged(FName InName);
	
	/** List of asset classes whose tags are dynamic and therefore we should union all asset's tags rather than grabbing the first available. */
	TArray<FTopLevelAssetPath> AssetClassesRequiringDynamicTags;

	/** Called when a custom folder color is set on a folder */
	FOnSetFolderColor OnSetFolderColor;

private:
	IContentBrowserSingleton* ContentBrowserSingleton;
	TSharedPtr<class FContentBrowserSpawner> ContentBrowserSpawner;
	
	/** All extender delegates for the content browser menus */
	TArray<FContentBrowserMenuExtender_SelectedPaths> AssetContextMenuExtenders;
	TArray<FContentBrowserMenuExtender_SelectedPaths> PathViewContextMenuExtenders;
	TArray<FContentBrowserMenuExtender> CollectionListContextMenuExtenders;
	TArray<FContentBrowserMenuExtender> CollectionViewContextMenuExtenders;
	TArray<FContentBrowserMenuExtender_SelectedAssets> AssetViewContextMenuExtenders;
	TArray<FContentBrowserMenuExtender> AssetViewViewMenuExtenders;
	TArray<FPathViewStateIconGenerator> PathViewStateIconGenerators;
	TArray<FContentBrowserCommandExtender> ContentBrowserCommandExtenders;

	/** All delegates generating extra state indicators */
	TArray<FAssetViewExtraStateGenerator> AssetViewExtraStateGenerators;

	/** All extender delegates for the drag-and-drop support of the asset view */
	TArray<FAssetViewDragAndDropExtender> AssetViewDragAndDropExtenders;

	/** All delegates that extend available path view plugin filters */
	TArray<FAddPathViewPluginFilters> PathViewPluginFilters;

	/** All delegates that return values for ContentItemData attributes */
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	TArray<FGetItemDataAttribute> ItemDataAttributeDelegates;
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	FOnFilterChanged OnFilterChanged;
	FOnSearchBoxChanged OnSearchBoxChanged;
	FOnAssetSelectionChanged OnAssetSelectionChanged;
	FOnSourcesViewChanged OnSourcesViewChanged;
	FOnAssetPathChanged OnAssetPathChanged;
	FOnContentBrowserSettingChanged OnContentBrowserSettingChanged;
	FOnDeleteFolders OnDeleteFolders;

	FDefaultSelectedPathsDelegate DefaultSelectedPathsDelegate;
	FDefaultPathsToExpandDelegate DefaultPathsToExpandDelegate;

	/** Extension used to provide a custom view to the Content Browser */
	FCreateViewExtender ContentBrowserViewExtender;

	/** The currently active asset search override, if any. Only one may be registered at a time. */
	TUniquePtr<UE::Editor::ContentBrowser::Extension::IAssetSearchOverride> ActiveAssetSearchOverride = nullptr;

	/** Override to selectively hide menu entries for viewing and setting asset as Public/Private */
	FOnContentBrowserCanShowSetAssetAccessMenu OnContentBrowserCanShowSetAssetAccessMenu;
};
