// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "IContentBrowserSingleton.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/IFilter.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

#define LOCTEXT_NAMESPACE "ContentBrowser"

class FContentBrowserPluginFilter : public IFilter<FPluginFilterType>
{
public:

	/** Returns the system name for this filter */
	virtual FString GetName() const = 0;

	/** Returns the human readable name for this filter */
	virtual FText GetDisplayName() const = 0;

	/** Returns the tooltip for this filter, shown in the filters menu */
	virtual FText GetToolTipText() const = 0;
	
	/** Returns the color this filter button will be when displayed as a button */
	virtual FLinearColor GetColor() const { return FLinearColor(0.6f, 0.6f, 0.6f, 1); }

	/** Returns the name of the icon to use in menu entries */
	virtual FName GetIconName() const { return NAME_None; }

	/** Returns true if the filter should be in the list when disabled and not in the list when enabled */
	virtual bool IsInverseFilter() const { return false; }

	/** Notification that the filter became active or inactive */
	virtual void ActiveStateChanged(bool bActive) { }

	/** Called when the state of a particular Content Browser is being saved to INI */
	virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const {}

	/** Called when the state of a particular Content Browser is being loaded from INI */
	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) {}

	/** Set this filter as active/inactive */
	void SetActive(bool bInActive) { SetActiveEvent.Broadcast(bInActive); }

	// IFilter implementation
	DECLARE_DERIVED_EVENT( FContentBrowserPluginFilter, IFilter<FPluginFilterType>::FChangedEvent, FChangedEvent );
	virtual FChangedEvent& OnChanged() override { return ChangedEvent; }

protected:
	void BroadcastChangedEvent() const { ChangedEvent.Broadcast(); }

private:
	FChangedEvent ChangedEvent;

	/** This event is broadcast to set this filter active in the content browser it is being used in */
	DECLARE_EVENT_OneParam(FContentBrowserPluginFilter, FSetActiveEvent, bool);
	FSetActiveEvent SetActiveEvent;
};

class FContentBrowserPluginFilter_ContentOnlyPlugins : public FContentBrowserPluginFilter
{
public:
	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("Content Only Plugins"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("ContentBrowserPluginFilter_ContentOnlyPlugins", "Content Only Plugins"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("ContentBrowserPluginFilter_ContentOnlyPluginsTooltip", "Show Content Only Plugins"); }

	// IFilter implementation
	virtual bool PassesFilter(FPluginFilterType InItem) const override;
};

#undef LOCTEXT_NAMESPACE
