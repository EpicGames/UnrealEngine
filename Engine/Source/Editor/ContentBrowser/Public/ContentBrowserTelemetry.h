// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IContentBrowserSingleton.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"

struct FContentBrowserDataFilter;

namespace UE::Telemetry::ContentBrowser
{
	struct FBackendFilterTelemetry
	{
		static inline constexpr FGuid TelemetryID = FGuid(0x3a026bff,0x041444d0, 0x9457c4e5, 0x1378d03c);

		FBackendFilterTelemetry(FGuid InViewCorrelationGuid, FGuid InFilterSessionCorrelationGuid)
		: ViewCorrelationGuid(InViewCorrelationGuid)
		, FilterSessionCorrelationGuid(InFilterSessionCorrelationGuid)
		{
		}

		// Guid assigned to the originating instance of SAssetView
		FGuid ViewCorrelationGuid;
		// Guid assigned to this set of backend items for correlation with frontend filtering	
		FGuid FilterSessionCorrelationGuid;
		// Whether this view had a delegate bound for providing custom items
		bool bHasCustomItemSources = false;
		// Wall-clock duration of fetching all items from the backend
		double RefreshSourceItemsDurationSeconds = 0.0;
		// Number of items returned from the backend
		int32 NumBackendItems = 0;
		// Optional pointer to the filter passed to the backend - not to be stored past the lifetime of the telemetry handler.
		const FContentBrowserDataFilter* DataFilter = nullptr;
	};
	
	struct FFrontendFilterTelemetry
	{
		static inline constexpr FGuid TelemetryID = FGuid(0xee47ffd0, 0x86c74117, 0x86416a64, 0x85460b31);

		FFrontendFilterTelemetry()
		{
		}

		FFrontendFilterTelemetry(FGuid InViewCorrelationGuid, FGuid InFilterSessionCorrelationGuid)
		: ViewCorrelationGuid(InViewCorrelationGuid)
		, FilterSessionCorrelationGuid(InFilterSessionCorrelationGuid)
		{
		}

		// Guid assigned to the originating instance of SAssetView
		FGuid ViewCorrelationGuid = FGuid();
		// Guid assigned to the set of backend items the frontend is filtering here	
		FGuid FilterSessionCorrelationGuid = FGuid();
		// The total number of items returned from the backend that needed filtering
		int32 TotalItemsToFilter = 0;
		// The number of items that were required to be filtered immediately
		int32 PriorityItemsToFilter = 0;
        // Total number of items that passed the filters 
        int32 TotalResults = 0;
		// Wall clock duration of filtering all items
		double AmortizeDuration = 0.0;
		// Actual cumulative tick time of filtering 
		double WorkDuration = 0.0;
        // Latency until first result is added (if any results were found)
        TOptional<double> ResultLatency;
		// Time until user interacted with filter results - e.g. opened an asset from the list, even if filtering was not complete
		TOptional<double> TimeUntilInteraction;
		// Whether this filtering session completed before being interrupted/cancelled in some way
		bool bCompleted = false;
		// 'Frontend' filter (amortized in asset view tick) 
		// 'Backend' filter data is available in backend items telemetry - requires correlation 
		TSharedPtr<FAssetFilterCollectionType> FrontendFilters;
	};
}