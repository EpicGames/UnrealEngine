// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentSources/IContentSource.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "ContentSourcesColumns.generated.h"

/**
 * Column added to a widget row that contains a pointer to the content source the widget belongs to
 */
USTRUCT(meta = (DisplayName = "Content Source"))
struct FContentSourceColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	TWeakPtr<UE::Editor::ContentBrowser::IContentSource> ContentSource;
};