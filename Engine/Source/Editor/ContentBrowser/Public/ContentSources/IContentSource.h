// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Queries/Description.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Framework/Views/ITypedTableView.h"
#include "Internationalization/Text.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"

namespace UE::Editor::ContentBrowser
{
	/*
	 * Init Params used by the STedsTableViewer displayed in place of the legacy Content Browser when this content source is active
	 */
	struct FTableViewerInitParams
	{
		// Query description to populate the rows of the table viewer
		DataStorage::FQueryDescription QueryDescription;

		// The columns shown in the table viewer
		TArray<TWeakObjectPtr<const UScriptStruct>> Columns;

		// The widget purposes used by the table viewer to display the widgets in the cells
		DataStorage::IUiProvider::FPurposeID CellWidgetPurpose;

		// The table view type to use, currently only supports list and tile
		ETableViewMode::Type TableViewMode = ETableViewMode::Type::List;
	};

	/*
	 * A Content Source is a construct that can customize the look and behavior of the content browser through TEDS (Editor Data Storage) intended
	 * to replace the legacy asset specific content browser.
	 * Currently supports displaying an STedsTableViewer driven by a query specified by the content source in place of the legacy content browser
	 * layout and widgets
	 * Content Sources can be registered with IContentBrowserSingleton which will cause them to be displayed in a vertical toolbar next to the
	 * Content Browser allowing a user to select and activate them.
	 * 
	 * NOTE: This API is experimental and subject to change
	 */
	class IContentSource
	{
	public:
		virtual ~IContentSource() = default;

		// Get the internal name of the content source
		virtual FName GetName() = 0;

		// Get the user facing name of the content source
		virtual FText GetDisplayName() = 0;

		// Get an icon representing the content source
		virtual FSlateIcon GetIcon() = 0;

		// Get the init params used by the TEDS Table Viewer when this content source is active
		virtual void GetAssetViewInitParams(FTableViewerInitParams& OutInitParams) = 0;

		// Called when this content source is switched in
		virtual void OnContentSourceEnabled() {}

		// Called when this content source is swapped out
		virtual void OnContentSourceDisabled() {}
	};
}