// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Math/Color.h"

struct FContentBrowserItem;
struct FContentBrowserItemPath;

namespace UE::Editor::ContentBrowser::ExtensionUtils
{
	// Get the custom color of the given folder (if any). Prefer using this over raw FName variants.
	UE_EXPERIMENTAL(5.8, "The Content Browser extension utils are experimental hooks that may be removed in the future")
	CONTENTBROWSER_API TOptional<FLinearColor> GetFolderColor(const FContentBrowserItem& FolderItem);

	// Get the custom color of the given folder path (if any)
	UE_EXPERIMENTAL(5.8, "The Content Browser extension utils are experimental hooks that may be removed in the future")
	CONTENTBROWSER_API TOptional<FLinearColor> GetFolderColor(const FContentBrowserItemPath& FolderPath);

	// Get the custom color of the given internal folder path (if any)
	UE_EXPERIMENTAL(5.8, "The Content Browser extension utils are experimental hooks that may be removed in the future")
	CONTENTBROWSER_API TOptional<FLinearColor> GetFolderColor(const FName& FolderPath);

	// Set a custom color for the given folder path
	UE_EXPERIMENTAL(5.8, "The Content Browser extension utils are experimental hooks that may be removed in the future")
	CONTENTBROWSER_API void SetFolderColor(const FName& FolderPath, const FLinearColor& FolderColor);

	// Returns if this folder has been marked as a favorite folder
	UE_EXPERIMENTAL(5.8, "The Content Browser extension utils are experimental hooks that may be removed in the future")
	CONTENTBROWSER_API bool IsFolderFavorite(const FContentBrowserItemPath& ItemPath);

	// Set if a folder is favorite
	UE_EXPERIMENTAL(5.8, "The Content Browser extension utils are experimental hooks that may be removed in the future")
	CONTENTBROWSER_API void SetFolderFavorite(const FString& FolderPath, bool bIsFavorite);
	
}
