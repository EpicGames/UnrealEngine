// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "ContentBrowserItem.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Types/SlateEnums.h"

class FAssetViewItem;

namespace UE::Editor::ContentBrowser::Extension
{
	/**
	 * Interface to contains the state per view of the search override
	 */
	class ICBSearchOverrideInstance
	{
	public:
		CONTENTBROWSER_API ICBSearchOverrideInstance();
		CONTENTBROWSER_API virtual ~ICBSearchOverrideInstance();

		CONTENTBROWSER_API bool IsOverrideActive() const;
		CONTENTBROWSER_API void ToggleOverride();

		/**
		 * Call from the override to notify the Content Browser to update the asset view's user-searching state.
		 * Pass true when a search is in progress, false when results are cleared.
		 */
		CONTENTBROWSER_API void SetUserSearching(bool bIsSearching);

		/**
		 * Call from the override to trigger a quick frontend list refresh on the asset view.
		 * Use this when the search query changes so the filter pipeline re-runs immediately.
		 */
		CONTENTBROWSER_API  void RequestQuickFrontendListRefresh();

		virtual TSharedRef<class SWidget> GetOverrideModeToggleWidget() = 0;

		/** Called when the search box text changes, only when IsOverrideActive() returns true. */
		virtual void OnSearchTextChanged(const FText& NewText) = 0;

		/** Called when the search box text is committed, only when IsOverrideActive() returns true. */
		virtual void OnSearchTextCommitted(const FText& NewText, ETextCommit::Type CommitType) = 0;


		/**
		 * Called each filter pass with all items that passed the frontend filters.
		 * The override is responsible for deciding which of these to surface via PublishResults().
		 */
		virtual void OnItemsAvailable(TArrayView<const TSharedPtr<FAssetViewItem>> Items) = 0;

		virtual void SortItemList(TArrayView<TSharedPtr<FAssetViewItem>> Items) = 0;

		/**
		 * Returns true if the override wants to handle sorting instead of the default
		 * Content Browser sort manager. When false, SortItemList will not be called.
		 */
		virtual bool IsSortOverridden() const { return true; }

		/**
		 * Returns true while the override has async work in flight whose results
		 * have not yet been published. The asset view will keep ticking while this is true.
		 */
		virtual bool HasPendingResults() const { return false; }

		/**
		 * Call when results are ready. The asset view will inject these items into the
		 * visible set on the next tick.
		 */
		CONTENTBROWSER_API void PublishResults(TArrayView<TSharedPtr<FAssetViewItem>> Items);

		/** Private API for ContentBrowser internals. */
		class FInternalApiWrapper& GetInternalApi();

		friend class FInternalApiWrapper;

	protected:

		/**
		 * Called at the start of each filter pass when the override is active.
		 * Override to reset any pending state before new items start arriving.
		 */
		virtual void OnFilteringResetImplementation() = 0;

		/**
		 * Called when the extension has received all the known items that it has to process.
		 */
		virtual void OnAllKnownItemsAvailableImplementation() = 0;

	private:
		bool bIsOverrideActive = false;
		TUniquePtr<FInternalApiWrapper> InternalApi;
	};


	/**
	 * Interface for an asset search override that can hook into the Content Browser's
	 * filter and sort pipeline. Only one override may be active at any given time.
	 */
	class IAssetSearchOverride
	{
	public:
		virtual ~IAssetSearchOverride() = default;

		virtual TSharedRef<ICBSearchOverrideInstance> MakePerViewOverride() const = 0;
	};
}
