// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetThumbnail.h"
#include "AssetViewTypes.h"
#include "Delegates/DelegateCombinations.h"
#include "Framework/SlateDelegates.h"

class SWidget;

/*
 * Interface that can be used to add a custom view to the Content Browser
 * NOTE: This API is likely to change as it is being actively iterated on
 */
class IContentBrowserViewExtender
{
public:
	// Create the widget for the view with the given input items
	virtual TSharedRef<SWidget> CreateView(TArray<TSharedPtr<FAssetViewItem>>* InItemsSource) = 0;

	virtual ~IContentBrowserViewExtender() = default;
	
	// Function called when the items source has some changes and the view needs a refresh
    virtual void OnItemListChanged(TArray<TSharedPtr<FAssetViewItem>>* InItemsSource) = 0;

	using FOnSelectionChanged = TSlateDelegates<TSharedPtr<FAssetViewItem>>::FOnSelectionChanged;
	// Delegate the extender must fire when the selection in the UI changes
	virtual FOnSelectionChanged& OnSelectionChanged() = 0;

	using FOnItemScrolledIntoView = TSlateDelegates<TSharedPtr<FAssetViewItem>>::FOnItemScrolledIntoView;
	// Delegate the extender must fire when an item is scrolled into view in the UI
	virtual FOnItemScrolledIntoView& OnItemScrolledIntoView() = 0;
	
	using FOnMouseButtonClick = TSlateDelegates<TSharedPtr<FAssetViewItem>>::FOnMouseButtonClick;
	// Delegate the extender must fire when an item is double clicked
	virtual FOnMouseButtonClick& OnItemDoubleClicked() = 0;
	
	// Delegate the extender must fire when the context menu is opened on the list
	virtual FOnContextMenuOpening& OnContextMenuOpened() = 0;

	// Get the items selected by the custom view
	virtual TArray<TSharedPtr<FAssetViewItem>> GetSelectedItems() = 0;

	// Select the given item in the view
	virtual void SetSelection(const TSharedPtr<FAssetViewItem>& Item, bool bSelected, const ESelectInfo::Type SelectInfo) = 0;

	// Clear the current selection in the view
	virtual void ClearSelection() = 0;

	// Transfer focus to the internal view widget
	virtual void FocusList() = 0;

	// Scroll the given item into view
	virtual void RequestScrollIntoView(const TSharedPtr<FAssetViewItem>& Item) = 0;

	// true if the view is currently right click scrolling
	virtual bool IsRightClickScrolling() = 0;

	// Get a display name for the view used in the Settings menu
	virtual FText GetViewDisplayName() = 0;
	
	// Get a tooltip for the custom view
	virtual FText GetViewTooltipText() = 0;
};