// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/IFilter.h"
#include "IContentBrowserSingleton.h"
#include "Filters/FilterBase.h"

class FMenuBuilder;
struct FContentBrowserDataFilter;

class FFrontendFilterCategory : public FFilterCategory
{
public:
	FFrontendFilterCategory(const FText& InTitle, const FText& InTooltip) : FFilterCategory(InTitle,InTooltip) {}
};

class FFrontendFilter : public FFilterBase<FAssetFilterType>
{
public:
	FFrontendFilter(TSharedPtr<FFrontendFilterCategory> InCategory) : FFilterBase<FAssetFilterType>(MoveTemp(InCategory)) {}

	/** Invoke to set the source filter that is currently used to filter assets in the asset view */
	virtual void SetCurrentFilter(TArrayView<const FName> InSourcePaths, const FContentBrowserDataFilter& InBaseFilter) { }
	
	/** If this filter is a saved custom text filter, return the text query it represents. */
	virtual TOptional<FText> GetAsCustomTextFilter() { return TOptional<FText>(); }
	
	// FFilterBase Interface
	
	/** Returns the color this filter button will be when displayed as a button */
	virtual FLinearColor GetColor() const override { return FLinearColor(0.6f, 0.6f, 0.6f, 1); }

	/** Returns the name of the icon to use in menu entries */
	virtual FName GetIconName() const override { return NAME_None; }

	/** Returns true if the filter should be in the list when disabled and not in the list when enabled */
	virtual bool IsInverseFilter() const override { return false; }
	
	/** Notification that the filter became active or inactive */
	virtual void ActiveStateChanged(bool bActive) override { }

	/** Called when the right-click context menu is being built for this filter */
	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override { }

	/** Called when the state of a particular Content Browser is being saved to INI */
	virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const {}

	/** Called when the state of a particular Content Browser is being loaded from INI */
	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) {}

};
