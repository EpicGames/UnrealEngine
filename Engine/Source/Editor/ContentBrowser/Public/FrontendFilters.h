// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CollectionManagerTypes.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/IDelegateInstance.h"
#include "FrontendFilterBase.h"
#include "HAL/Platform.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"
#include "Internationalization/Text.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Misc/TextFilterUtils.h"
#include "SourceControlOperations.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"

#define UE_API CONTENTBROWSER_API

class UPackage;
struct FAssetRenameData;
struct FCollectionNameType;
struct FContentBrowserDataFilter;

#define LOCTEXT_NAMESPACE "ContentBrowser"

class FMenuBuilder;
struct FAssetCompileData;

/** A filter for text search */
UE_DEPRECATED(5.5, "FFrontendFilter_Text has been deprecated in favor of FAssetTextFilter, used by SAssetView to perform text filtering in parallel.");
class FFrontendFilter_Text : public FFrontendFilter
{
public:
	UE_API FFrontendFilter_Text();
	UE_API ~FFrontendFilter_Text();

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("TextFilter"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FrontendFilter_Text", "Text"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FrontendFilter_TextTooltip", "Show only assets that match the input text"); }

	// IFilter implementation
	UE_API virtual bool PassesFilter(FAssetFilterType InItem) const override;

public:
	/** Returns the unsanitized and unsplit filter terms */
	UE_API FText GetRawFilterText() const;

	/** Set the Text to be used as the Filter's restrictions */
	UE_API void SetRawFilterText(const FText& InFilterText);

	/** Get the last error returned from lexing or compiling the current filter text */
	UE_API FText GetFilterErrorText() const;

	/** If bIncludeClassName is true, the text filter will include an asset's class name in the search */
	UE_API void SetIncludeClassName(const bool InIncludeClassName);

	/** If bIncludeAssetPath is true, the text filter will match against full Asset path */
	UE_API void SetIncludeAssetPath(const bool InIncludeAssetPath);

	UE_API bool GetIncludeAssetPath() const;

	/** If bIncludeCollectionNames is true, the text filter will match against collection names as well */
	UE_API void SetIncludeCollectionNames(const bool InIncludeCollectionNames);

	UE_API bool GetIncludeCollectionNames() const;
private:
	/** Handles an on collection container created event */
	UE_API void HandleCollectionContainerCreated(const TSharedRef<ICollectionContainer>& CollectionContainer);

	/** Handles an on collection container destroyed event */
	UE_API void HandleCollectionContainerDestroyed(const TSharedRef<ICollectionContainer>& CollectionContainer);

	/** Handles an on collection container is hidden changed event */
	UE_API void HandleIsHiddenChanged(ICollectionContainer& CollectionContainer, bool bIsHidden);

	/** Handles an on collection created event */
	UE_API void HandleCollectionCreated(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection);

	/** Handles an on collection destroyed event */
	UE_API void HandleCollectionDestroyed(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection);

	/** Handles an on collection renamed event */
	UE_API void HandleCollectionRenamed(ICollectionContainer& CollectionContainer, const FCollectionNameType& OriginalCollection, const FCollectionNameType& NewCollection);

	/** Handles an on collection updated event */
	UE_API void HandleCollectionUpdated(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection);

	/** Rebuild the array of dynamic collections that are being referenced by the current query */
	UE_API void RebuildReferencedDynamicCollections();

	/** An array of collection containers that are being monitored */
	TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;

	/** An array of dynamic collections that are being referenced by the current query. These should be tested against each asset when it's looking for collections that contain it */
	TArray<FCollectionRef> ReferencedDynamicCollections;

	/** Transient context data, used when calling PassesFilter. Kept around to minimize re-allocations between multiple calls to PassesFilter */
	TSharedRef<class FFrontendFilter_TextFilterExpressionContext> TextFilterExpressionContext;

	/** Expression evaluator that can be used to perform complex text filter queries */
	FTextFilterExpressionEvaluator TextFilterExpressionEvaluator;

	/** Delegate handles */
	FDelegateHandle OnCollectionContainerCreatedHandle;
	FDelegateHandle OnCollectionContainerDestroyedHandle;

	struct FCollectionContainerHandles
	{
		FDelegateHandle OnIsHiddenChangedHandle;
		FDelegateHandle OnCollectionCreatedHandle;
		FDelegateHandle OnCollectionDestroyedHandle;
		FDelegateHandle OnCollectionRenamedHandle;
		FDelegateHandle OnCollectionUpdatedHandle;
	};
	TArray<FCollectionContainerHandles> CollectionContainerHandles;
};

/** A filter that displays only checked out assets */
class FFrontendFilter_CheckedOut : public FFrontendFilter, public TSharedFromThis<FFrontendFilter_CheckedOut>
{
public:
	UE_API FFrontendFilter_CheckedOut(TSharedPtr<FFrontendFilterCategory> InCategory);

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("CheckedOut"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FrontendFilter_CheckedOut", "Checked Out"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FrontendFilter_CheckedOutTooltip", "Show only assets that you have checked out or pending for add."); }
	UE_API virtual void ActiveStateChanged(bool bActive) override;
	UE_API virtual void SetCurrentFilter(TArrayView<const FName> InSourcePaths, const FContentBrowserDataFilter& InBaseFilter) override;

	// IFilter implementation
	UE_API virtual bool PassesFilter(FAssetFilterType InItem) const override;

private:
	
	/** Request the source control status for this filter */
	UE_API void RequestStatus();

	/** Callback when source control operation has completed */
	UE_API void SourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);

	bool bSourceControlEnabled;
};

/** A filter that displays assets not tracked by source control */
class FFrontendFilter_NotSourceControlled : public FFrontendFilter, public TSharedFromThis<FFrontendFilter_NotSourceControlled>
{
public:
	UE_API FFrontendFilter_NotSourceControlled(TSharedPtr<FFrontendFilterCategory> InCategory);

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("NotSourceControlled"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FrontendFilter_NotSourceControlled", "Not Revision Controlled"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FrontendFilter_NotSourceControlledTooltip", "Show only assets that are not tracked by revision control."); }
	UE_API virtual void ActiveStateChanged(bool bActive) override;
	UE_API virtual void SetCurrentFilter(TArrayView<const FName> InSourcePaths, const FContentBrowserDataFilter& InBaseFilter) override;

	// IFilter implementation
	UE_API virtual bool PassesFilter(FAssetFilterType InItem) const override;

private:

	/** Request the source control status for this filter */
	UE_API void RequestStatus();

	/** Callback when source control operation has completed */
	UE_API void SourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);

	bool bSourceControlEnabled;
	bool bIsRequestStatusRunning;
	bool bInitialRequestCompleted;
};

/** A filter that displays only modified assets */
class FFrontendFilter_Modified : public FFrontendFilter
{
public:
	UE_API FFrontendFilter_Modified(TSharedPtr<FFrontendFilterCategory> InCategory);
	UE_API ~FFrontendFilter_Modified();

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("Modified"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FrontendFilter_Modified", "Modified"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FrontendFilter_ModifiedTooltip", "Show only assets that have been modified and not yet saved."); }
	UE_API virtual void ActiveStateChanged(bool bActive) override;

	// IFilter implementation
	UE_API virtual bool PassesFilter(FAssetFilterType InItem) const override;

private:

	/** Handler for when a package's dirty state has changed */
	UE_API void OnPackageDirtyStateUpdated(UPackage* Package);

	bool bIsCurrentlyActive;
};

/** A filter that displays blueprints that have replicated properties */
class FFrontendFilter_ReplicatedBlueprint : public FFrontendFilter
{
public:
	FFrontendFilter_ReplicatedBlueprint(TSharedPtr<FFrontendFilterCategory> InCategory) : FFrontendFilter(InCategory) {}

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("ReplicatedBlueprint"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FFrontendFilter_ReplicatedBlueprint", "Replicated Blueprints"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FFrontendFilter_ReplicatedBlueprintToolTip", "Show only blueprints with replicated properties."); }

	// IFilter implementation
	UE_API virtual bool PassesFilter(FAssetFilterType InItem) const override;
};

/** A filter that compares the value of an asset registry tag to a target value */
class FFrontendFilter_ArbitraryComparisonOperation : public FFrontendFilter
{
public:
	UE_API FFrontendFilter_ArbitraryComparisonOperation(TSharedPtr<FFrontendFilterCategory> InCategory);

	// FFrontendFilter implementation
	UE_API virtual FString GetName() const override;
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual FText GetToolTipText() const override;
	UE_API virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override;
	UE_API virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const override;
	UE_API virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) override;

	// IFilter implementation
	UE_API virtual bool PassesFilter(FAssetFilterType InItem) const override;

protected:
	static UE_API FString ConvertOperationToString(ETextFilterComparisonOperation Op);
	
	UE_API void SetComparisonOperation(ETextFilterComparisonOperation NewOp);
	UE_API bool IsComparisonOperationEqualTo(ETextFilterComparisonOperation TestOp) const;

	UE_API FText GetKeyValueAsText() const;
	UE_API FText GetTargetValueAsText() const;
	UE_API void OnKeyValueTextCommitted(const FText& InText, ETextCommit::Type InCommitType);
	UE_API void OnTargetValueTextCommitted(const FText& InText, ETextCommit::Type InCommitType);

public:
	FName TagName;
	FString TargetTagValue;
	ETextFilterComparisonOperation ComparisonOp;
};

/** An inverse filter that allows display of content in developer folders that are not the current user's */
class UE_DEPRECATED(5.5, "This frontend filter has been deprecated and replaced with backend filtering. see FFilter_HideOtherDevelopers.")
CONTENTBROWSER_API FFrontendFilter_ShowOtherDevelopers : public FFrontendFilter
{
public:
	/** Constructor */
	FFrontendFilter_ShowOtherDevelopers(TSharedPtr<FFrontendFilterCategory> InCategory);

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("ShowOtherDevelopers"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FrontendFilter_ShowOtherDevelopers", "Other Developers"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FrontendFilter_ShowOtherDevelopersTooltip", "Allow display of assets in developer folders that aren't yours."); }
	virtual bool IsInverseFilter() const override { return true; }
	virtual void SetCurrentFilter(TArrayView<const FName> InSourcePaths, const FContentBrowserDataFilter& InBaseFilter) override;

	// IFilter implementation
	virtual bool PassesFilter(FAssetFilterType InItem) const override;

public:
	/** Sets if we should filter out assets from other developers */
	void SetShowOtherDeveloperAssets(bool bValue);

	/** Gets if we should filter out assets from other developers */
	bool GetShowOtherDeveloperAssets() const;

private:
	FString BaseDeveloperPath;
	TArray<ANSICHAR> BaseDeveloperPathAnsi;
	FString UserDeveloperPath;
	bool bIsOnlyOneDeveloperPathSelected;
	bool bShowOtherDeveloperAssets;
};

/** An inverse filter that allows display of object redirectors */
class UE_DEPRECATED(5.5, "FFrontendFilter_ShowRedirectors is deprecated. FFilter_ShowRedirectors is a virtual filter which controls backend search state instead.")
	CONTENTBROWSER_API FFrontendFilter_ShowRedirectors : public FFrontendFilter
{
public:
	/** Constructor */
	FFrontendFilter_ShowRedirectors(TSharedPtr<FFrontendFilterCategory> InCategory);

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("ShowRedirectors"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FrontendFilter_ShowRedirectors", "Show Redirectors"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FrontendFilter_ShowRedirectorsToolTip", "Allow display of Redirectors."); }
	virtual bool IsInverseFilter() const override { return true; }
	virtual void SetCurrentFilter(TArrayView<const FName> InSourcePaths, const FContentBrowserDataFilter& InBaseFilter) override;

	// IFilter implementation
	virtual bool PassesFilter(FAssetFilterType InItem) const override;

private:
	bool bAreRedirectorsInBaseFilter;
	FString RedirectorClassName;
};

/** A filter that only displays assets used by loaded levels */
class FFrontendFilter_InUseByLoadedLevels : public FFrontendFilter
{
public:
	/** Constructor/Destructor */
	UE_API FFrontendFilter_InUseByLoadedLevels(TSharedPtr<FFrontendFilterCategory> InCategory);
	UE_API ~FFrontendFilter_InUseByLoadedLevels() override;

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("InUseByLoadedLevels"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FrontendFilter_InUseByLoadedLevels", "In Use By Level"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FrontendFilter_InUseByLoadedLevelsToolTip", "Show only assets that are currently in use by any loaded level."); }
	UE_API virtual void ActiveStateChanged(bool bActive) override;

	// IFilter implementation
	UE_API virtual bool PassesFilter(FAssetFilterType InItem) const override;

	/** Handler for when maps change in the editor */
	UE_API void OnEditorMapChange( uint32 MapChangeFlags );

	/** Handler for when an asset is renamed */
	UE_API void OnAssetPostRename(const TArray<FAssetRenameData>& AssetsAndNames);

	/** Handler for when assets are finished compiling */
	UE_API void OnAssetPostCompile(const TArray<FAssetCompileData>& CompiledAssets);

private:
	UE_API void Refresh();
	UE_API void RegisterDelayedRefresh(float DelayInSeconds);
	UE_API void UnregisterDelayedRefresh();
	FTSTicker::FDelegateHandle DelayedRefreshHandle;
	bool bIsDirty = false;
	bool bIsCurrentlyActive = false;
};

/** A filter that only displays assets used by any level */
class FFrontendFilter_UsedInAnyLevel: public FFrontendFilter
{
public:
	/** Constructor/Destructor */
	UE_API FFrontendFilter_UsedInAnyLevel(TSharedPtr<FFrontendFilterCategory> InCategory);

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("UsedInAnyLevel"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FFrontendFilter_UsedInAnyLevel", "Used In Any Level"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FFrontendFilter_UsedInAnyLevelTooltip", "Show only assets that are used in any level."); }
	UE_API virtual void ActiveStateChanged(bool bActive) override;

	// IFilter implementation
	UE_API virtual bool PassesFilter(FAssetFilterType InItem) const override;

private:
	class IAssetRegistry* AssetRegistry;
	TSet<FName> LevelsDependencies;
};

/** A filter that only displays assets not used by any level */
class FFrontendFilter_NotUsedInAnyLevel : public FFrontendFilter
{
public:
	/** Constructor/Destructor */
	UE_API FFrontendFilter_NotUsedInAnyLevel(TSharedPtr<FFrontendFilterCategory> InCategory);

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("NotUsedInAnyLevel"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FFrontendFilter_NotUsedInAnyLevel", "Not Used In Any Level"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FFrontendFilter_NotUsedInAnyLevelTooltip", "Show only assets that are not used in any level."); }
	UE_API virtual void ActiveStateChanged(bool bActive) override;

	// IFilter implementation
	UE_API virtual bool PassesFilter(FAssetFilterType InItem) const override;

private:
	class IAssetRegistry* AssetRegistry;
	TSet<FName> LevelsDependencies;
};

/** A filter that only displays assets not used in another asset (Note It does not update itself automatically) */
class FFrontendFilter_NotUsedInAnyAsset : public FFrontendFilter
{
public:
	/** Constructor/Destructor */
	UE_API FFrontendFilter_NotUsedInAnyAsset(TSharedPtr<FFrontendFilterCategory> InCategory);

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("NotUsedInAnyAsset"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FFrontendFilter_NotUsedInAnyAsset", "Not Used In Any Asset"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FFrontendFilter_NotUsedInAnyAssetTooltip", "Show only the assets that aren't used by another asset."); }

	// IFilter implementation
	UE_API virtual bool PassesFilter(FAssetFilterType InItem) const override;

private:
	class IAssetRegistry* AssetRegistry = nullptr;
};

/** A filter that displays recently opened assets */
class FFrontendFilter_Recent : public FFrontendFilter
{
public:
	UE_API FFrontendFilter_Recent(TSharedPtr<FFrontendFilterCategory> InCategory);
	UE_API ~FFrontendFilter_Recent();

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("RecentlyOpened"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FrontendFilter_Recent", "Recently Opened"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FrontendFilter_RecentTooltip", "Show only recently opened assets."); }
	UE_API virtual void ActiveStateChanged(bool bActive) override;
	UE_API virtual void SetCurrentFilter(TArrayView<const FName> InSourcePaths, const FContentBrowserDataFilter& InBaseFilter) override;

	// IFilter implementation
	UE_API virtual bool PassesFilter(FAssetFilterType InItem) const override;

	UE_API void ResetFilter(FName InName);

private:
	UE_API void RefreshRecentPackagePaths();

	TSet<FName> RecentPackagePaths;
	bool bIsCurrentlyActive;
};

/** A filter that displays only assets that are not read only */
class FFrontendFilter_Writable : public FFrontendFilter
{
public:
	UE_API FFrontendFilter_Writable(TSharedPtr<FFrontendFilterCategory> InCategory);
	UE_API ~FFrontendFilter_Writable();

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("Writable"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FrontendFilter_Writable", "Writable"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FrontendFilter_WritableTooltip", "Show only assets that are not read only."); }

	// IFilter implementation
	UE_API virtual bool PassesFilter(FAssetFilterType InItem) const override;

private:
};

/** A filter that displays only packages that contain virtualized data  */
class FFrontendFilter_VirtualizedData : public FFrontendFilter
{
public:
	UE_API FFrontendFilter_VirtualizedData(TSharedPtr<FFrontendFilterCategory> InCategory);
	~FFrontendFilter_VirtualizedData() = default;

private:
	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("VirtualizedData"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FrontendFilter_VirtualizedData", "Virtualized Data"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FrontendFilter_VirtualizedDataTooltip", "Show only package that contain virtualized data."); }

	// IFilter implementation
	UE_API virtual bool PassesFilter(FAssetFilterType InItem) const override;
};

/** A filter that displays only assets the are mark as unsupported */
class FFrontendFilter_Unsupported : public FFrontendFilter
{
public:
	UE_API FFrontendFilter_Unsupported(TSharedPtr<FFrontendFilterCategory> InCategory);
	~FFrontendFilter_Unsupported() = default;

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("Unsupported"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FrontendFilter_Unsupported", "Unsupported"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FrontendFilter_UnsupportedTooltip", "Show only assets that are not supported by this project."); }

	// IFilter implementation
	UE_API virtual bool PassesFilter(FAssetFilterType InItem) const override;
};

/** A filter that displays only assets the are mark as Favorite */
class FFrontendFilter_Favorites : public FFrontendFilter
{
public:
	UE_API FFrontendFilter_Favorites(TSharedPtr<FFrontendFilterCategory> InCategory);
	~FFrontendFilter_Favorites() = default;

	// FFrontendFilter implementation
	virtual FString GetName() const override { return "Favorites"; }
	virtual FText GetDisplayName() const override { return LOCTEXT("FrontendFilter_Favorite", "Favorites"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FrontendFilter_FavoriteTooltip", "Filter by favorite assets."); }
	virtual FLinearColor GetColor() const override { return FStyleColors::AccentYellow.GetSpecifiedColor(); }

	// IFilter implementation
	UE_API virtual bool PassesFilter(FAssetFilterType InItem) const override;

private:
	UE::Editor::DataStorage::ICoreProvider* DataStorage;
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
