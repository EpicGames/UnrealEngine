// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "AssetViewContentSources.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Math/UnrealMathSSE.h"
#include "SourcesData.h"
#include "UObject/NameTypes.h"

class FMenuBuilder;

struct FSelectionData
{
	/** Virtual paths (for both folders and files) of the selected items */
	TSet<FName> SelectedVirtualPaths;

	int32 Num() const
	{
		return SelectedVirtualPaths.Num();
	}

	void Reset()
	{
		SelectedVirtualPaths.Reset();
	}

	void Empty()
	{
		SelectedVirtualPaths.Empty();
	}

	void AddMissingVirtualPaths(const FSelectionData& SelectionData)
	{
		for (const FName& SelectedVirtualPath : SelectionData.SelectedVirtualPaths)
		{
			SelectedVirtualPaths.Add(SelectedVirtualPath);
		}
	}
};

/** The history data object, storing all important history data */
struct FHistoryData
{
	FHistoryData() = default;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	FHistoryData(const FHistoryData&) = default;
	FHistoryData(FHistoryData&&) = default;

	FHistoryData& operator=(const FHistoryData&) = default;
	FHistoryData& operator=(FHistoryData&&) = default;

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** History Description */
	FText HistoryDesc;

	/** The base set of filters on the asset view which includes selected paths and collections */
	FAssetViewContentSources ContentSources;

	UE_DEPRECATED(5.6, "Use ContentSources instead.")
	FSourcesData SourcesData;

	/** The selection data from before the sources changed */
	FSelectionData SelectionData;
};

/** The delegate for when history data should be applied */
DECLARE_DELEGATE_OneParam(FOnApplyHistoryData, const FHistoryData& /*Data*/);

/** The delegate for when history data should be updated */
DECLARE_DELEGATE_OneParam(FOnUpdateHistoryData, FHistoryData& /*Data*/);

/** The class responsible for managing all content browser history */
class FHistoryManager
{
public:
	/** Constructor */
	FHistoryManager();

	/** Set the delegate for applying history data */
	void SetOnApplyHistoryData(const FOnApplyHistoryData& InOnApplyHistoryData);

	/** Set the delegate for updating history data */
	void SetOnUpdateHistoryData(const FOnUpdateHistoryData& InOnUpdateHistoryData);

	/** Goes back one history snapshot and returns the history data at that snapshot */
	bool GoBack();

	/** Goes forward one history snapshot and returns the history data at that snapshot */
	bool GoForward();

	/** Stores new history data.  Called when creating a history snapshot */
	void AddHistoryData();

	/** Triggers an update for the current history data. This is typically done right before changing the history. */
	void UpdateHistoryData();

	/** Determines if a user can go forward in history */
	bool CanGoForward() const;

	/** Determines if a user can go back in history */
	bool CanGoBack() const;

	/** Gets the description of the previous history entry */
	FText GetBackDesc() const;

	/** Gets the description of the next history entry */
	FText GetForwardDesc() const;

	/**
	 * Populates a list of menu items that can be added to a context menu
	 * to allow a user to jump to different history snapshots instead of using the back and forward buttons
	 *
	 * @param bGetPrior If true gets history snapshots prior to the current history index(for navigating back). If false get history snapshots after the current history index (for navigating forward).
	 * @param MenuBuilder The menubuilder to populate with menu items
	 */
	void GetAvailableHistoryMenuItems(bool bGetPrior, FMenuBuilder& MenuBuilder);

	/** Rewrite all history data as determined by the passed in predicate */
	template <class PREDICATE_CLASS>
	void RewriteHistoryData(const PREDICATE_CLASS& Predicate)
	{
		for (FHistoryData& HistoryDataEntry : HistoryData)
		{
			Predicate(HistoryDataEntry);
		}
	}

	/** Removes all history data as determined by the passed in predicate */
	template <class PREDICATE_CLASS>
	void RemoveHistoryData(const PREDICATE_CLASS& Predicate)
	{
		for (int32 HistoryIndex = 0; HistoryIndex < HistoryData.Num(); HistoryIndex++)
		{
			if (Predicate(HistoryData[HistoryIndex]))
			{
				HistoryData.RemoveAt(HistoryIndex);
				if (CurrentHistoryIndex >= HistoryIndex)
				{
					// Ensure if possible that current history index continues to point to the same item if something is removed before it.
					CurrentHistoryIndex = FMath::Max(0, CurrentHistoryIndex - 1);
				}
				HistoryIndex--;
			}
		}
	}

private:
	/** Notifies the owner to update to the state described by the current history data */
	void ApplyCurrentHistoryData();

	/** Notifies the owner to update the current history data */
	void UpdateCurrentHistoryData();

	/** Handler for when a history item is chosen in the GetAvailableHistroyMenuItems list */
	void ExecuteJumpToHistory(int32 HistoryIndex);

private:
	/** The delegate for when history data should be applied */
	FOnApplyHistoryData OnApplyHistoryData;

	/** The delegate for when history data should be updated */
	FOnUpdateHistoryData OnUpdateHistoryData;

	/** A list of history snapshots */
	TArray<FHistoryData> HistoryData;

	/** The current history index the user is at (changes when the user goes back,forward, or history snapshots are taken) */
	int32 CurrentHistoryIndex;

	/** Max number of history items that can be stored.  Once the max is reached, the oldest history item is removed */
	int32 MaxHistoryEntries;
};
