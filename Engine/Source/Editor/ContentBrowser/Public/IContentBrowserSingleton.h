// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Misc/Attribute.h"
#include "AssetRegistry/AssetData.h"
#include "AssetTypeCategories.h"
#include "AssetRegistry/ARFilter.h"
#include "CollectionManagerModule.h"
#include "CollectionManagerTypes.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserDataSubsystem.h"
#include "Filters/CustomTextFilters.h"
#include "ICollectionManager.h"
#include "Misc/FilterCollection.h"
#include "Framework/Views/ITypedTableView.h"
#include "AssetThumbnail.h"
#include "ContentBrowserItemPath.h"
#include "Misc/NamePermissionList.h"
#include "Filters/SBasicFilterBar.h"

class FViewport;
class FFrontendFilter;
class IPlugin;
class SWidget;
class UFactory;
class UToolMenu;

typedef const FContentBrowserItem& FAssetFilterType;
typedef TFilterCollection<FAssetFilterType> FAssetFilterCollectionType;

typedef const TSharedRef<IPlugin>& FPluginFilterType;
typedef TFilterCollection<FPluginFilterType> FPluginFilterCollectionType;

/** The view modes used in SAssetView */
namespace EAssetViewType
{
	enum Type
	{
		List,
		Tile,
		Column,
		Custom,

		MAX
	};
}

namespace UE::Editor::ContentBrowser
{
	class IContentSource;
}


/** A selection of items in the Content Browser */
struct FContentBrowserSelection
{
	TArray<FContentBrowserItem> SelectedItems;

	// Legacy data - if set will take precedence over SelectedItems
	TArray<FAssetData> SelectedAssets;
	TArray<FString> SelectedFolders;

	bool IsLegacy() const
	{
		return SelectedAssets.Num() > 0 || SelectedFolders.Num() > 0;
	}

	int32 Num() const
	{
		return SelectedItems.Num() + SelectedAssets.Num() + SelectedFolders.Num();
	}

	void Reset()
	{
		SelectedItems.Reset();
		SelectedAssets.Reset();
		SelectedFolders.Reset();
	}

	void Empty()
	{
		SelectedItems.Reset();
		SelectedAssets.Empty();
		SelectedFolders.Empty();
	}
};


/** A struct containing details about how the content browser should behave */
struct FContentBrowserConfig
{
	/** The contents of the label on the thumbnail */
	EThumbnailLabel::Type ThumbnailLabel;

	/** The default scale for thumbnails. [0-1] range */
	TAttribute< float > ThumbnailScale;

	/** The default view mode */
	EAssetViewType::Type InitialAssetViewType;

	/** Collection to view initially */
	FCollectionRef SelectedCollection;

	UE_DEPRECATED(5.6, "Use SelectedCollection instead.")
	FCollectionNameType SelectedCollectionName;

	/** If true, show the bottom toolbar which shows # of assets selected, view mode buttons, etc... */
	bool bShowBottomToolbar : 1;

	/** Indicates if this view is allowed to show classes */
	bool bCanShowClasses : 1;

	/** Whether the sources view for choosing folders/collections is available or not */
	bool bUseSourcesView : 1;

	/** Whether the sources view should initially be expanded or not */
	bool bExpandSourcesView : 1;

	/** Whether the path picker is available or not */
	bool bUsePathPicker : 1;

	/** Whether to show filters */
	bool bCanShowFilters : 1;

	/** Whether to show asset search */
	bool bCanShowAssetSearch : 1;

	/** Indicates if the 'Show folders' option should be enabled or disabled */
	bool bCanShowFolders : 1;

	/** Indicates if the 'Real-Time Thumbnails' option should be enabled or disabled */
	bool bCanShowRealTimeThumbnails : 1;

	/** Indicates if the 'Show Developers' option should be enabled or disabled */
	bool bCanShowDevelopersFolder : 1;

	/** Whether the 'lock' button is visible on the toolbar */
	bool bCanShowLockButton : 1;

	/** Whether or not this Content Browser can be used as the Primary Browser for SyncBrowserTo functions */
	bool bCanSetAsPrimaryBrowser : 1;

	FContentBrowserConfig()
		: ThumbnailLabel( EThumbnailLabel::ClassName )
		, ThumbnailScale(0.1f)
		, InitialAssetViewType(EAssetViewType::Tile)
		, SelectedCollection(nullptr, NAME_None, ECollectionShareType::CST_Local)
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, SelectedCollectionName( NAME_None, ECollectionShareType::CST_Local )
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
		, bShowBottomToolbar(true)
		, bCanShowClasses(true)
		, bUseSourcesView(true)
		, bExpandSourcesView(true)
		, bUsePathPicker(true)
		, bCanShowFilters(true)
		, bCanShowAssetSearch(true)
		, bCanShowFolders(true)
		, bCanShowRealTimeThumbnails(true)
		, bCanShowDevelopersFolder(true)
		, bCanShowLockButton(true)
		, bCanSetAsPrimaryBrowser(true)
	{ }

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FContentBrowserConfig& operator=(const FContentBrowserConfig&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};


/** A struct containing details about how the asset picker should behave */
struct FAssetPickerConfig
{
	/** The selection mode the picker should use */
	ESelectionMode::Type SelectionMode;

	/** An array of pointers to existing delegates which the AssetView will register a function which returns the current selection */
	TArray<FGetCurrentSelectionDelegate*> GetCurrentSelectionDelegates;

	/** An array of pointers to existing delegates which the AssetView will register a function which sync the asset list*/
	TArray<FSyncToAssetsDelegate*> SyncToAssetsDelegates;

	/** A pointer to an existing delegate that, when executed, will set the filter an the asset picker after it is created. */
	TArray<FSetARFilterDelegate*> SetFilterDelegates;

	/** A pointer to an existing delegate that, when executed, will refresh the asset view. */
	TArray<FRefreshAssetViewDelegate*> RefreshAssetViewDelegates;

	/** A pointer to an existing delegate that, when executed, will save the asset view settings. */
	TArray<FSaveAssetViewSettingsDelegate*> SaveAssetViewSettingsDelegates;

	/** A pointer to an existing delegate that, when executed, will load/restore the asset view settings. */
	TArray<FLoadAssetViewSettingsDelegate*> LoadAssetViewSettingsDelegates;

	/** The asset registry filter to use to cull results */
	FARFilter Filter;

	/** Custom front end filters to be displayed */
	TArray< TSharedRef<class FFrontendFilter> > ExtraFrontendFilters;

	/** The names of columns to hide by default in the column view */
	TArray<FString> HiddenColumnNames;

	/** The names of columns to hide by default in the list view, by default the Type column is hidden */
	TArray<FString> ListHiddenColumnNames;

	/** List of custom columns that fill out data with a callback */
	TArray<FAssetViewCustomColumn> CustomColumns;

	/** The contents of the label on the thumbnail */
	EThumbnailLabel::Type ThumbnailLabel;

	/** The default scale for thumbnails. [0-1] range */
	TAttribute< float > ThumbnailScale;

	/** Initial thumbnail size */
	EThumbnailSize InitialThumbnailSize;

	/** Only display results in these collections */
	TArray<FCollectionRef> CollectionsFilter;

	UE_DEPRECATED(5.6, "Use CollectionsFilter instead.")
	TArray<FCollectionNameType> Collections;

	/** The asset that should be initially selected */
	FAssetData InitialAssetSelection;

	/** The handle to the property that opened this picker. Needed for contextual filtering. */
	TSharedPtr<class IPropertyHandle> PropertyHandle;

	/** The passed in property handle will be used to gather referencing assets. If additional referencing assets should be reported, supply them here. */
	TArray<FAssetData> AdditionalReferencingAssets;

	/** The delegate that fires when an asset was selected */
	FOnAssetSelected OnAssetSelected;

	/** The delegate that fires when a folder was double clicked */
	FOnPathSelected OnFolderEntered;

	/** The delegate that fires when an asset is double clicked */
	FOnAssetDoubleClicked OnAssetDoubleClicked;

	/** The delegate that fires when an asset has enter pressed while selected */
	FOnAssetEnterPressed OnAssetEnterPressed;

	/** The delegate that fires when any number of assets are activated */
	FOnAssetsActivated OnAssetsActivated;

	/** The delegate that fires when any number of assets are dragged */
	FOnAssetsDragged OnAssetsDragged;

	/** Fires when the user changes which columns are hidden in the column or list view. Useful for syncing column state. */
	FOnColumnVisibilityChanged OnColumnVisibilityChanged;

	/** The delegate that fires when an asset is right clicked and a context menu is requested */
	FOnGetAssetContextMenu OnGetAssetContextMenu;

	/** The delegate that fires when a folder is right clicked and a context menu is requested */
	FOnGetFolderContextMenu OnGetFolderContextMenu;

	/** Called to see if it is valid to get a custom asset tool tip */
	FOnIsAssetValidForCustomToolTip OnIsAssetValidForCustomToolTip;

	/** Fired when an asset item is constructed and a tooltip is requested. If unbound the item will use the default widget */
	FOnGetCustomAssetToolTip OnGetCustomAssetToolTip;

	/** Called to add extra asset data to the asset view, to display virtual assets. These get treated similar to Class assets */
	FOnGetCustomSourceAssets OnGetCustomSourceAssets;

	/** Fired when an asset item is about to show its tool tip */
	FOnVisualizeAssetToolTip OnVisualizeAssetToolTip;

	/** Fired when an asset item's tooltip is closing */
	FOnAssetToolTipClosing OnAssetToolTipClosing;

	/** If more detailed filtering is required than simply Filter, this delegate will get fired for every asset to determine if it should be culled. */
	FOnShouldFilterAsset OnShouldFilterAsset;

	/** This delegate will be called in Details view when a new asset registry searchable tag is encountered, to
	    determine if it should be displayed or not.  If it returns true or isn't bound, the tag will be displayed normally. */
	FOnShouldDisplayAssetTag OnAssetTagWantsToBeDisplayed;

	/** Fired when the search box text changes. Allows embedding widgets to observe the search text without replacing the built-in search behavior. */
	FOnSearchTextChanged OnSearchTextChanged;

	/** The default view mode */
	EAssetViewType::Type InitialAssetViewType;

	/** The warning text to show as heading (e.g. when there are no assets to show) */
	TAttribute<FText> AssetShowWarningText;

	/** The justification to use for the AssetShowWarning text (see AssetShowWarningText) */
	TAttribute<ETextJustify::Type> AssetShowWarningTextJustification;
	
	/** Whether the warning text is selectable or not (see AssetShowWarningText) */
	TAttribute<bool> AssetShowWarningTextSelectable;

	/** If set, view settings will be saved and loaded for the asset view using this name in ini files */
	FString SaveSettingsName;

	/** If set, will use this name for filter bar saving purposes instead of the SaveSettingsName. */
	TOptional<FString> FilterBarSaveSettingsNameOverride;
	
	/** If true, the search box will gain focus when the asset picker is created */
	bool bFocusSearchBoxWhenOpened;

	/** If true, a "None" item will always appear at the top of the list */
	bool bAllowNullSelection;

	/** If true, show the bottom toolbar which shows # of assets selected, view mode buttons, etc... */
	bool bShowBottomToolbar;

	/** If false, auto-hide the search bar above */
	bool bAutohideSearchBar;

	/** Whether to allow dragging of items */
	bool bAllowDragging;

	/** Whether to allow renaming of items */
	bool bAllowRename;

	/** Indicates if this view is allowed to show classes */
	bool bCanShowClasses;

	/** Indicates if the 'Show folders' option should be enabled or disabled */
	bool bCanShowFolders;

	/** If true, will allow read-only folders to be shown in the picker */
	bool bCanShowReadOnlyFolders;

	/** Indicates if the 'Real-Time Thumbnails' option should be enabled or disabled */
	bool bCanShowRealTimeThumbnails;

	/** Indicates if the 'Show Developers' option should be enabled or disabled */
	bool bCanShowDevelopersFolder;

	/** Indicates if engine content should always be shown */
	bool bForceShowEngineContent;

	/** Indicates if plugin content should always be shown */
	bool bForceShowPluginContent;

	/** Indicates that we would like to build the filter UI with the Asset Picker */
	bool bAddFilterUI;

	/** If true, show path in column view */
	bool bShowPathInColumnView; 
	/** If true, show class in column view */
	bool bShowTypeInColumnView;
	/** If true, sort by path in column view. Only works if initial view type is Column */
	bool bSortByPathInColumnView;

	/** Can be used to freely change the Add Filter menu. This is executed on the dynamically instanced menu, not the registered menu. */
	TDelegate<void(UToolMenu*)> OnExtendAddFilterMenu;

	/**
	 * Can be used to extend the asset picker top bar via slot manipulation. Current default slots are as follows:
	 * 
	 * 0. Add Filter Menu
	 * 1. SearchBar
	 * 2. Show Developer Content Checkbox
	 * 3. Asset View Options
	 */
	TDelegate<void(TSharedRef<SHorizontalBox>)> OnExtendAssetPickerTopBar;
	
	/** Override the default filter context menu layout */
	EAssetTypeCategories::Type DefaultFilterMenuExpansion;

	/** If we display filters & set to true, we will add sections instead of sub-menus for other filters. Useful if the number of additional filters is small. */
	bool bUseSectionsForCustomFilterCategories;

	/** If specified, the asset view options menu will use this tool menu profile for customization purposes. */
	TOptional<FName> AssetViewOptionsProfile;

	FAssetPickerConfig()
		: SelectionMode( ESelectionMode::Multi )
		, ListHiddenColumnNames( { TEXT("Class") } )
		, ThumbnailLabel( EThumbnailLabel::ClassName )
		, ThumbnailScale(0.1f)
		, InitialThumbnailSize(EThumbnailSize::Medium)
		, InitialAssetViewType(EAssetViewType::Tile)
		, bFocusSearchBoxWhenOpened(true)
		, bAllowNullSelection(false)
		, bShowBottomToolbar(true)
		, bAutohideSearchBar(false)
		, bAllowDragging(true)
		, bAllowRename(true)
		, bCanShowClasses(true)
		, bCanShowFolders(false)
		, bCanShowReadOnlyFolders(true)
		, bCanShowRealTimeThumbnails(false)
		, bCanShowDevelopersFolder(true)
		, bForceShowEngineContent(false)
		, bForceShowPluginContent(false)
		, bAddFilterUI(false)
		, bShowPathInColumnView(false)
		, bShowTypeInColumnView(true)
		, bSortByPathInColumnView(false)
		, DefaultFilterMenuExpansion(EAssetTypeCategories::Basic)
		, bUseSectionsForCustomFilterCategories(false)
	{
		if (IsNewStyleEnabled())
		{
			// Default size in the new style AssetPicker will be Large
			InitialThumbnailSize = EThumbnailSize::Large;
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	FAssetPickerConfig(const FAssetPickerConfig&) = default;
	FAssetPickerConfig(FAssetPickerConfig&&) = default;

	FAssetPickerConfig& operator=(const FAssetPickerConfig&) = default;
	FAssetPickerConfig& operator=(FAssetPickerConfig&&) = default;

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:
	static bool IsNewStyleEnabled()
	{
		static bool bIsNewStyleEnabled = [&]()
		{
			if (const IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("ContentBrowser.EnableNewStyle")))
			{
				ensureAlwaysMsgf(!EnumHasAnyFlags(CVar->GetFlags(), ECVF_Default), TEXT("The CVar should have already been set from commandline, @see: UnrealEdGlobals.cpp, UE::Editor::ContentBrowser::EnableContentBrowserNewStyleCVarRegistration."));
				return CVar->GetBool();
			}
			return false;
		}();

		return bIsNewStyleEnabled;
	}
};

/** A struct containing details about how the path picker should behave */
struct FPathPickerConfig
{
	/** The initial path to select. Leave empty to skip initial selection. */
	FString DefaultPath;

	/** Custom Folder permissions to be used to filter folders in this Path Picker. */
	TSharedPtr<FPathPermissionList> CustomFolderPermissionList;

	/** The delegate that fires when a path was selected */
	FOnPathSelected OnPathSelected;

	/** The delegate that fires when a path is right clicked and a context menu is requested */
	FContentBrowserMenuExtender_SelectedPaths OnGetPathContextMenuExtender;

	/** The delegate that fires when a folder is right clicked and a context menu is requested */
	FOnGetFolderContextMenu OnGetFolderContextMenu;

	/** A pointer to an existing delegate that, when executed, will set the paths for the path picker after it is created. */
	TArray<FSetPathPickerPathsDelegate*> SetPathsDelegates;

	/** If true, the search box will gain focus when the path picker is created */
	bool bFocusSearchBoxWhenOpened : 1;

	/** If false, the context menu will not open when an item is right clicked */
	bool bAllowContextMenu : 1;

	/** If true, will allow class folders to be shown in the picker */
	bool bAllowClassesFolder : 1;

	/** If true, will allow read-only folders to be shown in the picker */
	bool bAllowReadOnlyFolders : 1;

	/** If true, will add the path specified in DefaultPath to the tree if it doesn't exist already */
	bool bAddDefaultPath : 1;

	/** If true, passes virtual paths to OnPathSelected instead of internal asset paths */
	bool bOnPathSelectedPassesVirtualPaths : 1;

	/** Whether or not to show the favorites selector. */
	bool bShowFavorites : 1;

	/** Whether to call OnPathSelected during construction for DefaultPath if DefaultPath is allowed */
	bool bNotifyDefaultPathSelected : 1;

	/** Indicates if the 'Show Developers' option should be enabled or disabled */
	bool bCanShowDevelopersFolder;

	/** Indicates if engine content should always be shown */
	bool bForceShowEngineContent;

	/** Indicates if plugin content should always be shown */
	bool bForceShowPluginContent;

	/** if true, will add the filter setting to the path picker */
	bool bShowViewOptions;

	FPathPickerConfig()
		: bFocusSearchBoxWhenOpened(true)
		, bAllowContextMenu(true)
		, bAllowClassesFolder(false)
		, bAllowReadOnlyFolders(true)
		, bAddDefaultPath(false)
		, bOnPathSelectedPassesVirtualPaths(false)
		, bShowFavorites(true)
		, bNotifyDefaultPathSelected(false)
		, bCanShowDevelopersFolder(true)
		, bForceShowEngineContent(false)
		, bForceShowPluginContent(false)
		, bShowViewOptions(false)
	{}
};

/** A struct containing details about how the collection picker should behave */
struct FCollectionPickerConfig
{
	/** If true, collection buttons will be displayed */
	bool AllowCollectionButtons;

	/** If true, users will be able to access the right-click menu of a collection */
	bool AllowRightClickMenu;

	/** The collection container to view */
	TSharedPtr<ICollectionContainer> CollectionContainer;

	/** Called when a collection was selected */
	FOnCollectionSelected OnCollectionSelected;

	FCollectionPickerConfig()
		: AllowCollectionButtons(true)
		, AllowRightClickMenu(true)
		, CollectionContainer(FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer())
		, OnCollectionSelected()
	{}
};

namespace EAssetDialogType
{
	enum Type
	{
		Open,
		Save
	};
}

/** A struct containing shared details about how asset dialogs should behave. You should not instanciate this config, but use FOpenAssetDialogConfig or FSaveAssetDialogConfig instead. */
struct FSharedAssetDialogConfig
{
	FText DialogTitleOverride;
	FString DefaultPath;
	TArray<FTopLevelAssetPath> AssetClassNames;
	FVector2D WindowSizeOverride;
	FOnPathSelected OnPathSelected;
	/** When specified, this window will be used instead of the mainframe window. */
	TSharedPtr<SWindow> WindowOverride;

	virtual EAssetDialogType::Type GetDialogType() const = 0;

	FSharedAssetDialogConfig()
		: WindowSizeOverride(ForceInitToZero)
	{}

	virtual ~FSharedAssetDialogConfig() = default;
};

/** A struct containing details about how the open asset dialog should behave. */
struct FOpenAssetDialogConfig : public FSharedAssetDialogConfig
{
	bool bAllowMultipleSelection;

	virtual EAssetDialogType::Type GetDialogType() const override { return EAssetDialogType::Open; }

	FOpenAssetDialogConfig()
		: FSharedAssetDialogConfig()
		, bAllowMultipleSelection(false)
	{}
};

/** An enum to choose the behavior of the save asset dialog when the user chooses an asset that already exists */
namespace ESaveAssetDialogExistingAssetPolicy
{
	enum Type
	{
		/** Display an error and disallow the save */
		Disallow,

		/** Allow the save, but warn that the existing file will be overwritten */
		AllowButWarn
	};
}

/** A struct containing details about how the save asset dialog should behave. */
struct FSaveAssetDialogConfig : public FSharedAssetDialogConfig
{
	FString DefaultAssetName;
	ESaveAssetDialogExistingAssetPolicy::Type ExistingAssetPolicy;

	virtual EAssetDialogType::Type GetDialogType() const override { return EAssetDialogType::Save; }

	FSaveAssetDialogConfig()
		: FSharedAssetDialogConfig()
		, ExistingAssetPolicy(ESaveAssetDialogExistingAssetPolicy::Disallow)
	{}
};

/**
 * Content browser module singleton
 */
class IContentBrowserSingleton
{
public:
	CONTENTBROWSER_API static IContentBrowserSingleton& Get();
	
	/** Virtual destructor */
	virtual ~IContentBrowserSingleton() = default;

	/**
	 * Generates a content browser.  Generally you should not call this function, but instead use CreateAssetPicker().
	 *
	 * @param	InstanceName			Global name of this content browser instance
	 * @param	ContainingTab			The tab the browser is contained within or nullptr
	 * @param	ContentBrowserConfig	Initial defaults for the new content browser, or nullptr to use saved settings
	 *
	 * @return The newly created content browser widget
	 */
	virtual TSharedRef<SWidget> CreateContentBrowser( const FName InstanceName, TSharedPtr<SDockTab> ContainingTab, const FContentBrowserConfig* ContentBrowserConfig ) = 0;

	/**
	 * Generates an asset picker widget locked to the specified FARFilter.
	 *
	 * @param AssetPickerConfig		A struct containing details about how the asset picker should behave				
	 * @return The asset picker widget
	 */
	virtual TSharedRef<SWidget> CreateAssetPicker(const FAssetPickerConfig& AssetPickerConfig) = 0;

	/** Focus the search box of the given asset picker widget. */
	virtual TSharedPtr<SWidget> GetAssetPickerSearchBox(const TSharedRef<SWidget>& AssetPickerWidget) = 0;

	/**
	 * Generates a path picker widget.
	 *
	 * @param PathPickerConfig		A struct containing details about how the path picker should behave				
	 * @return The path picker widget
	 */
	virtual TSharedRef<SWidget> CreatePathPicker(const FPathPickerConfig& PathPickerConfig) = 0;

	/**
	 * Generates a collection picker widget.
	 *
	 * @param CollectionPickerConfig		A struct containing details about how the collection picker should behave				
	 * @return The collection picker widget
	 */
	virtual TSharedRef<SWidget> CreateCollectionPicker(const FCollectionPickerConfig& CollectionPickerConfig) = 0;

	/**
	 * Generates a content browser for use in a drawer. This content browser is a singleton and is reused among all drawers.
	 *
	 * @param ContentBrowserConfig	Initial defaults for the new content browser
	 *
	 * @return The content browser drawer widget
	 */
	virtual TSharedRef<SWidget> CreateContentBrowserDrawer(const FContentBrowserConfig& ContentBrowserConfig, TFunction<TSharedPtr<SDockTab>()> InOnGetTabForDrawer) = 0;

	/**
	 * Opens the Open Asset dialog in a non-modal window
	 *
	 * @param OpenAssetConfig				A struct containing details about how the open asset dialog should behave
	 * @param OnAssetsChosenForOpen			A delegate that is fired when assets are chosen and the open button is pressed
	 * @param OnAssetDialogCancelled		A delegate that is fired when the asset dialog is closed or cancelled
	 */
	virtual void CreateOpenAssetDialog(const FOpenAssetDialogConfig& OpenAssetConfig, const FOnAssetsChosenForOpen& OnAssetsChosenForOpen, const FOnAssetDialogCancelled& OnAssetDialogCancelled) = 0;

	/**
	 * Opens the Open Asset dialog in a modal window
	 *
	 * @param OpenAssetConfig				A struct containing details about how the open asset dialog should behave
	 * @return The assets that were chosen to be opened
	 */
	virtual TArray<FAssetData> CreateModalOpenAssetDialog(const FOpenAssetDialogConfig& InConfig) = 0;

	/**
	 * Opens the Save Asset dialog in a non-modal window
	 *
	 * @param SaveAssetConfig				A struct containing details about how the save asset dialog should behave
	 * @param OnAssetNameChosenForSave		A delegate that is fired when an object path is chosen and the save button is pressed
	 * @param OnAssetDialogCancelled		A delegate that is fired when the asset dialog is closed or cancelled
	 */
	virtual void CreateSaveAssetDialog(const FSaveAssetDialogConfig& SaveAssetConfig, const FOnObjectPathChosenForSave& OnAssetNameChosenForSave, const FOnAssetDialogCancelled& OnAssetDialogCancelled) = 0;

	/**
	 * Opens the Save Asset dialog in a modal window
	 *
	 * @param SaveAssetConfig				A struct containing details about how the save asset dialog should behave
	 * @return The object path that was chosen
	 */
	virtual FString CreateModalSaveAssetDialog(const FSaveAssetDialogConfig& SaveAssetConfig) = 0;

	/** Returns true if there is at least one browser open that is eligible to be a primary content browser */
	virtual bool HasPrimaryContentBrowser() const = 0;

	/** Sets the primary content browser for subsequent state changes through this singleton
	 * Returns true if content browser was changed sucessfully
	 */
	virtual bool SetPrimaryContentBrowser(FName InstanceName) = 0;

	/** Brings the primary content browser to the front or opens one if it does not exist. */
	virtual void FocusPrimaryContentBrowser(bool bFocusSearch) = 0;

	/** Focuses the search field of a content browser widget */
	virtual void FocusContentBrowserSearchField(TSharedPtr<SWidget> ContentBrowserWidget) = 0;

	/** Sets up an inline-name for the creation of a new asset in the primary content browser using the specified path and the specified class and/or factory */
	virtual void CreateNewAsset(const FString& DefaultAssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory) = 0;

	/** 
	 * Selects the supplied assets in the primary content browser. 
     *
	 * @param AssetList                     An array of AssetDataList structs to sync.
	 * @param bAllowLockedBrowsers 	        When true, even locked browsers may handle the sync. Only set to true if the sync doesn't seem external to the content browser.
	 * @param bFocusContentBrowser          When true, brings the ContentBrowser into the foreground.
	 * @param InstanceName					When supplied, will only sync the Content Browser with the matching InstanceName.  bAllowLockedBrowsers is ignored.
	 */ 
	virtual void SyncBrowserToAssets(const TArray<struct FAssetData>& AssetDataList, bool bAllowLockedBrowsers = false, bool bFocusContentBrowser = true, const FName& InstanceName = FName(), bool bNewSpawnBrowser = false) = 0;

	/** 
	 * Selects the supplied assets in the primary content browser. 
     *
	 * @param AssetList                     An array of UObject pointers to sync.
	 * @param bAllowLockedBrowsers 	        When true, even locked browsers may handle the sync. Only set to true if the sync doesn't seem external to the content browser.
	 * @param bFocusContentBrowser          When true, brings the ContentBrowser into the foreground.
	 * @param InstanceName					When supplied, will only sync the Content Browser with the matching InstanceName.  bAllowLockedBrowsers is ignored.
	 */ 
	virtual void SyncBrowserToAssets(const TArray<UObject*>& AssetList, bool bAllowLockedBrowsers = false, bool bFocusContentBrowser = true, const FName& InstanceName = FName(), bool bNewSpawnBrowser = false) = 0;

	/** 
	 * Selects the supplied assets in the primary content browser. 
     *
	 * @param AssetList                     An array of strings representing folder asset paths to sync.
	 * @param bAllowLockedBrowsers 	        When true, even locked browsers may handle the sync. Only set to true if the sync doesn't seem external to the content browser.
	 * @param bFocusContentBrowser          When true, brings the ContentBrowser into the foreground.
	 * @param InstanceName					When supplied, will only sync the Content Browser with the matching InstanceName.  bAllowLockedBrowsers is ignored.
	 * @param bNewSpawnBrowser				When supplied, will spawn a new Content Browser instead of selecting the assets in an existing one.
	 */ 
	virtual void SyncBrowserToFolders(const TArray<FString>& FolderList, bool bAllowLockedBrowsers = false, bool bFocusContentBrowser = true, const FName& InstanceName = FName(), bool bNewSpawnBrowser = false) = 0;

	/**
	 * Selects the supplied collections in the primary content browser.
	 *
	 * @param CollectionsToSync				An array of collections to sync.
	 * @param bAllowLockedBrowsers 	        When true, even locked browsers may handle the sync. Only set to true if the sync doesn't seem external to the content browser.
	 * @param bFocusContentBrowser          When true, brings the ContentBrowser into the foreground.
	 * @param InstanceName					When supplied, will only sync the Content Browser with the matching InstanceName.  bAllowLockedBrowsers is ignored.
	 * @param bNewSpawnBrowser				When supplied, will spawn a new Content Browser instead of selecting the assets in an existing one.
	 */
	virtual void SyncBrowserToCollections(const TArray<FCollectionRef>& CollectionsToSync, bool bAllowLockedBrowsers = false, bool bFocusContentBrowser = true, const FName& InstanceName = FName(), bool bNewSpawnBrowser = false) = 0;

	/**
	 * Selects the supplied items in the primary content browser.
	 *
	 * @param ItemsToSync                   An array of items to sync.
	 * @param bAllowLockedBrowsers 	        When true, even locked browsers may handle the sync. Only set to true if the sync doesn't seem external to the content browser.
	 * @param bFocusContentBrowser          When true, brings the ContentBrowser into the foreground.
	 * @param InstanceName					When supplied, will only sync the Content Browser with the matching InstanceName.  bAllowLockedBrowsers is ignored.
	 * @param bNewSpawnBrowser				When supplied, will spawn a new Content Browser instead of selecting the assets in an existing one.
	 */
	virtual void SyncBrowserToItems(const TArray<FContentBrowserItem>& ItemsToSync, bool bAllowLockedBrowsers = false, bool bFocusContentBrowser = true, const FName& InstanceName = FName(), bool bNewSpawnBrowser = false) = 0;

	/** 
	 * Selects the supplied assets in the primary content browser. 
     *
	 * @param ItemSelection 				A struct containing AssetData and Folders to sync.
	 * @param bAllowLockedBrowsers 	        When true, even locked browsers may handle the sync. Only set to true if the sync doesn't seem external to the content browser.
	 * @param bFocusContentBrowser          When true, brings the ContentBrowser into the foreground.
	 * @param InstanceName					When supplied, will only sync the Content Browser with the matching InstanceName.  bAllowLockedBrowsers is ignored.
	 * @param bNewSpawnBrowser				When supplied, will spawn a new Content Browser instead of selecting the assets in an existing one.
	 */ 
	virtual void SyncBrowserTo(const FContentBrowserSelection& ItemSelection, bool bAllowLockedBrowsers = false, bool bFocusContentBrowser = true, const FName& InstanceName = FName(), bool bNewSpawnBrowser = false) = 0;

	/** Generates a list of assets that are selected in the primary content browser */
	virtual void GetSelectedAssets(TArray<FAssetData>& SelectedAssets) = 0;

	/** Generates a list of assets that are selected across all open content browsers */
	virtual void GetAllSelectedAssets(TArray<FAssetData>& SelectedAssets) = 0;

	/** Generates a list of folders that are selected in the primary content browser */
	virtual void GetSelectedFolders(TArray<FString>& SelectedFolders) = 0;

	/** Returns the folders that are selected in the path view */
	virtual void GetSelectedPathViewFolders(TArray<FString>& SelectedFolders) = 0;

	/** Gets the current path if one exists, otherwise returns empty string. */
	UE_DEPRECATED(5.0, "This function is deprecated. Use GetCurrentPath without argument instead.")
	virtual FString GetCurrentPath(const EContentBrowserPathType PathType) = 0;

	/** Gets the current path if one exists, otherwise returns empty string. */
	virtual FContentBrowserItemPath GetCurrentPath() = 0;

	/**
	 * Capture active viewport to thumbnail and assigns that thumbnail to incoming assets
	 *
	 * @param InViewport - viewport to sample from
	 * @param InAssetsToAssign - assets that should receive the new thumbnail ONLY if they are assets that use GenericThumbnails
	 */
	virtual void CaptureThumbnailFromViewport(FViewport* InViewport, TArray<FAssetData>& SelectedAssets) = 0;

	/**
	 * Sets the content browser to display the selected paths
	 */
	virtual void SetSelectedPaths(const TArray<FString>& FolderPaths, bool bNeedsRefresh = false, bool bPathsAreVirtual = false) = 0;

	/**
	 * Forces the content browser to show plugin content if it's not already showing.
	 *
	 * @param bEnginePlugin	If this is true, it will also force the content browser to show engine content
	 */
	virtual void ForceShowPluginContent(bool bEnginePlugin) = 0;

	/**
	 * Saves the settings for a particular content browser instance
	 *
	 * @param ContentBrowserWidget The content browser widget to save
	 */
	virtual void SaveContentBrowserSettings(TSharedPtr<SWidget> ContentBrowserWidget) = 0;

	/**
	 * Rename current first selected content item on the passed in widget.
	 *
	 * @param PickerWidget The picker widget whose asset we want to rename, should be a asset or path picker widget.
	 */
	virtual void ExecuteRename(TSharedPtr<SWidget> PickerWidget) = 0;

	/**
	 * Add a folder to the path picker widget under the current selected path.
	 *
	 * @param PathPickerWidget The path picker widget where we want to add an folder
	 */
	virtual void ExecuteAddFolder(TSharedPtr<SWidget> PathPickerWidget) = 0;

	/**
	 * Force refresh on the path picker widget.  You may need to do this if you have changed the filter on the path picker.
	 *
	 * @param PathPickerWidget The path picker widget where we want to add an folder
	*/
	virtual void RefreshPathView(TSharedPtr<SWidget> PathPickerWidget) = 0;

	/** Returns InPath if can be written to, otherwise picks a default path that can be written to */
	virtual FContentBrowserItemPath GetInitialPathToSaveAsset(const FContentBrowserItemPath& InPath) = 0;

	/** Returns true if FolderPath is a private content edit folder */
	virtual bool IsShowingPrivateContent(const FStringView VirtualFolderPath) = 0;

	/** Returns true if FolderPath's private content edit mode is allowed to be toggled */
	virtual bool IsFolderShowPrivateContentToggleable(const FStringView VirtualFolderPath) = 0;

	/** Returns the Private Content Permission List */
	virtual const TSharedPtr<FPathPermissionList>& GetShowPrivateContentPermissionList() = 0;

	/** Returns the Internal Content Permission List */
	virtual const TSharedPtr<FPathPermissionList>& GetShowEpicInternalContentPermissionList() = 0;

	/** Declares the Private Content Permission List dirty */
	virtual void SetPrivateContentPermissionListDirty() = 0;

	/** Registers the delegate for custom handling of if a Folder allows private content edits */
	virtual void RegisterIsFolderShowPrivateContentToggleableDelegate(FIsFolderShowPrivateContentToggleableDelegate InIsFolderShowPrivateContentToggleableDelegate) = 0;

	/** Unregisters the delegate for custom handling of if a Folder allows private content edits */
	virtual void UnregisterIsFolderShowPrivateContentToggleableDelegate() = 0;

	/** Register a delegate to be called when the Favorites changes. */
	virtual FDelegateHandle RegisterOnFavoritesChangedHandler(FOnFavoritesChanged::FDelegate InOnFavoritesChanged) = 0;

	/** Unregister a previously-registered handler for when Favorites changes. */
	virtual void UnregisterOnFavoritesChangedDelegate(FDelegateHandle Handle) = 0;
	
	/** Triggers a refresh of favorite path views in content browser */
	UE_EXPERIMENTAL(5.8, "Experimental way to refresh the content browser favorites, API may change in the future")
	virtual void RefreshFavoritesPathView() = 0;

	/**
	 * Get a list of other paths that the data source may be using to represent a specific path
	 *
	 * @param The internal path (or object path) of an asset to get aliases for
	 * @return All alternative paths that represent the input path (not including the input path itself)
	 */
	virtual TArray<FString> GetAliasesForPath(const FSoftObjectPath& InPath) const = 0;

	/** Adds a Custom Text Filter to the Filter List */
	virtual void AddCustomTextFilter(const FCustomTextFilterData& FilterData, bool bApplyFilter) = 0;

	/** Deletes the specified Custom Text Filter */
	virtual void DeleteCustomTextFilterByLabel(const FText& FilterLabel) = 0;

	/** Modifies the specified Custom Text Filter */
	virtual void ModifyCustomTextFilterByLabel(const FCustomTextFilterData& FilterData, const FText& FilterLabel) = 0;

	/** Gets whether the AssetView in the Content Browser is done filtering */
	virtual bool IsAssetViewDoneFiltering() = 0;

	/** Sets the AssetSearchBox text */
	virtual void SetSearchText(const FText& NewText) = 0;

	/** Toggle the Favorite status of the given folder list */
	virtual void ToggleFolderFavorite(const TArray<FString>& FolderPaths) = 0;

	/** Iterate through each favorite folder and call the delegate */
	virtual void ForEachFavoriteFolder(TFunctionRef<void(const FContentBrowserItemPath&)> InDelegate) = 0;

	/** Sets the Content Browser filter layout to either vertical or horizontal */
	virtual void SetFilterLayout(EFilterBarLayout InFilterBarLayout) = 0;

	/** Gets the current Content Browser filter layout */
	virtual EFilterBarLayout GetFilterLayout() = 0;

	/** Gets a pointer to the active filter container widget (Horizontal or Vertical), nullptr if neither is active */
	virtual TSharedPtr<SWidget> GetActiveFilterContainer() = 0;

	/** Docks the current content browser drawer as a tabbed content browser in a layout */
	virtual void DockContentBrowserDrawer() = 0;

	/** Factory delegate that instantiates a content source  */
	DECLARE_DELEGATE_RetVal(TSharedRef<UE::Editor::ContentBrowser::IContentSource>, FContentSourceFactory)

	/** Register a content source by name, causing it to appear in the content sources bar as selectable (if the bar is enabled)  */
	virtual void RegisterContentSourceFactory(const FName& InContentSourceName, const FContentSourceFactory& InContentSourceFactory) = 0;

	/** Unregister a content source by name */
	virtual void UnregisterContentSourceFactory(const FName& InContentSourceName) = 0;

	/** Execute a delegate for each registered content source factory */
	virtual void ForEachContentSourceFactory(const TFunctionRef<void(const FName&, const FContentSourceFactory&)>& InFunction) = 0;

	/** Delegate fired when a new content source factory is registered/unregistered */
	DECLARE_MULTICAST_DELEGATE(FOnContentSourceFactoriesChanged)
	virtual FOnContentSourceFactoriesChanged& OnContentSourceFactoriesChanged() = 0;
	
	/** Register a delegate to be called when the Favorites changes. */
	UE_DEPRECATED(5.7, "Use the overload that takes in an FOnFavoritesChanged delegate")
	virtual FDelegateHandle RegisterOnFavoritesChangedHandler(FSimpleDelegate OnFavoritesChanged) = 0;
};
