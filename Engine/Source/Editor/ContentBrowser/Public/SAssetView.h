// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/CurveSequence.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetThumbnail.h"
#include "AssetViewSortManager.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "ContentBrowserDataFilter.h"
#include "ContentBrowserDataLegacyBridge.h"
#include "ContentBrowserDelegates.h"
#include "Experimental/ContentBrowserSearchOverride.h"
#include "ContentBrowserItem.h"
#include "ContentBrowserTelemetry.h"
#include "AssetViewContentSources.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Experimental/ContentBrowserViewExtender.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/Platform.h"
#include "HistoryManager.h"
#include "IContentBrowserSingleton.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/NamePermissionList.h"
#include "Misc/Optional.h"
#include "SourcesData.h"
#include "Styling/SlateColor.h"
#include "Templates/PimplPtr.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UniquePtr.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#define UE_API CONTENTBROWSER_API

class FAssetContextMenu;
class FAssetTextFilter;
class FAssetThumbnail;
class FAssetViewItem;
class FAssetViewItemCollection;
class FContentBrowserItemDataTemporaryContext;
class FContentBrowserItemDataUpdate;
class FDragDropEvent;
class FFilter_ShowRedirectors;
class FMenuBuilder;
class FPathPermissionList;
class FSlateRect;
class FWeakWidgetPath;
class FWidgetPath;
class ICollectionContainer;
class ITableRow;
class SAssetColumnView;
class SAssetListView;
class SAssetTileView;
class SBox;
class SComboButton;
class SContentBrowser;
class SFilterList;
class STableViewBase;
class SWidget;
class UClass;
class UFactory;
class UToolMenu;
struct FAssetViewInstanceConfig;
struct FCharacterEvent;
struct FCollectionNameType;
struct FContentBrowserInstanceConfig;
struct FFiltersAdditionalParams;
struct FFocusEvent;
struct FGeometry;
struct FKeyEvent;
struct FPointerEvent;
struct FPropertyChangedEvent;
struct FToolMenuContext;

/** Fires whenever the asset view is asked to start to create a temporary item */
DECLARE_DELEGATE_OneParam(FOnAssetViewNewItemRequested, const FContentBrowserItem& /*NewItem*/);

/** Fires whenever one of the "Search" options changes, useful for modifying search criteria to match */
DECLARE_DELEGATE(FOnSearchOptionChanged);

/** Fires whenever asset view options menu is being opened, gives chance for external code to set additional context */
DECLARE_DELEGATE_OneParam(FOnExtendAssetViewOptionsMenuContext, FToolMenuContext&);

/** Copy types */
enum class EAssetViewCopyType
{
	ExportTextPath,
	ObjectPath,
	PackageName
};

template <>
struct TWidgetTypeTraits<class SAssetView>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * A widget to display a list of filtered assets
 */
class SAssetView
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SAssetView )
		: _InitialCategoryFilter(EContentBrowserItemCategoryFilter::IncludeAll)
		, _AssetShowWarningTextJustification(ETextJustify::Center)
		, _AssetShowWarningTextSelectable(false)
		, _ThumbnailLabel( EThumbnailLabel::ClassName )
		, _AllowThumbnailHintLabel(true)
		, _bShowPathViewFilters(false)
		, _InitialViewType(EAssetViewType::Tile)
		, _InitialThumbnailSize(EThumbnailSize::Medium)
		, _ShowBottomToolbar(true)
		, _ShowViewOptions(true)
		, _AllowThumbnailEditMode(false)
		, _CanShowClasses(true)
		, _CanShowFolders(false)
		, _CanShowReadOnlyFolders(true)
		, _FilterRecursivelyWithBackendFilter(true)
		, _CanShowRealTimeThumbnails(false)
		, _CanShowDevelopersFolder(false)
		, _CanShowFavorites(false)
		, _CanDockCollections(false)
		, _SelectionMode( ESelectionMode::Multi )
		, _AllowDragging(true)
		, _AllowFocusOnSync(true)
		, _FillEmptySpaceInTileView(true)
		, _ShowPathInColumnView(false)
		, _ShowTypeInColumnView(true)
		, _ShowAssetAccessSpecifier(false)
		, _SortByPathInColumnView(false)
		, _ShowTypeInTileView(true)
		, _ForceShowEngineContent(false)
		, _ForceShowPluginContent(false)
		, _ForceHideScrollbar(false)
		, _ShowDisallowedAssetClassAsUnsupportedItems(false)
		, _AllowCustomView(false)
		{}

		/** Called to check if an asset should be filtered out by external code */
		SLATE_EVENT( FOnShouldFilterAsset, OnShouldFilterAsset )

		/** Called to check if an item should be filtered out by external code */
		SLATE_EVENT(FOnShouldFilterItem, OnShouldFilterItem)

		/** Called when the asset view is asked to start to create a temporary item */
		SLATE_EVENT( FOnAssetViewNewItemRequested, OnNewItemRequested )

		/** Called to when an item is selected */
		SLATE_EVENT( FOnContentBrowserItemSelectionChanged, OnItemSelectionChanged )

		/** Called when the user double clicks, presses enter, or presses space on an Content Browser item */
		SLATE_EVENT( FOnContentBrowserItemsActivated, OnItemsActivated )

		/** Called when the user double clicks, presses enter, or presses space on an Content Browser item */
		SLATE_EVENT( FOnContentBrowserItemsDragged, OnItemsDragged )

		/** Delegate to invoke when a context menu for an item is opening. */
		SLATE_EVENT( FOnGetContentBrowserItemContextMenu, OnGetItemContextMenu )

		/** Called when the user has committed a rename of one or more items */
		SLATE_EVENT( FOnContentBrowserItemRenameCommitted, OnItemRenameCommitted )

		/** Delegate to call (if bound) to check if it is valid to get a custom tooltip for this asset item */
		SLATE_EVENT(FOnIsAssetValidForCustomToolTip, OnIsAssetValidForCustomToolTip)

		/** Called to get a custom asset item tool tip (if necessary) */
		SLATE_EVENT( FOnGetCustomAssetToolTip, OnGetCustomAssetToolTip )

		/** Called when an asset item is about to show a tooltip */
		SLATE_EVENT( FOnVisualizeAssetToolTip, OnVisualizeAssetToolTip )

		/** Called when an asset item's tooltip is closing */
		SLATE_EVENT(FOnAssetToolTipClosing, OnAssetToolTipClosing)

		/** Called when opening view options menu */
		SLATE_EVENT(FOnExtendAssetViewOptionsMenuContext, OnExtendAssetViewOptionsMenuContext)

		/** Initial set of item categories that this view should show - may be adjusted further by things like CanShowClasses or legacy delegate bindings */
		SLATE_ARGUMENT( EContentBrowserItemCategoryFilter, InitialCategoryFilter )

		/** The warning text to show as heading (e.g. when there are no assets to show) */
		SLATE_ATTRIBUTE( FText, AssetShowWarningText )

		/** The justification to use for the AssetShowWarning text (see AssetShowWarningText) */
		SLATE_ATTRIBUTE( ETextJustify::Type, AssetShowWarningTextJustification )

		/** Whether the warning text is selectable or not (see AssetShowWarningText) */
		SLATE_ATTRIBUTE( bool, AssetShowWarningTextSelectable )

		/** Attribute to determine what text should be highlighted */
		SLATE_ATTRIBUTE( FText, HighlightedText )

		/** What the label on the asset thumbnails should be */
		SLATE_ARGUMENT( EThumbnailLabel::Type, ThumbnailLabel )

		/** Whether to ever show the hint label on thumbnails */
		SLATE_ARGUMENT( bool, AllowThumbnailHintLabel )

		/** The filter collection used to further filter down assets returned from the backend */
		SLATE_ARGUMENT( TSharedPtr<FAssetFilterCollectionType>, FrontendFilters )

		/** Text filter object */
		SLATE_ARGUMENT(TSharedPtr<FAssetTextFilter>, TextFilter)

		/** If true, redirectors are visible even if not explicitly searching for them. */
		SLATE_ATTRIBUTE(bool, ShowRedirectors);

		/** Show path view filters submenu in view options menu */
		SLATE_ARGUMENT( bool, bShowPathViewFilters )

		/** The initial base sources filter */
		SLATE_ARGUMENT( FAssetViewContentSources, InitialContentSources )

		/** The initial base sources filter */
		SLATE_ARGUMENT_DEPRECATED( FSourcesData, InitialSourcesData, 5.6, "Use InitialContentSources instead." )

		/** The initial backend filter */
		SLATE_ARGUMENT( FARFilter, InitialBackendFilter )

		/** The asset that should be initially selected */
		SLATE_ARGUMENT( FAssetData, InitialAssetSelection )

		/** The initial view type */
		SLATE_ARGUMENT( EAssetViewType::Type, InitialViewType )

		/** Initial thumbnail size */
		SLATE_ARGUMENT( EThumbnailSize, InitialThumbnailSize )

		/** Should the toolbar indicating number of selected assets, mode switch buttons, etc... be shown? */
		SLATE_ARGUMENT( bool, ShowBottomToolbar )

		/** Should view options be shown. Note: If ShowBottomToolbar is false, then view options are not shown regardless of this setting */
		SLATE_ARGUMENT( bool, ShowViewOptions)

		/** True if the asset view may edit thumbnails */
		SLATE_ARGUMENT( bool, AllowThumbnailEditMode )

		/** Indicates if this view is allowed to show classes */
		SLATE_ARGUMENT( bool, CanShowClasses )

		/** Indicates if the 'Show Folders' option should be enabled or disabled */
		SLATE_ARGUMENT( bool, CanShowFolders )

		/** Indicates if this view is allowed to show folders that cannot be written to */
		SLATE_ARGUMENT( bool, CanShowReadOnlyFolders )

		/** If true, recursive filtering will be caused by applying a backend filter */
		SLATE_ARGUMENT( bool, FilterRecursivelyWithBackendFilter )

		/** Indicates if the 'Real-Time Thumbnails' option should be enabled or disabled */
		SLATE_ARGUMENT( bool, CanShowRealTimeThumbnails )

		/** Indicates if the 'Show Developers' option should be enabled or disabled */
		SLATE_ARGUMENT( bool, CanShowDevelopersFolder )

		/** Indicates if the 'Show Favorites' option should be enabled or disabled */
		SLATE_ARGUMENT(bool, CanShowFavorites)

		/** Indicates if the 'Dock Collections' option should be enabled or disabled */
		SLATE_ARGUMENT_DEPRECATED(bool, CanDockCollections, 5.6, "Collection docking is deprecated.")

		/** The selection mode the asset view should use */
		SLATE_ARGUMENT( ESelectionMode::Type, SelectionMode )

		/** Whether to allow dragging of items */
		SLATE_ARGUMENT( bool, AllowDragging )

		/** Whether this asset view should allow focus on sync or not */
		SLATE_ARGUMENT( bool, AllowFocusOnSync )

		/** Whether this asset view should allow the thumbnails to consume empty space after the user scale is applied */
		SLATE_ARGUMENT( bool, FillEmptySpaceInTileView )

		/** Should show Path in column view if true */
		SLATE_ARGUMENT(bool, ShowPathInColumnView)

		/** Should show Type in column view if true */
		SLATE_ARGUMENT(bool, ShowTypeInColumnView)

		/** Should show EAssetAccessSpecifier if true */
		SLATE_ARGUMENT(bool, ShowAssetAccessSpecifier)

		/** Sort by path in the column view. Only works if the initial view type is Column */
		SLATE_ARGUMENT(bool, SortByPathInColumnView)
		
		/** Should show Type in tile view if true */
		SLATE_ARGUMENT(bool, ShowTypeInTileView)

		/** Should always show engine content */
		SLATE_ARGUMENT(bool, ForceShowEngineContent)

		/** Should always show plugin content */
		SLATE_ARGUMENT(bool, ForceShowPluginContent)

		/** Should always hide scrollbar (Removes scrollbar) */
		SLATE_ARGUMENT(bool, ForceHideScrollbar)

		/** Allow the asset view to display the hidden asset class as unsupported items */
		SLATE_ARGUMENT(bool, ShowDisallowedAssetClassAsUnsupportedItems)

		/** Allow the asset view to display a custom view registered with the Content Browser module */
		SLATE_ARGUMENT(bool, AllowCustomView)

		/** Called to check if an asset tag should be display in details view. */
		SLATE_EVENT( FOnShouldDisplayAssetTag, OnAssetTagWantsToBeDisplayed )

		/** Called to add extra asset data to the asset view, to display virtual assets. These get treated similar to Class assets */
		SLATE_EVENT( FOnGetCustomSourceAssets, OnGetCustomSourceAssets )

		/** Columns to hide by default for the ColumnView */
		SLATE_ARGUMENT( TArray<FString>, HiddenColumnNames )

		/** Columns to hide by default for the ListView, works only in the new style */
		SLATE_ARGUMENT( TArray<FString>, ListHiddenColumnNames )

		/** Custom columns that can be use specific */
		SLATE_ARGUMENT( TArray<FAssetViewCustomColumn>, CustomColumns )

		/** Custom columns that can be use specific */
		SLATE_EVENT( FOnSearchOptionChanged, OnSearchOptionsChanged)

		/** Called when the user changes column visibility. */
		SLATE_EVENT( FOnColumnVisibilityChanged, OnColumnVisibilityChanged )

		/** The content browser that owns this view if any */
		SLATE_ARGUMENT(TSharedPtr<SContentBrowser>, OwningContentBrowser)

		/** The menu profile to use for the asset view options. The profile needs to be registered with the ToolMenus API. */
		SLATE_ARGUMENT(TOptional<FName>, AssetViewOptionsProfile)
	SLATE_END_ARGS()

	friend FAssetContextMenu;
	friend class UContentBrowserAssetViewContextMenuContext;

	UE_API SAssetView();
	UE_API virtual ~SAssetView() override;

	/** Constructs this widget with InArgs */
	UE_API void Construct( const FArguments& InArgs );

	/** Changes the base sources for this view */
	UE_API void SetContentSources(const FAssetViewContentSources& InContentSources);

	UE_DEPRECATED(5.6, "Use SetContentSources() instead.")
	UE_API void SetSourcesData(const FSourcesData& InSourcesData);

	/** Returns the sources filter applied to this asset view */
	UE_API const FAssetViewContentSources& GetContentSources() const;

	UE_DEPRECATED(5.6, "Use GetContentSources() instead.")
	UE_API FSourcesData GetSourcesData() const;

	/** Returns true if a real asset path is selected (i.e \Engine\* or \Game\*) */
	UE_API bool IsAssetPathSelected() const;

	/**
	 * @brief Provide a backend filter for the asset view and invalidate current source items.
	 * 
	 * @param InBackendFilter Asset registry filter for uassets
	 * @param InCustomPermissionLists Optional permission lists to allow/deny specific folders.
	 * 	All folders will be allowed by default, so only denials will have any effect.
	 */
	UE_API void SetBackendFilter(const FARFilter& InBackendFilter, TArray<TSharedRef<const FPathPermissionList>>* InCustomPermissionLists = nullptr);

	/** Get the current backend filter */
	const FARFilter& GetBackendFilter() const { return BackendFilter; }
	
	/** Handler for when a data source requests folder item creation */
	UE_API void NewFolderItemRequested(const FContentBrowserItemTemporaryContext& NewItemContext);

	/** Handler for when a data source requests file item creation */
	UE_API void NewFileItemRequested(const FContentBrowserItemDataTemporaryContext& NewItemContext);

	/** Creates a new asset item designed to allocate a new object once it is named. Uses the supplied factory to create the asset */
	UE_API void CreateNewAsset(const FString& DefaultAssetName, const FString& PackagePath, UClass* AssetClass, UFactory* Factory);

	/** Sets up an inline rename for the specified item */
	UE_API void RenameItem(const FContentBrowserItem& ItemToRename);

	/** Selects the specified items. */
	UE_API void SyncToItems( TArrayView<const FContentBrowserItem> ItemsToSync, const bool bFocusOnSync = true );

	/** Selects the specified virtual paths. */
	UE_DEPRECATED(5.8, "Use SyncToItems() instead.")
	UE_API void SyncToVirtualPaths( TArrayView<const FName> VirtualPathsToSync, const bool bFocusOnSync = true );

	/** Selects the specified assets and paths. */
	UE_API void SyncToLegacy( TArrayView<const FAssetData> AssetDataList, TArrayView<const FString> FolderList, const bool bFocusOnSync = true );

	/** Setup Deferred Pending Sync for recently added PendingSyncItems. */
	UE_API void InitDeferredPendingSyncItems();

	/** Sets the state of the asset view to the one described by the history data */
	UE_API void ApplyHistoryData( const FHistoryData& History );

	/** Returns all the items currently selected in the view */
	UE_API TArray<TSharedPtr<FAssetViewItem>> GetSelectedViewItems() const;

	/** Returns all the items currently selected in the view */
	UE_API TArray<FContentBrowserItem> GetSelectedItems() const;

	/** Returns all the folder items currently selected in the view */
	UE_API TArray<FContentBrowserItem> GetSelectedFolderItems() const;

	/** Returns all the file items currently selected in the view */
	UE_API TArray<FContentBrowserItem> GetSelectedFileItems() const;

	/** Returns all the asset data objects in items currently selected in the view */
	UE_API TArray<FAssetData> GetSelectedAssets() const;

	/** Returns all the folders currently selected in the view */
	UE_API TArray<FString> GetSelectedFolders() const;

	/** Requests that the asset view refreshes all it's source items. This is slow and should only be used if the source items change. */
	UE_API void RequestSlowFullListRefresh();

	/** Requests that the asset view refreshes only items that are filtered through frontend sources. This should be used when possible. */
	UE_API void RequestQuickFrontendListRefresh();

	/** Saves any settings to config that should be persistent between editor sessions */
	UE_API void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const;

	/** Loads any settings to config that should be persistent between editor sessions */
	UE_API void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString);

	/** Adjusts the selected asset by the selection delta, which should be +1 or -1) */
	UE_API void AdjustActiveSelection(int32 SelectionDelta);

	/** Returns true if an asset is currently in the process of being renamed */
	UE_API bool IsRenamingAsset() const;

	// SWidget inherited
	UE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	UE_API virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	UE_API virtual FReply OnKeyChar( const FGeometry& MyGeometry,const FCharacterEvent& InCharacterEvent ) override;
	UE_API virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	UE_API virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual void OnFocusChanging( const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent ) override;

	/** Opens the selected assets or folders, depending on the selection */
	UE_API void OnOpenAssetsOrFolders();

	/** Loads the selected assets and previews them if possible */
	UE_API void OnPreviewAssets();

	/** Clears the selection of all the lists in the view */
	UE_API void ClearSelection(bool bForceSilent = false);

	/** Returns true if the asset view is in thumbnail editing mode */
	UE_API bool IsThumbnailEditMode() const;

	/** Delegate called when an editor setting is changed */
	UE_API void HandleSettingChanged(FName PropertyName);

	/** Set whether the user is currently searching or not */
	UE_API void SetUserSearching(bool bInSearching);

	/**
	 * Forces the plugin content folder to be shown.
	 *
	 * @param bEnginePlugin		If true, also forces the engine folder to be shown.
	 */
	UE_API void ForceShowPluginFolder( bool bEnginePlugin );

	/** Enables the Show Engine Content setting for the active Content Browser. The user can still toggle the setting manually. */
	UE_API void OverrideShowEngineContent();

	/** Enables the Show Developer Content setting for the active Content Browser. The user can still toggle the setting manually. */
	UE_API void OverrideShowDeveloperContent();

	/** Enables the Show Plugin Content setting for the active Content Browser. The user can still toggle the setting manually. */
	UE_API void OverrideShowPluginContent();

	/** Enables the Show Localized Content setting for the active Content Browser. The user can still toggle the setting manually. */
	UE_API void OverrideShowLocalizedContent();

	/** @return true when we are including class names in search criteria */
	UE_API bool IsIncludingClassNames() const;

	/** @return true when we are including the entire asset path in search criteria */
	UE_API bool IsIncludingAssetPaths() const;

	/** @return true when we are including collection names in search criteria */
	UE_API bool IsIncludingCollectionNames() const;

	/** Handler for when the view combo button is clicked */
	UE_API TSharedRef<SWidget> GetViewButtonContent();

	/** Sets the view type and updates lists accordingly */
	UE_API void SetCurrentViewType(EAssetViewType::Type NewType);

	/** Sets the thumbnail size and updates lists accordingly */
	UE_API void SetCurrentThumbnailSize(EThumbnailSize NewThumbnailSize);

	/** Gets the text for the asset count label */
	UE_API FText GetAssetCountText() const;

	/** Gets the visibility for the majority asset type label */
	EVisibility GetMajorityAssetTypeTextVisibility() const;

	/** Gets the text for the majority asset type label */
	UE_API FText GetMajorityAssetTypeText() const;

	/** Gets the tooltip text for the majority asset type label */
	UE_API FText GetMajorityAssetTypeToolTipText() const;

	/** Gets text name for given thumbnail */
	static UE_API FText ThumbnailSizeToDisplayName(EThumbnailSize InSize);

	/** Set the filter list attached to this asset view - allows toggling of the the filter bar layout from the view options */
	UE_API void SetFilterBar(TSharedPtr<SFilterList> InFilterBar);

	/** Change the delegate bound via the widget argument OnShouldFilterAsset after construction. */
	UE_API void SetShouldFilterItem(FOnShouldFilterItem InCallback);

	/**
	 * Pass the active per-view search override instance to the asset view.
	 * The asset view will intercept items at the publish step and wire publish-results back.
	 * EXPERIMENTAL: This API is subject to change.
	 */
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	UE_API void SetAssetSearchOverride(TSharedPtr<UE::Editor::ContentBrowser::Extension::ICBSearchOverrideInstance> InOverride);
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	/** True if we have items pending that ProcessItemsPendingFilter still needs to process */
	UE_API bool HasItemsPendingFilter() const;

	/** True if we have thumbnails that are pending update */
	UE_API bool HasThumbnailsPendingUpdate() const;

	/** True if we have deferred item to create */
	UE_API bool HasDeferredItemToCreate() const;

	/** Get the underlying sort manager */
	UE_API TWeakPtr<FAssetViewSortManager> GetSortManager() const;

	/** Set's sorting based on the given parameters, using the current value if not specified */
	UE_API void SetSortParameters(
		const TOptional<const EColumnSortPriority::Type>& InSortPriority,
		const TOptional<const FName>& InColumnId,
		const TOptional<const EColumnSortMode::Type>& InNewSortMode);

	/** @return true when we are showing favorites */
	UE_INTERNAL UE_API bool IsShowingFavorites() const;

	/** @return true when we are showing C++ content */
	UE_INTERNAL UE_API bool IsShowingCppContent() const;

	/** @return true when we are showing folders */
	UE_INTERNAL UE_API bool IsShowingFolders() const;

	/** @return true when we are showing empty folders */
	UE_INTERNAL UE_API bool IsShowingEmptyFolders() const;

	/** @return true when we are showing plugin content */
	UE_INTERNAL UE_API bool IsShowingPluginContent() const;

	/** @return true when we are showing engine content */
	UE_INTERNAL UE_API bool IsShowingEngineContent() const;

	/** @return true when we are showing the developers content */
	UE_INTERNAL UE_API bool IsShowingDevelopersContent() const;

	/** @return true when we are showing the all virtual folder */
	UE_INTERNAL UE_API bool IsShowingAllFolder() const;

	/** @return true when we are organizing folders into virtual locations */
	UE_INTERNAL UE_API bool IsOrganizingFolders() const;

	/** Is the current thumbnail size menu option checked */
	UE_INTERNAL UE_API bool IsThumbnailSizeChecked(EThumbnailSize InThumbnailSize) const;

	/** Returns true if ViewType is the current view type */
	UE_INTERNAL UE_API bool IsCurrentViewType(EAssetViewType::Type ViewType) const;

private:
	/** Sets the pending selection to the current selection (used when changing views or refreshing the view). */
	UE_API void SyncToSelection( const bool bFocusOnSync = true );

	/** @return the thumbnail scale setting path to use when looking up the setting in an ini. */
	UE_API FString GetThumbnailScaleSettingPath(const FString& SettingsString, const FString& InViewTypeString) const;

	/** Load the config for each AssetViewType size into their own container */
	UE_API void LoadScaleSetting(const FString& IniFilename, const FString& IniSection, const FString& SettingsString, const FString& InViewTypeString, EThumbnailSize& OutThumbnailSize);

	/** @return the view type setting path to use when looking up the setting in an ini. */
	UE_API FString GetCurrentViewTypeSettingPath(const FString& SettingsString) const;

	/** Calculates a new filler scale used to adjust the thumbnails to fill empty space. */
	UE_API void CalculateFillScale( const FGeometry& AllottedGeometry );

	/** Calculates the latest color and opacity for the hint on thumbnails */
	UE_API void CalculateThumbnailHintColorAndOpacity();

	/** Handles amortizing the additional filters */
	UE_API void ProcessItemsPendingFilter(const double TickStartTime);

	/** Creates a new tile view */
	UE_API TSharedRef<SAssetTileView> CreateTileView();

	/** Creates a new list view */
	UE_API TSharedRef<SAssetListView> CreateListView();

	/** Creates a new column view */
	UE_API TSharedRef<SAssetColumnView> CreateColumnView();
	
	/** Creates a custom view (if specified to the content browser module) */
	UE_API TSharedRef<SWidget> CreateCustomView();

	UE_API const FSlateBrush* GetRevisionControlColumnIconBadge() const;

	/** Returns true if the specified search token is allowed */
	UE_API bool IsValidSearchToken(const FString& Token) const;

	/** Regenerates the AssetItems list from the AssetRegistry */
	UE_API void RefreshSourceItems();

	/** Regenerates the FilteredAssetItems list from the AssetItems list */
	UE_API void RefreshFilteredItems();

	/** Sets the asset type that represents the majority of the assets in view */
	UE_API void SetMajorityAssetType(FName NewMajorityAssetType);

	/** Handler for when an asset is added to a collection */
	UE_API void OnAssetsAddedToCollection( ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> ObjectPaths );

	/** Handler for when an asset is removed from a collection */
	UE_API void OnAssetsRemovedFromCollection(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection, TConstArrayView<FSoftObjectPath> ObjectPaths );

	/** Handler for when a collection is renamed */
	UE_API void OnCollectionRenamed(ICollectionContainer& CollectionContainer, const FCollectionNameType& OriginalCollection, const FCollectionNameType& NewCollection );

	/** Handler for when a collection is updated */
	UE_API void OnCollectionUpdated(ICollectionContainer& CollectionContainer, const FCollectionNameType& Collection );

	/** Handler for when any frontend filters have been changed */
	UE_API void OnFrontendFiltersChanged();

	/** Returns true if there is any frontend filter active */
	UE_API bool IsFrontendFilterActive() const;

	/** Returns true if the specified Content Browser item passes all applied frontend (non asset registry) filters */
	UE_API bool PassesCurrentFrontendFilter(const FContentBrowserItem& Item) const;

	/** Returns true if the current filters deem that the asset view should be filtered recursively (overriding folder view) */
	UE_API bool ShouldFilterRecursively() const;

	/** Sorts the contents of the asset view alphabetically */
	UE_API void SortList(bool bSyncToSelection = true);

	/** Returns the thumbnails hint color and opacity */
	UE_API FLinearColor GetThumbnailHintColorAndOpacity() const;

	/** Populate the given params for this AssetView */
	UE_API void PopulateFilterAdditionalParams(FFiltersAdditionalParams& OutParams);

	/** Registers the menu used by the settings button */
	static UE_API FDelayedAutoRegisterHelper AssetViewOptionsMenuRegistration;

	/** Extend the top toolbar */
	static UE_API FDelayedAutoRegisterHelper ToolBarMenuExtensionRegistration;

	/** Extend the bottom navigation bar */
	static UE_API FDelayedAutoRegisterHelper NavigationBarMenuExtensionRegistration;

	UE_API void OnSetSortParameters(const FToolMenuContext& InMenuContext,
		const TOptional<const EColumnSortPriority::Type> InSortPriority,
		const TOptional<const FName> InColumnId,
		const TOptional<const EColumnSortMode::Type> InNewSortMode);

	/** Fill in menu content for when the sorting combo button is clicked */
	UE_API void PopulateSortingButtonMenu(UToolMenu* Menu);

	/** Toggle whether folders should be shown or not */
	UE_API void ToggleShowFolders();

	/** Whether or not it's possible to show folders */
	UE_API bool IsToggleShowFoldersAllowed() const;

	/** @return true when we are showing read-only folders */
	UE_API bool IsShowingReadOnlyFolders() const;

	/** Toggle whether empty folders should be shown or not */
	UE_API void ToggleShowEmptyFolders();

	/** Whether or not it's possible to show empty folders */
	UE_API bool IsToggleShowEmptyFoldersAllowed() const;

	/** @return true when the asset view is showing object redirectors */
	UE_API bool IsShowingRedirectors() const;

	/** Toggle whether localized content should be shown or not */
	UE_API void ToggleShowLocalizedContent();

	/** Whether or not it's possible to show localized content */
	UE_API bool IsToggleShowLocalizedContentAllowed() const;

	/** @return true when we are showing folders */
	UE_API bool IsShowingLocalizedContent() const;

	/** Toggle whether to show real-time thumbnails */
	UE_API void ToggleRealTimeThumbnails();

	/** Whether it is possible to show real-time thumbnails */
	UE_API bool CanShowRealTimeThumbnails() const;

	/** @return true if we are showing real-time thumbnails */
	UE_API bool IsShowingRealTimeThumbnails() const;

	/** Toggle whether plugin content should be shown or not */
	UE_API void ToggleShowPluginContent();

	/** Toggle whether engine content should be shown or not */
	UE_API void ToggleShowEngineContent();

	/** Toggle whether developers content should be shown or not */
	UE_API void ToggleShowDevelopersContent();

	/** Whether or not it's possible to toggle developers content */
	UE_API bool IsToggleShowDevelopersContentAllowed() const;

	/** Whether or not it's possible to toggle engine content */
	UE_API bool IsToggleShowEngineContentAllowed() const;

	/** Whether or not it's possible to toggle plugin content */
	UE_API bool IsToggleShowPluginContentAllowed() const;

	/** Toggle whether favorites should be shown or not */
	UE_API void ToggleShowFavorites();

	/** Whether or not it's possible to toggle favorites */
	UE_API bool IsToggleShowFavoritesAllowed() const;

	/** Whether or not it's possible to show C++ content */
	UE_API bool IsToggleShowCppContentAllowed() const;

	/** Toggle whether class names should be included in search criteria */
	UE_API void ToggleIncludeClassNames();

	/** Whether or not it's possible to include class names in search criteria */
	UE_API bool IsToggleIncludeClassNamesAllowed() const;

	/** Toggle whether the entire asset path should be included in search criteria */
	UE_API void ToggleIncludeAssetPaths();

	/** Whether or not it's possible to include the entire asset path in search criteria */
	UE_API bool IsToggleIncludeAssetPathsAllowed() const;

	/** Toggle whether collection names should be included in search criteria */
	UE_API void ToggleIncludeCollectionNames();

	/** Whether or not it's possible to include collection names in search criteria */
	UE_API bool IsToggleIncludeCollectionNamesAllowed() const;

	/** @return true when we are filtering recursively when we have an asset path */
	UE_API bool IsFilteringRecursively() const;

	/** Whether or not it's possible to toggle how to filtering recursively */
	UE_API bool IsToggleFilteringRecursivelyAllowed() const;

	/** Toggle whether we're filtering recursively */
	UE_API void ToggleFilteringRecursively();

	/** Toggle whether we are showing the all virtual folder */
	UE_API void ToggleShowAllFolder();

	/** Toggle whether we are re-organizing folders into virtual locations */
	UE_API void ToggleOrganizeFolders();

	/** Sets the view type and forcibly dismisses all currently open context menus */
	UE_API void SetCurrentViewTypeFromMenu(EAssetViewType::Type NewType);

	/** Clears the reference to the current view and creates a new one, based on CurrentViewType */
	UE_API void CreateCurrentView();

	/** Gets the current view type (list or tile) */
	UE_API EAssetViewType::Type GetCurrentViewType() const;

	UE_API TSharedRef<SWidget> CreateShadowOverlay( TSharedRef<STableViewBase> Table );

	/** Set the keyboard focus to the correct list view that should be active */
	UE_API void FocusList() const;

	/** Return true if there is currently a refresh pending, false otherwise */
	UE_API bool IsPendingRefresh();

	/** Refreshes the list view to display any changes made to the non-filtered assets */
	UE_API void RefreshList();

	/** Sets the sole selection for all lists in the view */
	UE_API void SetSelection(const TSharedPtr<FAssetViewItem>& Item);

	/** Sets selection for an item in all lists in the view */
	UE_API void SetItemSelection(const TSharedPtr<FAssetViewItem>& Item, bool bSelected, const ESelectInfo::Type SelectInfo = ESelectInfo::Direct);

	/** Scrolls the selected item into view for all lists in the view */
	UE_API void RequestScrollIntoView(const TSharedPtr<FAssetViewItem>& Item);

	/** Handler for list view widget creation */
	UE_API TSharedRef<ITableRow> MakeListViewWidget(TSharedPtr<FAssetViewItem> AssetItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Handler for tile view widget creation */
	UE_API TSharedRef<ITableRow> MakeTileViewWidget(TSharedPtr<FAssetViewItem> AssetItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Handler for column view widget creation */
	UE_API TSharedRef<ITableRow> MakeColumnViewWidget(TSharedPtr<FAssetViewItem> AssetItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Handler for when any asset item widget gets destroyed */
	UE_API void AssetItemWidgetDestroyed(const TSharedPtr<FAssetViewItem>& Item);
	
	/** Creates new thumbnails that are near the view area and deletes old thumbnails that are no longer relevant. */
	UE_API void UpdateThumbnails();

	/**  Helper function for UpdateThumbnails. Adds the specified item to the new thumbnail relevancy map and creates any thumbnails for new items. Returns the thumbnail. */
	UE_API TSharedPtr<FAssetThumbnail> AddItemToNewThumbnailRelevancyMap(const TSharedPtr<FAssetViewItem>& Item, TMap< TSharedPtr<FAssetViewItem>, TSharedPtr<FAssetThumbnail> >& NewRelevantThumbnails);

	/** Handler for tree view selection changes */
	UE_API void AssetSelectionChanged( TSharedPtr<FAssetViewItem > AssetItem, ESelectInfo::Type SelectInfo );

	/** Handler for when an item has scrolled into view after having been requested to do so */
	UE_API void ItemScrolledIntoView(TSharedPtr<FAssetViewItem> AssetItem, const TSharedPtr<ITableRow>& Widget);

	/** Handler for context menus */
	UE_API TSharedPtr<SWidget> OnGetContextMenuContent();

	/** Handler called when an asset context menu is about to open */
	UE_API bool CanOpenContextMenu() const;

	/** Handler for double clicking an item */
	UE_API void OnListMouseButtonDoubleClick(TSharedPtr<FAssetViewItem> AssetItem);

	/** Handle dragging an asset */
	UE_API FReply OnDraggingAssetItem(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Checks if the asset name being committed is valid */
	UE_API bool AssetVerifyRenameCommit(const TSharedPtr<FAssetViewItem>& Item, const FText& NewName, const FSlateRect& MessageAnchor, FText& OutErrorMessage);

	/** An asset item has started to be renamed */
	UE_API void AssetRenameBegin(const TSharedPtr<FAssetViewItem>& Item, const FString& NewName, const FSlateRect& MessageAnchor);

	/** An asset item that was prompting the user for a new name was committed. Return false to indicate that the user should enter a new name */
	UE_API void AssetRenameCommit(const TSharedPtr<FAssetViewItem>& Item, const FString& NewName, const FSlateRect& MessageAnchor, const ETextCommit::Type CommitType);

	/** Gets the color and opacity for all names of assets in the asset view */
	UE_API FLinearColor GetAssetNameColorAndOpacity() const;

	/** Returns true if tooltips should be allowed right now. Tooltips are typically disabled while right click scrolling. */
	UE_API bool ShouldAllowToolTips() const;

	/** Returns true if the asset view is currently allowing the user to edit thumbnails */
	UE_API bool IsThumbnailEditModeAllowed() const;

	/** The "Done Editing" button was pressed in the thumbnail edit mode strip */
	UE_API FReply EndThumbnailEditModeClicked();

	/** Gets the visibility of the Thumbnail Edit Mode label */
	UE_API EVisibility GetEditModeLabelVisibility() const;

	/** Gets the visibility of the list view */
	UE_API EVisibility GetListViewVisibility() const;

	/** Gets the visibility of the tile view */
	UE_API EVisibility GetTileViewVisibility() const;

	/** Gets the visibility of the column view */
	UE_API EVisibility GetColumnViewVisibility() const;

	/** Toggles tooltip expanded default state */
	UE_API void ToggleTooltipExpandedState();

	/** True if the default state of tooltip is expanded */
	UE_API bool IsTooltipExpandedByDefault();

	/** Toggles thumbnail editing mode */
	UE_API void ToggleThumbnailEditMode();

	/** Called when  thumbnail size is changed */
	UE_API void OnThumbnailSizeChanged(EThumbnailSize NewThumbnailSize);

	/** Are thumbnails allowed to be scaled by the user */
	UE_API bool IsThumbnailScalingAllowed() const;

	/** Gets the current thumbnail scale */
	UE_API float GetThumbnailScale() const;

	/** Gets the current thumbnail size */
	UE_API float GetThumbnailSizeValue() const;

	/** Set the Min/Max Thumbnail size based on the EThumbnailSize chosen */
	UE_API void UpdateThumbnailSizeValue();

	/** Gets the current thumbnail size enum */
	EThumbnailSize GetThumbnailSize() const { return ThumbnailSizes.Contains(CurrentViewType) ? ThumbnailSizes[CurrentViewType] : EThumbnailSize::Medium; }

	/** Gets the spacing dedicated to support type names */
	UE_API float GetTileViewTypeNameHeight() const;

	/** Gets the spacing needed to support source control icons. No scaling is required for this since the icon itself is not scaled by the thumbnail size. */
	UE_API float GetSourceControlIconHeight() const;

	/** Gets the scaled item height for the list view */
	UE_API float GetListViewItemHeight() const;

	/** Gets the padding for the list view rows */
	UE_API FMargin GetListViewItemPadding() const;

	/** Gets the final scaled item height for the tile view */
	UE_API float GetTileViewItemHeight() const;

	/** Get the TileView Thumbnail dimension height and width are the same for the thumbnail itself */
	UE_API float GetTileViewThumbnailDimension() const;

	/** Gets the scaled item height for the tile view before the filler scale is applied */
	UE_API float GetTileViewItemBaseHeight() const;

	/** Gets the final scaled item width for the tile view */
	UE_API float GetTileViewItemWidth() const;

	/** Gets the scaled item width for the tile view before the filler scale is applied */
	UE_API float GetTileViewItemBaseWidth() const;

	/** Gets the sort mode for the supplied ColumnId */
	UE_API EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

	/** Gets the sort order for the supplied ColumnId */
	UE_API EColumnSortPriority::Type GetColumnSortPriority(const FName ColumnId) const;

	/** Handler for when a column header is clicked */
	UE_API void OnSortColumnHeader(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type NewSortMode);

	/** @return The state of the is working progress bar */
	UE_API TOptional< float > GetIsWorkingProgressBarState() const;

	/** Is the no assets to show warning visible? */
	UE_API EVisibility IsAssetShowWarningTextVisible() const;

	/** Gets the warning text to show as heading (e.g. when there are no assets to show) */
	UE_API FText GetAssetShowWarningText() const;

	/** Whether we have a single source collection selected */
	UE_API bool HasSingleCollectionSource() const;

	/** Starts the async flow of creating a file or folder item from deferred data */
	UE_API void BeginCreateDeferredItem();

	/** Finishes the async flow of creating a file or folder item from deferred data */
	UE_API FContentBrowserItem EndCreateDeferredItem(const TSharedPtr<FAssetViewItem>& InItem, const FString& InName, const bool bFinalize, FText& OutErrorText);

	/** @return The current quick-jump term */
	UE_API FText GetQuickJumpTerm() const;

	/** @return Whether the quick-jump term is currently visible? */
	UE_API EVisibility IsQuickJumpVisible() const;

	/** @return The color that should be used for the quick-jump term */
	UE_API FSlateColor GetQuickJumpColor() const;

	/** Reset the quick-jump to its empty state */
	UE_API void ResetQuickJump();

	/**
	 * Called from OnKeyChar and OnKeyDown to handle quick-jump key presses
	 * @param InCharacter		The character that was typed
	 * @param bIsControlDown	Was the control key pressed?
	 * @param bIsAltDown		Was the alt key pressed?
	 * @param bTestOnly			True if we only want to test whether the key press would be handled, but not actually update the quick-jump term
	 * @return FReply::Handled or FReply::Unhandled
	 */
	UE_API FReply HandleQuickJumpKeyDown(const TCHAR InCharacter, const bool bIsControlDown, const bool bIsAltDown, const bool bTestOnly);

	/**
	 * Perform a quick-jump to the next available asset in FilteredAssetItems that matches the current term
	 * @param bWasJumping		True if we were performing an ongoing quick-jump last Tick
	 * @return True if the quick-jump found a valid match, false otherwise
	 */
	UE_API bool PerformQuickJump(const bool bWasJumping);
	
	/** Resets the column filtering state to make them all visible */
	UE_API void ResetColumns();

	/** Export columns to CSV */
	UE_API void ExportColumns();

	/** Called when a column is shown/hidden in the column view */
	UE_API void OnHiddenColumnsChanged();

	/** Will compute the max row size from all its children for the specified column id*/
	UE_API FVector2D GetMaxRowSizeForColumn(const FName& ColumnId);

	/** Append the current effective backend filter (intersection of BackendFilter and SupportedFilter) to the given filter. */
	UE_API void AppendBackendFilter(FARFilter& FilterToAppendTo) const;

	UE_API EContentBrowserItemCategoryFilter DetermineItemCategoryFilter() const;
	UE_API FContentBrowserDataFilter CreateBackendDataFilter(bool bInvalidateCache) const;

	/** Handles updating the view when content items are changed */
	UE_API void HandleItemDataUpdated(TArrayView<const FContentBrowserItemDataUpdate> InUpdatedItems);

	/** Notification for when the content browser has completed it's initial search */
	UE_API void HandleItemDataDiscoveryComplete();

	/** Get the config struct for the owning Content Browser, if one exists. */
	UE_API FContentBrowserInstanceConfig* GetContentBrowserConfig() const;

	/** Get the config struct for this asset view, if one exists. */
	UE_API FAssetViewInstanceConfig* GetAssetViewConfig() const;

	/** Bind our UI commands */
	UE_API void BindCommands();

	/** Populate the given parameters based on the current selection */
	UE_API void PopulateSelectedFilesAndFolders(TArray<FContentBrowserItem>& OutSelectedFolders, TArray<FContentBrowserItem>& OutSelectedFiles) const;

	/** Handler for the CopyReference CopyObjectPath and CopyPackageName */
	UE_API void ExecuteCopy(EAssetViewCopyType InCopyType) const;

	/** Append folders path to the given ClipboardText */
	UE_API void ExecuteCopyFolders(const TArray<FContentBrowserItem>& InSelectedFolders, FString& OutClipboardText) const;

	/** Handler for Paste */
	UE_API void ExecutePaste();

	/** Check if the custom view view is available */
	UE_API bool IsCustomViewSet() const;

private:
	/** Cancel the ongoing rename if there is one before starting another one. */
	void CancelPendingRename();

	friend class FAssetViewFrontendFilterHelper;

	/** Private type managing data retrieved from the backend and async filtering thereof */
	TPimplPtr<FAssetViewItemCollection> Items;

	/** The items that are being shown in the filtered view list */
	TArray<TSharedPtr<FAssetViewItem>> FilteredAssetItems;

	/* Map of an item name to the current count of FilteredAssetItems that are of that type */ 
	TMap<FName, int32> FilteredAssetItemTypeCounts;

	/** The list view that is displaying the assets */
	EAssetViewType::Type CurrentViewType;
	TSharedPtr<SAssetListView> ListView;
	TSharedPtr<SAssetTileView> TileView;
	TSharedPtr<SListView<TSharedPtr<FAssetViewItem>>> ColumnView;
	TSharedPtr<SBox> ViewContainer;

	/** The content browser that created this asset view if any */
	TWeakPtr<SContentBrowser> OwningContentBrowser;

	/** The Filter Bar attached to this asset view if any */
	TWeakPtr<SFilterList> FilterBar;

	/** The current base source filter for the view */
	FAssetViewContentSources ContentSources;
	FARFilter BackendFilter;
	TSharedPtr<FPathPermissionList> AssetClassPermissionList;
	TSharedPtr<FPathPermissionList> FolderPermissionList;
	TSharedPtr<FPathPermissionList> WritableFolderPermissionList;
	// Paths which should be filtered out based on current filters the user has selected 
	//  - not 'permissions' so may be ignore if e.g. the user explicitly selects a filtered folder
	TArray<TSharedRef<const FPathPermissionList>> BackendCustomPathFilters;
	TSharedPtr<FAssetFilterCollectionType> FrontendFilters;
	TSharedPtr<FAssetTextFilter> TextFilter;

	/** Scale when using CTRL+Wheel, will go from 0.f to 1.f and reset accordingly when ThumbnailSize chosen changes\n
	 * When going over/under the limit will also change the ThumbnailSize
	 */
	float ZoomScale = 0.0f;

	/** Vertical padding for the TileViewItem */
	static constexpr int32 TileViewHeightPadding = 9;

	/** Horizontal padding for the TileViewItem */
	static constexpr int32 TileViewWidthPadding = 8;

	TAttribute<bool> bShowRedirectors;
	bool bLastShowRedirectors;

	/** Show path view filters submenu in view options menu  */
	bool bShowPathViewFilters;

	/** If true, the source items will be refreshed next frame. Very slow. */
	bool bSlowFullListRefreshRequested;

	/** If true, the frontend items will be refreshed next frame. Much faster. */
	bool bQuickFrontendListRefreshRequested;

	/** True when loading settings, this is needed for the HiddenColumns for example to not double update them */
	bool bLoadingSettings = false;

	/** Set to true as soon as the user change the ListView visibility of a column, after that point the Loaded Hidden Columns will be used instead of any default given */
	bool bListViewColumnsManuallyChangedOnce = false;

	/** Set to true as soon as the user change the ColumnView visibility of a column, after that point the Loaded Hidden Columns will be used instead of any default given */
	bool bColumnViewColumnsManuallyChangedOnce = false;

	/** The list of items to sync next frame */
	FSelectionData PendingSyncItems;

	/** The list of items used to ensure all the pending sync items are selected due to async nature of filtering*/
	FSelectionData DeferredPendingSyncItems;

	/** A Timeout counter to safeguard against infinite deferement of pending sync items*/
	int32 DeferredSyncTimeoutFrames = 0;

	/** Should we take focus when the PendingSyncAssets are processed? */
	bool bPendingFocusOnSync;

	/** Was recursive filtering enabled the last time a full slow refresh performed? */
	bool bWereItemsRecursivelyFiltered;

	/** Called to check if an asset should be filtered out by external code */
	FOnShouldFilterAsset OnShouldFilterAsset;

	/** Called to check if an item should be filtered out by external code */
	FOnShouldFilterItem OnShouldFilterItem;

	/** Active per-view search override instance; when set, intercepts items at the publish step. */
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	TSharedPtr<UE::Editor::ContentBrowser::Extension::ICBSearchOverrideInstance> AssetSearchOverride;
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	/** Called when the asset view is asked to start to create a temporary item */
	FOnAssetViewNewItemRequested OnNewItemRequested;

	/** Called when an item was selected in the list. Provides more context than OnAssetSelected. */
	FOnContentBrowserItemSelectionChanged OnItemSelectionChanged;

	/** Called when the user double clicks, presses enter, or presses space on a Content Browser item */
	FOnContentBrowserItemsActivated OnItemsActivated;

	/** Called when the user drags a Content Browser item */
	FOnContentBrowserItemsDragged OnItemsDragged;

	/** Delegate to invoke when generating the context menu for an item */
	FOnGetContentBrowserItemContextMenu OnGetItemContextMenu;

	/** Called when the user has committed a rename of one or more items */
	FOnContentBrowserItemRenameCommitted OnItemRenameCommitted;

	/** Called to check if an asset tag should be display in details view. */
	FOnShouldDisplayAssetTag OnAssetTagWantsToBeDisplayed;

	/** Called to see if it is valid to get a custom asset tool tip */
	FOnIsAssetValidForCustomToolTip OnIsAssetValidForCustomToolTip;

	/** Called to get a custom asset item tooltip (If necessary) */
	FOnGetCustomAssetToolTip OnGetCustomAssetToolTip;

	/** Called when a custom asset item is about to show a tooltip */
	FOnVisualizeAssetToolTip OnVisualizeAssetToolTip;

	/** Called when a custom asset item's tooltip is closing */
	FOnAssetToolTipClosing OnAssetToolTipClosing;

	/** Called to add extra asset data to the asset view, to display virtual assets. These get treated similar to Class assets */
	FOnGetCustomSourceAssets OnGetCustomSourceAssets;

	/** Called when a search option changes to notify that results should be rebuilt */
	FOnSearchOptionChanged OnSearchOptionsChanged;

	/** Called when the user changes column visibility */
	FOnColumnVisibilityChanged OnColumnVisibilityChanged;

	/** Called when opening view options menu */
	FOnExtendAssetViewOptionsMenuContext OnExtendAssetViewOptionsMenuContext;

	/** An optional profile name for the asset view options menu. */
	TOptional<FName> AssetViewOptionsProfile;
	
	/** When true, filtered list items will be sorted next tick. Provided another sort hasn't happened recently or we are renaming an asset */
	bool bPendingSortFilteredItems;
	double CurrentTime;
	double LastSortTime;
	double SortDelaySeconds;

	/** Weak ptr to the asset that is waiting to be renamed when scrolled into view, and the window is active */
	TWeakPtr<FAssetViewItem> AwaitingScrollIntoViewForRename;

	/** Weak ptr to the asset that is waiting to be renamed now that it has scrolled into view, if window is active */
	TWeakPtr<FAssetViewItem> AwaitingRename;

	/** Set when the user is in the process of naming an asset */
	TWeakPtr<FAssetViewItem> RenamingAsset;

	/** Pool for maintaining and rendering thumbnails */
	TSharedPtr<FAssetThumbnailPool> AssetThumbnailPool;

	/** A map of FAssetViewAsset to the thumbnail that represents it. Only items that are currently visible or within half of the FilteredAssetItems array index distance described by NumOffscreenThumbnails are in this list */
	TMap< TSharedPtr<FAssetViewItem>, TSharedPtr<FAssetThumbnail> > RelevantThumbnails;

	/** The set of FAssetItems that currently have widgets displaying them. */
	TArray< TSharedPtr<FAssetViewItem> > VisibleItems;

	/** The number of thumbnails to keep for asset items that are not currently visible. Half of the thumbnails will be before the earliest item and half will be after the latest. */
	int32 NumOffscreenThumbnails;

	/** The current size of relevant thumbnails */
	int32 CurrentThumbnailSize;

	/** Flag to defer thumbnail updates until the next frame */
	bool bPendingUpdateThumbnails;

	/** The size of thumbnails */
	int32 ListViewThumbnailResolution;
	int32 ListViewThumbnailSize;
	int32 ListViewThumbnailPadding;
	float ListViewItemHeight;
	int32 TileViewThumbnailResolution;
	int32 TileViewThumbnailSize;
	int32 TileViewThumbnailPadding;
	int32 TileViewNameHeight;

	/** The max and min thumbnail scales as a fraction of the rendered size */
	float MinThumbnailScale;
	float MaxThumbnailScale;

	/** The max and min thumbnail sizes */
	float MinThumbnailSize;
	float MaxThumbnailSize;

	/** Scalar applied to thumbnail sizes so that users thumbnails are scaled based on users display area size*/
	float ThumbnailScaleRangeScalar;

	/** Current thumbnail sizes */
	TMap<EAssetViewType::Type, EThumbnailSize> ThumbnailSizes;

	// Specifiers used for Config
	const FString GridViewSpecifier = TEXT("Grid");
	const FString ListViewSpecifier = TEXT("List");
	const FString ColumnViewSpecifier = TEXT("Column");
	const FString CustomViewSpecifier = TEXT("Custom");

	/** The amount to scale each thumbnail so that the empty space is filled. */
	float FillScale;

	/** When in columns view, this is the name of the asset type which is most commonly found in the recent results */
	FName MajorityAssetType;

	/** When in columns view, this is the display name of the asset type which is most commonly found in the recent results */
	FText MajorityAssetTypeText;

	/** The manager responsible for sorting assets in the view */
	TSharedPtr<FAssetViewSortManager> SortManager;

	/** Flag indicating if we will be filling the empty space in the tile view. */
	bool bFillEmptySpaceInTileView;

	/** When true, selection change notifications will not be sent */
	bool bBulkSelecting;

	/** When true, the user may edit thumbnails */
	bool bAllowThumbnailEditMode : 1;

	/** True when the asset view is currently allowing the user to edit thumbnails */
	bool bThumbnailEditMode : 1;

	/** Indicates if this view is allowed to show classes */
	bool bCanShowClasses : 1;

	/** Indicates if the 'Show Folders' option should be enabled or disabled */
	bool bCanShowFolders : 1;

	/** Indicates if this view is allowed to show folders that cannot be written to */
	bool bCanShowReadOnlyFolders : 1;

	/** If true, recursive filtering will be caused by applying a backend filter */
	bool bFilterRecursivelyWithBackendFilter : 1;

	/** Indicates if the 'Real-Time Thumbnails' option should be enabled or disabled */
	bool bCanShowRealTimeThumbnails : 1;

	/** Indicates if the 'Show Developers' option should be enabled or disabled */
	bool bCanShowDevelopersFolder : 1;

	/** Indicates if the 'Show Favorites' option should be enabled or disabled */
	bool bCanShowFavorites : 1;

	/** If true, it will show path column in the asset view */
	bool bShowPathInColumnView : 1;

	/** If true, it will show type in the asset view */
	bool bShowTypeInColumnView : 1;

	/** If true, it will show EAssetAccessSpecifier */
	bool bShowAssetAccessSpecifier : 1;

	/** If true, it sorts by path and then name */
	bool bSortByPathInColumnView : 1;

	/** If true, it will show type in the tile view */
	bool bShowTypeInTileView : 1;

	/** If true, engine content is always shown */
	bool bForceShowEngineContent : 1;

	/** If true, plugin content is always shown */
	bool bForceShowPluginContent : 1;

	/** If true, scrollbar is never shown, removes scroll border entirely */
	bool bForceHideScrollbar : 1;

	/** Whether to allow dragging of items */
	bool bAllowDragging : 1;

	/** Whether this asset view should allow focus on sync or not */
	bool bAllowFocusOnSync : 1;

	/** Flag set if the user is currently searching */
	bool bUserSearching : 1;
	
	/** Whether or not to notify about newly selected items on on the next asset sync */
	bool bShouldNotifyNextAssetSync : 1;

	/** The current selection mode used by the asset view */
	ESelectionMode::Type SelectionMode;

	/** The max number of results to process per tick */
	float MaxSecondsPerFrame;

	/** When delegate amortization began */
	double AmortizeStartTime;

	/** The total time spent amortizing the delegate filter */
	double TotalAmortizeTime;

	/** The initial number of tasks when we started performing amortized work (used to display a progress cue to the user) */
	int32 InitialNumAmortizedTasks;

	/** The text to highlight on the assets */
	TAttribute< FText > HighlightedText;

	/** What the label on the thumbnails should be */
	EThumbnailLabel::Type ThumbnailLabel;

	/** Whether to ever show the hint label on thumbnails */
	bool AllowThumbnailHintLabel;

	/** The sequence used to generate the opacity of the thumbnail hint */
	FCurveSequence ThumbnailHintFadeInSequence;

	/** The current thumbnail hint color and opacity*/
	FLinearColor ThumbnailHintColorAndOpacity;

	/** The warning text to show as heading (e.g. when there are no assets to show) */
	TAttribute<FText> AssetShowWarningText;

	/** The justification to use for the AssetShowWarning text (see AssetShowWarningText) */
	TAttribute<ETextJustify::Type> AssetShowWarningTextJustification;
	
	/** Whether to allow selection of the warning text or not */
	TAttribute<bool> AssetShowWarningTextSelectable;

	/** Initial set of item categories that this view should show - may be adjusted further by things like CanShowClasses or legacy delegate bindings */
	EContentBrowserItemCategoryFilter InitialCategoryFilter;

	/** Commands handled by this widget */
	TSharedPtr<FUICommandList> Commands;

	bool bShowDisallowedAssetClassAsUnsupportedItems = false;

	/** A struct to hold data for the deferred creation of a file or folder item */
	struct FCreateDeferredItemData
	{
		FContentBrowserItemTemporaryContext ItemContext;

		bool bWasAddedToView = false;
	};

	/** File or folder item pending deferred creation */
	TUniquePtr<FCreateDeferredItemData> DeferredItemToCreate;

	/** Struct holding the data for the asset quick-jump */
	struct FQuickJumpData
	{
		FQuickJumpData()
			: bIsJumping(false)
			, bHasChangedSinceLastTick(false)
			, bHasValidMatch(false)
			, LastJumpTime(0)
		{
		}

		/** True if we're currently performing an ongoing quick-jump */
		bool bIsJumping;

		/** True if the jump data has changed since the last Tick */
		bool bHasChangedSinceLastTick;

		/** True if the jump term found a valid match */
		bool bHasValidMatch;

		/** Time (taken from Tick) that we last performed a quick-jump */
		double LastJumpTime;

		/** The string we should be be looking for */
		FString JumpTerm;
	};

	/** Data for the asset quick-jump */
	FQuickJumpData QuickJumpData;
	
	/** Column filtering state */
	TArray<FString> DefaultHiddenColumnNames;
	TArray<FString> HiddenColumnNames;
	TArray<FString> ListHiddenColumnNames;
	TArray<FString> DefaultListHiddenColumnNames;

	TArray<FAssetViewCustomColumn> CustomColumns;

	/** An Id for the cache of the data sources for the filters compilation */
	FContentBrowserDataFilterCacheIDOwner FilterCacheID;

	/** An extender to create the custom view */
	TSharedPtr<IContentBrowserViewExtender> ViewExtender;

	/** The actual widget for the custom view */
	TSharedPtr<SWidget> CustomView;
	
	/** Existing item paths to avoid duplicates */
	TSet<FSoftObjectPath> AlreadyAddedOriginalAssets;
	
	/*
	 * Telemetry-related fields
	 */ 
	// Guid to correlate different telemetry events for this instance of an asset view over time 
	FGuid ViewCorrelationGuid; 
	// Guid to correlate different telemetry events for a 'session' of search/filtering, reset when new backend items are gathered for filtering
	FGuid FilterSessionCorrelationGuid;
	// In progress filter session telemetry - will be sent when filtering ends or user cancels or interacts with a partial filter set
	UE::Telemetry::ContentBrowser::FFrontendFilterTelemetry CurrentFrontendFilterTelemetry;
	
	// If there is in-progress filter telemetry, mark it as complete and send pending data 
	UE_API void OnCompleteFiltering(double InAmortizeDuration);
	// If there is in-progress filter telemetry, mark it as incomplete and send pending data 
	UE_API void OnInterruptFiltering();
	// Update telemetry for user interaction during an incomplete filtering
	UE_API void OnInteractDuringFiltering();

public:
	UE_API bool ShouldColumnGenerateWidget(const FString ColumnName) const;
};

#undef UE_API
