// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CollectionManagerTypes.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "CollectionManagerModule.h"

UE_DEPRECATED(5.6, "FSourcesData has been deprecated in favor of FAssetViewContentSources");
struct FSourcesData
{
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnEnumerateCustomSourceItemDatas, TFunctionRef<bool(class FContentBrowserItemData&&)>)

	TArray<FName> VirtualPaths;
	TArray<FCollectionNameType> Collections;
	FOnEnumerateCustomSourceItemDatas OnEnumerateCustomSourceItemDatas;
	bool bIncludeVirtualPaths = true;

	FSourcesData() = default;

	explicit FSourcesData(FName InVirtualPath)
	{
		VirtualPaths.Add(InVirtualPath);
	}

	explicit FSourcesData(FCollectionNameType InCollection)
	{
		Collections.Add(InCollection);
	}

	FSourcesData(TArray<FName> InVirtualPaths, TArray<FCollectionNameType> InCollections)
		: VirtualPaths(MoveTemp(InVirtualPaths))
		, Collections(MoveTemp(InCollections))
	{
	}

	FSourcesData(const FSourcesData& Other) = default;
	FSourcesData(FSourcesData&& Other) = default;

	FSourcesData& operator=(const FSourcesData& Other) = default;
	FSourcesData& operator=(FSourcesData&& Other) = default;

	FORCEINLINE bool IsEmpty() const
	{
		return VirtualPaths.Num() == 0 && Collections.Num() == 0;
	}

	FORCEINLINE bool HasVirtualPaths() const
	{
		return VirtualPaths.Num() > 0;
	}

	FORCEINLINE bool HasCollections() const
	{
		return Collections.Num() > 0;
	}

	FORCEINLINE bool IsIncludingVirtualPaths() const
	{
		return bIncludeVirtualPaths;
	}

	bool IsDynamicCollection() const
	{
		if ( Collections.Num() == 1 && FCollectionManagerModule::IsModuleAvailable() )
		{
			// Collection manager module should already be loaded since it may cause a hitch on the first search
			FCollectionManagerModule& CollectionManagerModule = FCollectionManagerModule::GetModule();

			ECollectionStorageMode::Type StorageMode = ECollectionStorageMode::Static;
			return (CollectionManagerModule.Get().GetProjectCollectionContainer()->GetCollectionStorageMode(
				Collections[0].Name, Collections[0].Type, StorageMode) && StorageMode == ECollectionStorageMode::Dynamic);
		}

		return false;
	}
};
