// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ContentBrowserItem.h"

#include "TextFilterKeyValueHandler.generated.h"

UCLASS(abstract, transient, MinimalAPI)
class UE_DEPRECATED(5.5, "This type has been replaced by IAssetTextFilterHandler which must be manually instantiated and registered and must be implemented in a threadsafe way.")
UTextFilterKeyValueHandler : public UObject
{
	GENERATED_BODY()
public:
	virtual bool HandleTextFilterKeyValue(const FContentBrowserItem& InContentBrowserItem, const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode, bool& bOutIsMatch) const { return false; }
};