// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ContentBrowserItem.h"

#include "TextFilterValueHandler.generated.h"

UCLASS(abstract, transient, MinimalAPI)
class UE_DEPRECATED(5.5, "This type has been replaced by IAssetTextFilterHandler which must be manually instantiated and registered and must be implemented in a threadsafe way.")
UTextFilterValueHandler : public UObject
{
	GENERATED_BODY()
public:
	virtual bool HandleTextFilterValue(const FContentBrowserItem& InContentBrowserItem, const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode, bool& bOutIsMatch) const { return false; }
};