// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

#define UE_API CONTENTBROWSER_API

/** Represents an Asset Type within a menu. */
class SAssetMenuIcon : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAssetMenuIcon)
		: _IconContainerSize(32.0f, 32.0f)
		, _IconSize(28.0f, 28.0f)
	{}
		SLATE_ARGUMENT(FVector2D, IconContainerSize)
		SLATE_ARGUMENT(FVector2D, IconSize)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs, const UClass* InAssetClass, const FName InIconOverride = NAME_None);
};

#undef UE_API
